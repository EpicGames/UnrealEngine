{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "AudioWidgets",
	"Description": "Collection of widgets tailored to interacting with audio-related data and systems.",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"SupportedPrograms": [ "UnrealInsights" ],
	"Modules": [
		{
			"Name": "AudioWidgetsCore",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "Default",
			"ProgramAllowList":
			[
				"UnrealInsights"
			]
		},
		{
			"Name": "AudioWidgets",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AudioWidgetsEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "AudioSynesthesia",
			"Enabled":  true
		}
	]
}