// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioWidgets : ModuleRules
{
	public AudioWidgets(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange
		(
			new string[]
			{
				"AudioAnalyzer",
				"AudioSynesthesia",
				"AudioWidgetsCore",
				"Core",
				"UMG"
			}
		);

		PrivateDependencyModuleNames.AddRange
		(
			new string[]
			{
				"AppFramework",
				"AudioMixer",
				"CoreUObject",
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore",
				"AdvancedWidgets",
				"SignalProcessing"
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange
			(
				new string[]
				{
					"UnrealEd",
				}
			);
		}

		bDisableAutoRTFMInstrumentation = true;
	}
}
