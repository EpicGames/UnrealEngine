// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioAnalyzerRack.h"

#include "AudioAnalyzerRackUnitRegistry.h"
#include "AudioMeter.h"
#include "AudioOscilloscope.h"
#include "AudioSpectrumAnalyzer.h"
#include "AudioWidgetsStyle.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "FAudioAnalyzerRack"

namespace AudioWidgets
{
	FAudioAnalyzerRack::FAudioAnalyzerRack(const FRackConstructParams& Params)
		: TabManagerLayoutName(Params.TabManagerLayoutName)
	{
		if (TabManagerLayoutName.IsNone())
		{
			TabManagerLayoutName = TEXT("AudioWidgets_FAudioAnalyzerRack_v0");
		}

		{
			// Append rack unit type names to the tab manager layout name.
			// When a new rack unit type becomes available a new default layout will be generated including the new rack unit.
			TStringBuilder<256> StringBuilder;
			StringBuilder.Append(TabManagerLayoutName.ToString());

			TArray<FName> RackUnitTypeNames;
			FAudioAnalyzerRackUnitRegistry::Get().GetRegisteredRackUnitTypeNames(RackUnitTypeNames);
			RackUnitTypeNames.Sort([](const FName& A, const FName& B) { return A.Compare(B) < 0; });
			for (FName RackUnitTypeName : RackUnitTypeNames)
			{
				StringBuilder << '_' << RackUnitTypeName;
			}

			TabManagerLayoutName = FName(StringBuilder);
		}

		ExcludedRackUnitTypes = Params.ExcludedRackUnitTypes;
		RackUnitConstructParams.StyleSet = (Params.StyleSet != nullptr) ? Params.StyleSet : &FAudioWidgetsStyle::Get();
		RackUnitConstructParams.EditorSettingsClass = Params.EditorSettingsClass;
	}

	FAudioAnalyzerRack::~FAudioAnalyzerRack()
	{
		if (TabManager.IsValid())
		{
			TabManager->UnregisterAllTabSpawners();
			TabManager->SetOnPersistLayout(nullptr);
			TabManager->CloseAllAreas();
		}
	}

	void FAudioAnalyzerRack::Init(int32 InNumChannels, Audio::FDeviceId InAudioDeviceId)
	{
		const EAudioBusChannels AudioBusChannels = AudioBusUtils::ConvertIntToEAudioBusChannels(InNumChannels);
		if (RackUnitConstructParams.AudioBusInfo.AudioDeviceId != InAudioDeviceId || !AudioBus.IsValid() || AudioBus->AudioBusChannels != AudioBusChannels)
		{
			// Create a UAudioBus with the required num channels:
			AudioBus = TStrongObjectPtr(NewObject<UAudioBus>());
			AudioBus->AudioBusChannels = AudioBusChannels;

			// Update cached AudioBusInfo:
			RackUnitConstructParams.AudioBusInfo = { .AudioDeviceId = InAudioDeviceId, .AudioBus = AudioBus.Get() };

			// Reinit any existing rack units:
			for (auto&& KVPair : RackUnits)
			{
				KVPair.Value->SetAudioBusInfo(RackUnitConstructParams.AudioBusInfo);
			}
		}
	}

	void FAudioAnalyzerRack::DestroyAnalyzers()
	{
		RackUnits.Reset();
	}

	TSharedRef<SWidget> FAudioAnalyzerRack::CreateWidget(TSharedRef<SDockTab> DockTab, const FSpawnTabArgs& SpawnTabArgs)
	{
		if (TabManager.IsValid())
		{
			TabManager->UnregisterAllTabSpawners();
			TabManager->SetOnPersistLayout(nullptr);
			TabManager->CloseAllAreas();
		}

		// Create a TabManager:
		TabManager = FGlobalTabmanager::Get()->NewTabManager(DockTab);
		TabManager->SetOnPersistLayout(FTabManager::FOnPersistLayout::CreateStatic(&FAudioAnalyzerRack::SaveTabLayout));

		// Register TabSpawners for all registered rack unit types:
		const TSharedRef<FWorkspaceItem> AppMenuGroup = TabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("AnalyzerRackGroupName", "Analyzers"));

		TArray<const FAudioAnalyzerRackUnitTypeInfo*> RackUnitTypes;
		FAudioAnalyzerRackUnitRegistry::Get().GetRegisteredRackUnitTypes(RackUnitTypes);
		
		if (!ExcludedRackUnitTypes.IsEmpty())
		{
			RackUnitTypes.RemoveAll([this](const FAudioAnalyzerRackUnitTypeInfo* Type) { return ExcludedRackUnitTypes.Contains(Type->TypeName); });
		}

		for (const FAudioAnalyzerRackUnitTypeInfo* RackUnitType : RackUnitTypes)
		{
			TabManager->RegisterTabSpawner(RackUnitType->TypeName, FOnSpawnTab::CreateLambda([this](const FSpawnTabArgs& Args)
				{
					const FName RackUnitTypeName = Args.GetTabId().TabType;
					if (TSharedRef<IAudioAnalyzerRackUnit>* ExistingRackUnit = RackUnits.Find(RackUnitTypeName))
					{
						return (*ExistingRackUnit)->SpawnTab(Args);
					}
					else
					{
						const TSharedRef<IAudioAnalyzerRackUnit> RackUnit = RackUnits.Add(RackUnitTypeName, MakeRackUnit(RackUnitTypeName));
						return RackUnit->SpawnTab(Args);
					}
				}))
				.SetGroup(AppMenuGroup)
				.SetDisplayName(RackUnitType->DisplayName)
				.SetIcon(RackUnitType->Icon)
				.SetCanSidebarTab(false);
		}

		// Create a ToolBar that can toggle visible analyzers:
		const TSharedRef<FUICommandList> CommandList = MakeShared<FUICommandList>();
		constexpr bool bForceSmallIcons = true;
		FSlimHorizontalToolBarBuilder ToolBarBuilder(CommandList, FMultiBoxCustomization::None, nullptr, bForceSmallIcons);

		const TSharedRef<FWorkspaceItem> LocalWorkspaceMenuRoot = TabManager->GetLocalWorkspaceMenuRoot();
		for (const TSharedRef<FWorkspaceItem>& WorkspaceGroup : LocalWorkspaceMenuRoot->GetChildItems())
		{
			ToolBarBuilder.AddComboButton(
				FUIAction(),
				FOnGetContent::CreateStatic(&FAudioAnalyzerRack::MakeVisibleAnalyzersMenu, CommandList, WorkspaceGroup.ToWeakPtr(), TabManager.ToWeakPtr()),
				WorkspaceGroup->GetDisplayName(),
				WorkspaceGroup->GetTooltipText(),
				WorkspaceGroup->GetIcon(),
				bForceSmallIcons);
		}

		// Load saved tab layout, or create the default layout:
		const TSharedRef<FTabManager::FLayout> TabLayout = LoadTabLayout();

		// Create the actual SWidget, containing the toolbar and the rack unit docking tab layout:
		const FLinearColor BackgroundColor = RackUnitConstructParams.StyleSet->GetColor("AudioAnalyzerRack.BackgroundColor");
		const TSharedRef<SWidget> Widget = SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(EHorizontalAlignment::HAlign_Right)
					[
						ToolBarBuilder.MakeWidget()
					]
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SOverlay)
					+ SOverlay::Slot()
					[
						SNew(SColorBlock)
							.Color(BackgroundColor)
					]
					+ SOverlay::Slot()
					[
						TabManager->RestoreFrom(TabLayout, SpawnTabArgs.GetOwnerWindow()).ToSharedRef()
					]
			];

		// If the dock tab that contains the analyzer rack is closed, also close any undocked analyzer rack units:
		DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateLambda([AnalyzerRackTabManager = TabManager.ToWeakPtr()](TSharedRef<SDockTab> AnalyzerRackDockTab)
			{
				if (TSharedPtr<FTabManager> PinnedTabManager = AnalyzerRackTabManager.Pin())
				{
					PinnedTabManager->SetOnPersistLayout(nullptr);
					PinnedTabManager->CloseAllAreas();
				}
			}));

		return Widget;
	}

	UAudioBus* FAudioAnalyzerRack::GetAudioBus() const
	{
		return AudioBus.Get();
	}

	void FAudioAnalyzerRack::StartProcessing()
	{
		for (auto&& KVPair : RackUnits)
		{
			KVPair.Value->StartProcessing();
		}

		bIsProcessingStarted = true;
	}

	void FAudioAnalyzerRack::StopProcessing()
	{
		for (auto&& KVPair : RackUnits)
		{
			KVPair.Value->StopProcessing();
		}

		bIsProcessingStarted = false;
	}

	TSharedRef<SWidget> FAudioAnalyzerRack::MakeVisibleAnalyzersMenu(TSharedRef<FUICommandList> InCommandList, TWeakPtr<FWorkspaceItem> InWorkspaceGroup, TWeakPtr<FTabManager> InTabManager)
	{
		// Create a menu, with toggle buttons for each rack unit type.
		FMenuBuilder MenuBuilder(true, InCommandList);

		if (TSharedPtr<FWorkspaceItem> WorkspaceGroup = InWorkspaceGroup.Pin())
		{
			// Find all rack unit types (implemented here generically using the registered tab spawners):
			for (const TSharedRef<FWorkspaceItem>& ChildItem : WorkspaceGroup->GetChildItems())
			{
				if (TSharedPtr<FTabSpawnerEntry> TabSpawnerEntry = ChildItem->AsSpawnerEntry())
				{
					if (TabSpawnerEntry->IsHidden())
					{
						continue;
					}

					const FName TabId = TabSpawnerEntry->GetFName();

					MenuBuilder.AddMenuEntry(
						TabSpawnerEntry->GetDisplayName(),
						TabSpawnerEntry->GetTooltipText(),
						TabSpawnerEntry->GetIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([InTabManager, TabId]()
								{
									if (TSharedPtr<FTabManager> PinnedTabManager = InTabManager.Pin())
									{
										TSharedPtr<SDockTab> LiveTab = PinnedTabManager->FindExistingLiveTab(TabId);
										if (!LiveTab.IsValid())
										{
											PinnedTabManager->TryInvokeTab(TabId);
										}
										else
										{
											LiveTab->RequestCloseTab();
										}
										PinnedTabManager->SavePersistentLayout();
									}
								}),
							FCanExecuteAction(),
							FIsActionChecked::CreateLambda([InTabManager, TabId]()
								{
									TSharedPtr<FTabManager> PinnedTabManager = InTabManager.Pin();
									return PinnedTabManager.IsValid() && PinnedTabManager->FindExistingLiveTab(TabId).IsValid();
								})
						),
						NAME_None,
						EUserInterfaceActionType::ToggleButton
					);
				}
			}
		}

		return MenuBuilder.MakeWidget();
	}

	void FAudioAnalyzerRack::SaveTabLayout(const TSharedRef<FTabManager::FLayout>& InLayout)
	{
		FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, InLayout);
	}

	TSharedRef<FTabManager::FLayout> FAudioAnalyzerRack::LoadTabLayout()
	{
		return FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, GetDefaultTabLayout());
	}

	TSharedRef<FTabManager::FLayout> FAudioAnalyzerRack::GetDefaultTabLayout()
	{
		TArray<const FAudioAnalyzerRackUnitTypeInfo*> RackUnitTypes;
		FAudioAnalyzerRackUnitRegistry::Get().GetRegisteredRackUnitTypes(RackUnitTypes);

		// Create the default layout primary area (can be overriden by derived class):
		TSharedRef<FTabManager::FArea> PrimaryArea = CreatePrimaryArea(RackUnitTypes);
		return FTabManager::NewLayout(TabManagerLayoutName)->AddArea(PrimaryArea);
	}

	TSharedRef<FTabManager::FArea> FAudioAnalyzerRack::CreatePrimaryArea(const TArray<const FAudioAnalyzerRackUnitTypeInfo*>& RackUnitTypes)
	{
		TSharedRef<FTabManager::FArea> PrimaryArea = FTabManager::NewPrimaryArea()
			->SetOrientation(EOrientation::Orient_Vertical);

		// Add entries for all registered rack units:
		for (const FAudioAnalyzerRackUnitTypeInfo* RackUnitType : RackUnitTypes)
		{
			// Set three known rack units visible by default:
			const bool bOpenTab =
				RackUnitType->TypeName == FAudioMeter::RackUnitTypeInfo.TypeName ||
				RackUnitType->TypeName == FAudioOscilloscope::RackUnitTypeInfo.TypeName ||
				RackUnitType->TypeName == FAudioSpectrumAnalyzer::RackUnitTypeInfo.TypeName;

			const ETabState::Type TabState = (bOpenTab) ? ETabState::OpenedTab : ETabState::ClosedTab;

			PrimaryArea->Split
			(
				FTabManager::NewStack()
				->SetSizeCoefficient(RackUnitType->VerticalSizeCoefficient)
				->SetHideTabWell(true)
				->AddTab(RackUnitType->TypeName, TabState)
			);
		}

		return PrimaryArea;
	}

	TSharedRef<IAudioAnalyzerRackUnit> FAudioAnalyzerRack::MakeRackUnit(FName RackUnitTypeName)
	{
		TSharedRef<IAudioAnalyzerRackUnit> RackUnit = FAudioAnalyzerRackUnitRegistry::Get().MakeRackUnit(RackUnitTypeName, RackUnitConstructParams);

		if (bIsProcessingStarted)
		{
			RackUnit->StartProcessing();
		}

		return RackUnit;
	}
}

#undef LOCTEXT_NAMESPACE
