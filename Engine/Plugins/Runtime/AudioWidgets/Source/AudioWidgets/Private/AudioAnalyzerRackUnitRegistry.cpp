// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioAnalyzerRackUnitRegistry.h"

#include "Misc/LazySingleton.h"

namespace AudioWidgets
{
	FAudioAnalyzerRackUnitRegistry& FAudioAnalyzerRackUnitRegistry::Get()
	{
		return TLazySingleton<FAudioAnalyzerRackUnitRegistry>::Get();
	}

	void FAudioAnalyzerRackUnitRegistry::TearDown()
	{
		TLazySingleton<FAudioAnalyzerRackUnitRegistry>::TearDown();
	}

	void FAudioAnalyzerRackUnitRegistry::RegisterRackUnitType(const FAudioAnalyzerRackUnitTypeInfo* RackUnitTypeInfo)
	{
		RegisteredRackUnitTypes.Add(RackUnitTypeInfo->TypeName, RackUnitTypeInfo);
	}

	TSharedRef<IAudioAnalyzerRackUnit> FAudioAnalyzerRackUnitRegistry::MakeRackUnit(FName RackUnitTypeName, const FAudioAnalyzerRackUnitConstructParams& Params)
	{
		const FAudioAnalyzerRackUnitTypeInfo* RegisteredRackUnitTypeInfo = RegisteredRackUnitTypes.FindChecked(RackUnitTypeName);
		return RegisteredRackUnitTypeInfo->OnMakeAudioAnalyzerRackUnit.Execute(Params);
	}

	void FAudioAnalyzerRackUnitRegistry::GetRegisteredRackUnitTypes(TArray<const FAudioAnalyzerRackUnitTypeInfo*>& OutArray) const
	{
		RegisteredRackUnitTypes.GenerateValueArray(OutArray);
	}

	void FAudioAnalyzerRackUnitRegistry::GetRegisteredRackUnitTypeNames(TArray<FName>& OutArray) const
	{
		RegisteredRackUnitTypes.GenerateKeyArray(OutArray);
	}
}
