// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioAnalyzerRack.h"

namespace AudioWidgets
{
	class FAudioAnalyzerRackUnitRegistry
	{
	public:
		static FAudioAnalyzerRackUnitRegistry& Get();
		static void TearDown();

		void RegisterRackUnitType(const FAudioAnalyzerRackUnitTypeInfo* RackUnitTypeInfo);

		TSharedRef<IAudioAnalyzerRackUnit> MakeRackUnit(FName RackUnitTypeName, const FAudioAnalyzerRackUnitConstructParams& Params);

		void GetRegisteredRackUnitTypes(TArray<const FAudioAnalyzerRackUnitTypeInfo*>& OutArray) const;
		void GetRegisteredRackUnitTypeNames(TArray<FName>& OutArray) const;

	private:
		TMap<FName, const FAudioAnalyzerRackUnitTypeInfo*> RegisteredRackUnitTypes;
	};
}
