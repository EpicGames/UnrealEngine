// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioColorMapper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioColorMapper)

FColor FAudioColorMapper::GetColorFromValue(const float Value) const
{
	const float ScaledValue = ValueScaling * (Value - MinValue);
	const float ScaledClampedValue = FMath::Clamp(ScaledValue, 0.0f, 1.0f);
	switch (ColorMap)
	{
	default:
	case EAudioColorGradient::BlackToWhite:
		return ColorMapBlackToWhite(ScaledClampedValue);
	case EAudioColorGradient::WhiteToBlack:
		return ColorMapWhiteToBlack(ScaledClampedValue);
	}
}

FColor FAudioColorMapper::ColorMapBlackToWhite(const float T)
{
	const uint8 Quantized = FColor::QuantizeUNormFloatTo8(T);
	return FColor(Quantized, Quantized, Quantized);
}

FColor FAudioColorMapper::ColorMapWhiteToBlack(const float T)
{
	const uint8 Quantized = FColor::QuantizeUNormFloatTo8(1.0f - T);
	return FColor(Quantized, Quantized, Quantized);
}

