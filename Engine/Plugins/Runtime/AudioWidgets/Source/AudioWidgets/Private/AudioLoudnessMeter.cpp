// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioLoudnessMeter.h"

#include "AudioMeterTypes.h"
#include "LoudnessMeterRackUnitSettings.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "FAudioLoudnessMeter"

namespace AudioWidgets
{
	/**
	 * Light wrapper for accessing settings for analyzer rack units. Can be passed by value.
	 */
	template<class SettingsT>
	class TRackUnitSettingsHelper
	{
	public:
		TRackUnitSettingsHelper(const FProperty& InSettingsProperty)
			: SettingsProperty(InSettingsProperty)
		{
		}

		SettingsT& GetRackUnitSettings() const
		{
			UObject* EditorSettingsObject = GetEditorSettingsObject();
			return *(SettingsProperty.ContainerPtrToValuePtr<SettingsT>(EditorSettingsObject));
		}

		void SaveConfig() const
		{
			GetEditorSettingsObject()->SaveConfig();
		}

	private:
		UObject* GetEditorSettingsObject() const
		{
			return SettingsProperty.GetOwnerClass()->GetDefaultObject();
		}

		const FProperty& SettingsProperty;
	};

	const FAudioAnalyzerRackUnitTypeInfo FAudioLoudnessMeter::RackUnitTypeInfo
	{
		.TypeName = TEXT("FAudioLoudnessMeter"),
		.DisplayName = LOCTEXT("AudioLoudnessMeterDisplayName", "Loudness"),
		.OnMakeAudioAnalyzerRackUnit = FOnMakeAudioAnalyzerRackUnit::CreateStatic(&MakeRackUnit),
		.VerticalSizeCoefficient = 0.5f,
	};

	FAudioLoudnessMeter::FAudioLoudnessMeter(const FAudioLoudnessMeterParams& Params)
	{
		Init(Params.NumChannels, Params.AudioDeviceId, Params.ExternalAudioBus);
	}

	FAudioLoudnessMeter::~FAudioLoudnessMeter()
	{
		if (WidgetRefreshTicker.IsValid())
		{
			FTSTicker::RemoveTicker(WidgetRefreshTicker);
			WidgetRefreshTicker.Reset();
		}

		Teardown();
	}

	void FAudioLoudnessMeter::SetAudioBusInfo(const FAudioBusInfo& AudioBusInfo)
	{
		Init(AudioBusInfo.AudioBus->GetNumChannels(), AudioBusInfo.AudioDeviceId, AudioBusInfo.AudioBus);
	}

	TSharedRef<SDockTab> FAudioLoudnessMeter::SpawnTab(const FSpawnTabArgs& Args) const
	{
		return SNew(SDockTab)
			.Clipping(EWidgetClipping::ClipToBounds)
			.Label(RackUnitTypeInfo.DisplayName)
			[
				LoudnessMeterWidgetView.MakeWidget()
			];
	}

	TSharedRef<IAudioAnalyzerRackUnit> FAudioLoudnessMeter::MakeRackUnit(const FAudioAnalyzerRackUnitConstructParams& Params)
	{
		const FAudioLoudnessMeterParams AnalyzerParams
		{
			.NumChannels = Params.AudioBusInfo.GetNumChannels(),
			.AudioDeviceId = Params.AudioBusInfo.AudioDeviceId,
			.ExternalAudioBus = Params.AudioBusInfo.AudioBus,
		};

		TSharedRef<FAudioLoudnessMeter> AudioLoudnessMeter = MakeShared<FAudioLoudnessMeter>(AnalyzerParams);
		AudioLoudnessMeter->InitLoudnessMeterWidgetView(Params);

		return AudioLoudnessMeter;
	}

	void FAudioLoudnessMeter::Init(const int32 InNumChannels, const Audio::FDeviceId InAudioDeviceId, const TObjectPtr<UAudioBus> InExternalAudioBus)
	{
		Teardown();

		LKFSSettings = TStrongObjectPtr(NewObject<ULKFSSettings>());
		LKFSSettings->IntegratedLoudnessDuration = 1800.0f; // 30 minutes
		LKFSSettings->bCalculateOverallLoudnessRange = true;

		TruePeakSettings = TStrongObjectPtr(NewObject<UMeterSettings>());
		TruePeakSettings->PeakMode = EMeterPeakType::TruePeak;
		TruePeakSettings->bIsAnalog = false;
		TruePeakSettings->MeterAttackTime = 0;
		TruePeakSettings->MeterReleaseTime = 0;

		AudioDeviceId = InAudioDeviceId;

		// Only create analyzers etc if we have an audio device:
		if (InAudioDeviceId != FAudioBusInfo::InvalidAudioDeviceId)
		{
			check(InNumChannels > 0);

			bUseExternalAudioBus = (InExternalAudioBus != nullptr);
			AudioBus = (bUseExternalAudioBus) ? TStrongObjectPtr(InExternalAudioBus.Get()) : TStrongObjectPtr(NewObject<UAudioBus>());
			AudioBus->AudioBusChannels = EAudioBusChannels(InNumChannels - 1);

			CreateLKFSAnalyzer();
			CreateTruePeakAnalyzer();

			LKFSAnalyzer->StartAnalyzing(InAudioDeviceId, AudioBus.Get());
			TruePeakAnalyzer->StartAnalyzing(InAudioDeviceId, AudioBus.Get());
		}
	}

	void FAudioLoudnessMeter::Teardown()
	{
		if (UObjectInitialized())
		{
			if (TruePeakAnalyzer.IsValid() && TruePeakAnalyzer->IsValidLowLevel())
			{
				TruePeakAnalyzer->StopAnalyzing();
				ReleaseTruePeakAnalyzer();
			}

			if (LKFSAnalyzer.IsValid() && LKFSAnalyzer->IsValidLowLevel())
			{
				LKFSAnalyzer->StopAnalyzing();
				ReleaseLKFSAnalyzer();
			}
		}

		PerChannelMeterResultsNativeDelegateHandle.Reset();
		TruePeakAnalyzer.Reset();

		LatestOverallLKFSResultsDelegateHandle.Reset();
		LKFSAnalyzer.Reset();

		AudioBus.Reset();
		TruePeakSettings.Reset();
		LKFSSettings.Reset();
		bUseExternalAudioBus = false;
	}

	void FAudioLoudnessMeter::CreateLKFSAnalyzer()
	{
		ensure(!LKFSAnalyzer.IsValid());
		ensure(!LatestOverallLKFSResultsDelegateHandle.IsValid());

		LKFSAnalyzer = TStrongObjectPtr(NewObject<ULKFSAnalyzer>());
		LKFSAnalyzer->Settings = LKFSSettings.Get();
		LatestOverallLKFSResultsDelegateHandle = LKFSAnalyzer->OnLatestOverallLKFSResultsNative.AddRaw(this, &FAudioLoudnessMeter::OnLoudnessOutput);
	}

	void FAudioLoudnessMeter::ReleaseLKFSAnalyzer()
	{
		if (ensure(LKFSAnalyzer.IsValid() && LatestOverallLKFSResultsDelegateHandle.IsValid()))
		{
			LKFSAnalyzer->OnLatestOverallLKFSResultsNative.Remove(LatestOverallLKFSResultsDelegateHandle);
		}

		LatestOverallLKFSResultsDelegateHandle.Reset();
		LKFSAnalyzer.Reset();
	}

	void FAudioLoudnessMeter::ResetLKFSAnalyzer()
	{
		// Reset loudness analyzer
		if (LKFSAnalyzer.IsValid())
		{
			LKFSAnalyzer->StopAnalyzing();
			ReleaseLKFSAnalyzer();
		}

		LatestLoudnessResults.Reset();

		LoudnessMeterMaxResults.ShortTermLoudness = MIN_VOLUME_DECIBELS;
		LoudnessMeterMaxResults.MomentaryLoudness = MIN_VOLUME_DECIBELS;

		// Recreate loudness analyzer
		if (AudioDeviceId != FAudioBusInfo::InvalidAudioDeviceId)
		{
			CreateLKFSAnalyzer();
			LKFSAnalyzer->StartAnalyzing(AudioDeviceId, AudioBus.Get());
		}
	}

	void FAudioLoudnessMeter::CreateTruePeakAnalyzer()
	{
		ensure(!TruePeakAnalyzer.IsValid());
		ensure(!PerChannelMeterResultsNativeDelegateHandle.IsValid());

		TruePeakAnalyzer = TStrongObjectPtr(NewObject<UMeterAnalyzer>());
		TruePeakAnalyzer->Settings = TruePeakSettings.Get();
		PerChannelMeterResultsNativeDelegateHandle = TruePeakAnalyzer->OnPerChannelMeterResultsNative.AddRaw(this, &FAudioLoudnessMeter::OnTruePeakOutput);
	}

	void FAudioLoudnessMeter::ReleaseTruePeakAnalyzer()
	{
		if (ensure(TruePeakAnalyzer.IsValid() && PerChannelMeterResultsNativeDelegateHandle.IsValid()))
		{
			TruePeakAnalyzer->OnPerChannelMeterResultsNative.Remove(PerChannelMeterResultsNativeDelegateHandle);
		}

		PerChannelMeterResultsNativeDelegateHandle.Reset();
		TruePeakAnalyzer.Reset();
	}

	void FAudioLoudnessMeter::ResetTruePeakAnalyzer()
	{
		// Reset true peak analyzer
		if (TruePeakAnalyzer.IsValid())
		{
			TruePeakAnalyzer->StopAnalyzing();
			ReleaseTruePeakAnalyzer();
		}

		LatestTruePeakResults.Reset();

		LoudnessMeterMaxResults.TruePeakDb = MIN_VOLUME_DECIBELS;

		// Recreate true peak analyzer
		if (AudioDeviceId != FAudioBusInfo::InvalidAudioDeviceId)
		{
			CreateTruePeakAnalyzer();
			TruePeakAnalyzer->StartAnalyzing(AudioDeviceId, AudioBus.Get());
		}
	}

	void FAudioLoudnessMeter::InitLoudnessMeterWidgetView(const FAudioAnalyzerRackUnitConstructParams& Params)
	{
		using namespace AudioWidgetsCore;

		// Initializing these callbacks outside of the FAudioLoudnessMeter constructor allows us to use the CreateSP(...) smart pointer variants for attributes/delegates:

		FLoudnessMeterWidgetView::FTimerPanelParams TimerPanelParams
		{
			.AnalysisTime = TAttribute<FTimespan>::CreateSP(this, &FAudioLoudnessMeter::GetAnalysisTime),
			.OnResetButtonClicked = FOnClicked::CreateSP(this, &FAudioLoudnessMeter::HandleResetButtonClicked),
			.bIsVisible = true
		};

		FLoudnessMeterWidgetView::FLoudnessScaleParams LoudnessScaleParams;
		FLoudnessMeterWidgetView::FDisplayOrderParams ValuesDisplayOrderParams;
		FLoudnessMeterWidgetView::FDisplayOrderParams MetersDisplayOrderParams;

		FLoudnessMeterWidgetView::FLoudnessMetric LoudnessMetrics[] =
		{
			{
				.Name = GET_MEMBER_NAME_CHECKED(FLoudnessMeterRackUnitSettings, LongTermLoudness),
				.DisplayName = LOCTEXT("IntegratedLoudnessDisplayName", "Integrated"),
				.MeterMetric = EAudioMeterMetric::Loudness,
				.Value = TAttribute<TOptional<float>>::CreateSPLambda(this, [this]() { return (LatestLoudnessResults.IsSet()) ? TOptional<float>(LatestLoudnessResults->GatedLoudness) : NullOpt; }),
				.bShowValue = true,
				.bShowMeter = false,
			},
			{
				.Name = GET_MEMBER_NAME_CHECKED(FLoudnessMeterRackUnitSettings, ShortTermLoudness),
				.DisplayName = LOCTEXT("ShortTermLoudnessDisplayName", "Short Term"),
				.MeterMetric = EAudioMeterMetric::Loudness,
				.Value = TAttribute<TOptional<float>>::CreateSPLambda(this, [this]() { return (LatestLoudnessResults.IsSet()) ? TOptional<float>(LatestLoudnessResults->ShortTermLoudness) : NullOpt; }),
				.MaxValue = TAttribute<TOptional<float>>::CreateSPLambda(this, [this]()	{ return LoudnessMeterMaxResults.ShortTermLoudness;	}),
				.bShowValue = false,
				.bShowMeter = true,
				.bHoldMaxForValue = false,
				.bHoldMaxForMeter = false,
				.OnClicked = FSimpleDelegate::CreateSPLambda(this, [this]() { LoudnessMeterMaxResults.ShortTermLoudness = MIN_VOLUME_DECIBELS; }),
			},
			{
				.Name = GET_MEMBER_NAME_CHECKED(FLoudnessMeterRackUnitSettings, MomentaryLoudness),
				.DisplayName = LOCTEXT("MomentaryLoudnessDisplayName", "Momentary"),
				.MeterMetric = EAudioMeterMetric::Loudness,
				.Value = TAttribute<TOptional<float>>::CreateSPLambda(this, [this]() { return (LatestLoudnessResults.IsSet()) ? TOptional<float>(LatestLoudnessResults->Loudness) : NullOpt; }),
				.MaxValue = TAttribute<TOptional<float>>::CreateSPLambda(this, [this]() { return LoudnessMeterMaxResults.MomentaryLoudness; }),
				.bShowValue = false,
				.bShowMeter = true,
				.bHoldMaxForValue = false,
				.bHoldMaxForMeter = false,
				.OnClicked = FSimpleDelegate::CreateSPLambda(this, [this]() { LoudnessMeterMaxResults.MomentaryLoudness = MIN_VOLUME_DECIBELS; }),
			},
			{
				.Name = GET_MEMBER_NAME_CHECKED(FLoudnessMeterRackUnitSettings, LoudnessRange),
				.DisplayName = LOCTEXT("LoudnessRangeDisplayName", "Range"),
				.MeterMetric = EAudioMeterMetric::Loudness,
				.Range = TAttribute<TOptional<FFloatInterval>>::CreateSPLambda(this, [this]() { return (LatestLoudnessResults.IsSet()) ? LatestLoudnessResults->LoudnessRange : NullOpt; }),
				.bShowValue = true,
				.bShowMeter = false,
			},
			{
				.Name = GET_MEMBER_NAME_CHECKED(FLoudnessMeterRackUnitSettings, TruePeak),
				.DisplayName = LOCTEXT("TruePeakDisplayName", "True Peak"),
				.MeterMetric = EAudioMeterMetric::Decibels,
				.Value = TAttribute<TOptional<float>>::CreateSPLambda(this, [this]() { return (LatestTruePeakResults.IsSet()) ? TOptional<float>(LatestTruePeakResults->MeterValue) : NullOpt; }),
				.MaxValue = TAttribute<TOptional<float>>::CreateSPLambda(this, [this]() { return LoudnessMeterMaxResults.TruePeakDb; }),
				.bShowValue = true,
				.bShowMeter = false,
				.bHoldMaxForValue = true,
				.bHoldMaxForMeter = true,
				.OnClicked = FSimpleDelegate::CreateSPLambda(this, [this]() { LoudnessMeterMaxResults.TruePeakDb = MIN_VOLUME_DECIBELS; })
			}
		};

		if (Params.EditorSettingsClass != nullptr)
		{
			// If we have been given a valid editor settings class, bind analyzer options to the settings:
			if (const FProperty* LoudnessMeterSettingsProperty = Params.EditorSettingsClass->FindPropertyByName("LoudnessMeterSettings"))
			{
				const TRackUnitSettingsHelper<FLoudnessMeterRackUnitSettings> SettingsHelper(*LoudnessMeterSettingsProperty);

				// Migrate loudness meter default colors (v0 -> v1)
				{
					FLoudnessMeterRackUnitSettings& RackSettings = SettingsHelper.GetRackUnitSettings();

					if (RackSettings.ConfigVersion < 1)
					{
						RackSettings.LoudnessRange.Color = FLoudnessMeterSettings::DefaultLoudnessRangeColor;
						RackSettings.TruePeak.Color = FLoudnessMeterSettings::DefaultTruePeakColor;
						RackSettings.ConfigVersion = 1;
						SettingsHelper.SaveConfig();
					}
				}

				TimerPanelParams.bIsVisible.BindLambda([=]() { return SettingsHelper.GetRackUnitSettings().bDisplayAnalysisTimer; });
				TimerPanelParams.OnVisibilityToggleRequested.BindLambda([=]()
					{
						FLoudnessMeterRackUnitSettings& LoudnessMeterSettings = SettingsHelper.GetRackUnitSettings();
						LoudnessMeterSettings.bDisplayAnalysisTimer = !LoudnessMeterSettings.bDisplayAnalysisTimer;
						SettingsHelper.SaveConfig();
					});

				LoudnessScaleParams.Range.BindLambda([=]() { return SettingsHelper.GetRackUnitSettings().LoudnessScaleRange; });
				LoudnessScaleParams.OnRangeValueChanged.BindLambda([=](int32 Value) { SettingsHelper.GetRackUnitSettings().LoudnessScaleRange = Value; });
				LoudnessScaleParams.OnRangeValueCommitted.BindLambda([=](int32 Value, ETextCommit::Type CommitType)
					{
						SettingsHelper.GetRackUnitSettings().LoudnessScaleRange = Value;
						SettingsHelper.SaveConfig();
					});

				LoudnessScaleParams.Offset.BindLambda([=]() { return SettingsHelper.GetRackUnitSettings().LoudnessScaleOffset; });
				LoudnessScaleParams.OnOffsetValueChanged.BindLambda([=](int32 Value) { SettingsHelper.GetRackUnitSettings().LoudnessScaleOffset = Value; });
				LoudnessScaleParams.OnOffsetValueCommitted.BindLambda([=](int32 Value, ETextCommit::Type CommitType)
					{
						SettingsHelper.GetRackUnitSettings().LoudnessScaleOffset = Value;
						SettingsHelper.SaveConfig();
					});

				LoudnessScaleParams.Target.BindLambda([=]() { return SettingsHelper.GetRackUnitSettings().LoudnessScaleTarget; });
				LoudnessScaleParams.OnTargetValueChanged.BindLambda([=](int32 Value) { SettingsHelper.GetRackUnitSettings().LoudnessScaleTarget = Value; });
				LoudnessScaleParams.OnTargetValueCommitted.BindLambda([=](int32 Value, ETextCommit::Type CommitType)
					{
						SettingsHelper.GetRackUnitSettings().LoudnessScaleTarget = Value;
						SettingsHelper.SaveConfig();
					});

				LoudnessScaleParams.TruePeakLimit.BindLambda([=]() { return SettingsHelper.GetRackUnitSettings().TruePeakLimit; });
				LoudnessScaleParams.OnTruePeakLimitValueChanged.BindLambda([=](float Value) { SettingsHelper.GetRackUnitSettings().TruePeakLimit = Value; });
				LoudnessScaleParams.OnTruePeakLimitValueCommitted.BindLambda([=](float Value, ETextCommit::Type CommitType)
					{
						SettingsHelper.GetRackUnitSettings().TruePeakLimit = Value;
						SettingsHelper.SaveConfig();
					});

				LoudnessScaleParams.TargetColor.BindLambda([=]() { return SettingsHelper.GetRackUnitSettings().TargetColor; });
				LoudnessScaleParams.OnTargetColorChanged.BindLambda([=](FLinearColor NewColor)
				{
					SettingsHelper.GetRackUnitSettings().TargetColor = NewColor;
					SettingsHelper.SaveConfig();
				});
				LoudnessScaleParams.OnTargetColorPickerWindowClosed.BindLambda([=](const TSharedRef<SWindow>& Window) { SettingsHelper.SaveConfig(); });
				LoudnessScaleParams.OnResetTargetColorRequested.BindLambda([=]()
				{
					SettingsHelper.GetRackUnitSettings().TargetColor = FLoudnessMeterWidgetView::GetDefaultTargetColor();
					SettingsHelper.SaveConfig();
				});

				LoudnessScaleParams.bRelativeScale.BindLambda([=]() { return SettingsHelper.GetRackUnitSettings().bUseRelativeLoudnessScale; });
				LoudnessScaleParams.OnRelativeScaleToggleRequested.BindLambda([=]()
					{
						FLoudnessMeterRackUnitSettings& LoudnessMeterSettings = SettingsHelper.GetRackUnitSettings();
						LoudnessMeterSettings.bUseRelativeLoudnessScale = !LoudnessMeterSettings.bUseRelativeLoudnessScale;
						SettingsHelper.SaveConfig();
					});

				ValuesDisplayOrderParams.PermutationIndex.BindLambda([=]() { return SettingsHelper.GetRackUnitSettings().ValuesOrderingPermutation; });
				ValuesDisplayOrderParams.OnPermutationChanged.BindLambda([=](int32 NewPermutationIndex)
					{
						SettingsHelper.GetRackUnitSettings().ValuesOrderingPermutation = NewPermutationIndex;
						SettingsHelper.SaveConfig();
					});

				MetersDisplayOrderParams.PermutationIndex.BindLambda([=]() { return SettingsHelper.GetRackUnitSettings().MetersOrderingPermutation; });
				MetersDisplayOrderParams.OnPermutationChanged.BindLambda([=](int32 NewPermutationIndex)
					{
						SettingsHelper.GetRackUnitSettings().MetersOrderingPermutation = NewPermutationIndex;
						SettingsHelper.SaveConfig();
					});

				for (FLoudnessMeterWidgetView::FLoudnessMetric& LoudnessMetric : LoudnessMetrics)
				{
					if (const FProperty* DisplayOptionsProperty = FLoudnessMeterRackUnitSettings::StaticStruct()->FindPropertyByName(LoudnessMetric.Name))
					{
						const auto GetLoudnessMetricDisplayOptions = [DisplayOptionsProperty](FLoudnessMeterRackUnitSettings& RackUnitSettings)
							{
								return DisplayOptionsProperty->ContainerPtrToValuePtr<FLoudnessMetricDisplayOptions>(&RackUnitSettings);
							};

						LoudnessMetric.bShowValue.BindLambda([=]()
							{
								const FLoudnessMetricDisplayOptions* DisplayOptions = GetLoudnessMetricDisplayOptions(SettingsHelper.GetRackUnitSettings());
								return DisplayOptions->bShowValue;
							});

						LoudnessMetric.bShowMeter.BindLambda([=]()
							{
								const FLoudnessMetricDisplayOptions* DisplayOptions = GetLoudnessMetricDisplayOptions(SettingsHelper.GetRackUnitSettings());
								return DisplayOptions->bShowMeter;
							});

						LoudnessMetric.bHoldMaxForValue.BindLambda([=]()
							{
								const FLoudnessMetricDisplayOptions* DisplayOptions = GetLoudnessMetricDisplayOptions(SettingsHelper.GetRackUnitSettings());
								return DisplayOptions->bHoldMaxForValue;
							});

						LoudnessMetric.bHoldMaxForMeter.BindLambda([=]()
							{
								const FLoudnessMetricDisplayOptions* DisplayOptions = GetLoudnessMetricDisplayOptions(SettingsHelper.GetRackUnitSettings());
								return DisplayOptions->bHoldMaxForMeter;
							});

						LoudnessMetric.OnShowValueToggleRequested.BindLambda([=]()
							{
								FLoudnessMetricDisplayOptions* DisplayOptions = GetLoudnessMetricDisplayOptions(SettingsHelper.GetRackUnitSettings());
								DisplayOptions->bShowValue = !DisplayOptions->bShowValue;
								SettingsHelper.SaveConfig();
							});

						LoudnessMetric.OnShowMeterToggleRequested.BindLambda([=]()
							{
								FLoudnessMetricDisplayOptions* DisplayOptions = GetLoudnessMetricDisplayOptions(SettingsHelper.GetRackUnitSettings());
								DisplayOptions->bShowMeter = !DisplayOptions->bShowMeter;
								SettingsHelper.SaveConfig();
							});

						LoudnessMetric.OnShowValueToggleFromDragDropRequested.BindLambda([=]()
							{
								FLoudnessMetricDisplayOptions* DisplayOptions = GetLoudnessMetricDisplayOptions(SettingsHelper.GetRackUnitSettings());
								DisplayOptions->bShowValue = !DisplayOptions->bShowValue;

								if (DisplayOptions->bShowValue)
								{
									DisplayOptions->bHoldMaxForValue = DisplayOptions->bHoldMaxForMeter;
								}

								SettingsHelper.SaveConfig();
							});

						LoudnessMetric.OnShowMeterToggleFromDragDropRequested.BindLambda([=]()
							{
								FLoudnessMetricDisplayOptions* DisplayOptions = GetLoudnessMetricDisplayOptions(SettingsHelper.GetRackUnitSettings());
								DisplayOptions->bShowMeter = !DisplayOptions->bShowMeter;

								if (DisplayOptions->bShowMeter)
								{
									DisplayOptions->bHoldMaxForMeter = DisplayOptions->bHoldMaxForValue;
								}

								SettingsHelper.SaveConfig();
							});

						LoudnessMetric.OnHoldMaxForValueToggleRequested.BindLambda([=, this]()
							{
								FLoudnessMetricDisplayOptions* DisplayOptions = GetLoudnessMetricDisplayOptions(SettingsHelper.GetRackUnitSettings());
								DisplayOptions->bHoldMaxForValue = !DisplayOptions->bHoldMaxForValue;
								SettingsHelper.SaveConfig();
							});

						LoudnessMetric.OnHoldMaxForMeterToggleRequested.BindLambda([=, this]()
							{
								FLoudnessMetricDisplayOptions* DisplayOptions = GetLoudnessMetricDisplayOptions(SettingsHelper.GetRackUnitSettings());
								DisplayOptions->bHoldMaxForMeter = !DisplayOptions->bHoldMaxForMeter;
								SettingsHelper.SaveConfig();
							});

						LoudnessMetric.Color.BindLambda([=]()
							{
								const FLoudnessMetricDisplayOptions* DisplayOptions = GetLoudnessMetricDisplayOptions(SettingsHelper.GetRackUnitSettings());
								return DisplayOptions->Color;
							});

						LoudnessMetric.OnColorChanged.BindLambda([=](FLinearColor NewColor)
							{
								FLoudnessMetricDisplayOptions* DisplayOptions = GetLoudnessMetricDisplayOptions(SettingsHelper.GetRackUnitSettings());
								DisplayOptions->Color = NewColor;
							});

						LoudnessMetric.OnColorPickerWindowClosed.BindLambda([=](const TSharedRef<SWindow>& Window) { SettingsHelper.SaveConfig(); });

						LoudnessMetric.OnResetColorRequested.BindLambda([=, MetricName = LoudnessMetric.Name]()
							{
								FLoudnessMetricDisplayOptions* DisplayOptions = GetLoudnessMetricDisplayOptions(SettingsHelper.GetRackUnitSettings());
								DisplayOptions->Color = FLoudnessMeterWidgetView::GetDefaultMeterColor(MetricName);
								SettingsHelper.SaveConfig();
							});
					}
				}

				// We just poll for visible loudness metrics:
				WidgetRefreshTicker = FTSTicker::GetCoreTicker().AddTicker(TEXT("FAudioLoudnessMeter::WidgetRefreshTicker"), 0.0f, [this](float DeltaTime)
					{
						LoudnessMeterWidgetView.RefreshVisibleLoudnessMetrics();
						return true;
					});
			}
		}

		LoudnessMeterWidgetView.InitTimerPanel(TimerPanelParams);
		LoudnessMeterWidgetView.InitLoudnessScale(LoudnessScaleParams);
		LoudnessMeterWidgetView.InitValuesDisplayOrder(ValuesDisplayOrderParams);
		LoudnessMeterWidgetView.InitMetersDisplayOrder(MetersDisplayOrderParams);
		for (const FLoudnessMeterWidgetView::FLoudnessMetric& LoudnessMetric : LoudnessMetrics)
		{
			LoudnessMeterWidgetView.AddLoudnessMetric(LoudnessMetric);
		}
	}

	void FAudioLoudnessMeter::OnLoudnessOutput(ULKFSAnalyzer* InLKFSAnalyzer, const FLKFSResults& InLoudnessResults)
	{
		if (InLKFSAnalyzer == LKFSAnalyzer.Get())
		{
			LatestLoudnessResults = InLoudnessResults;

			LoudnessMeterMaxResults.ShortTermLoudness = FMath::Max<float>(LoudnessMeterMaxResults.ShortTermLoudness, InLoudnessResults.ShortTermLoudness);
			LoudnessMeterMaxResults.MomentaryLoudness = FMath::Max<float>(LoudnessMeterMaxResults.MomentaryLoudness, InLoudnessResults.Loudness);
		}
	}

	void FAudioLoudnessMeter::OnTruePeakOutput(UMeterAnalyzer* InMeterAnalyzer, int32 InChannelIndex, const TArray<FMeterResults>& InMeterResultsArray)
	{
		if (InMeterAnalyzer != TruePeakAnalyzer.Get())
		{
			return;
		}

		const FMeterResults* MaxResult = Algo::MaxElementBy(InMeterResultsArray, &FMeterResults::MeterValue);
		if (!MaxResult)
		{
			return;
		}

		if (InChannelIndex == 0)
		{
			LatestTruePeakResults = *MaxResult;
		}
		else if (MaxResult->MeterValue > LatestTruePeakResults->MeterValue)
		{
			LatestTruePeakResults = *MaxResult;
		}

		LoudnessMeterMaxResults.TruePeakDb = FMath::Max(LoudnessMeterMaxResults.TruePeakDb, MaxResult->MeterValue);
	}

	FTimespan FAudioLoudnessMeter::GetAnalysisTime() const
	{
		if (LatestLoudnessResults.IsSet() && LKFSSettings.IsValid())
		{
			const float AnalysisTime = FMath::Min(LatestLoudnessResults->Timestamp, LKFSSettings->IntegratedLoudnessDuration);
			return FTimespan::FromSeconds(AnalysisTime);
		}
	
		return FTimespan::Zero();
	}

	FReply FAudioLoudnessMeter::HandleResetButtonClicked()
	{
		ResetLKFSAnalyzer();
		ResetTruePeakAnalyzer();

		return FReply::Handled();
	}
} // namespace AudioWidgets

#undef LOCTEXT_NAMESPACE
