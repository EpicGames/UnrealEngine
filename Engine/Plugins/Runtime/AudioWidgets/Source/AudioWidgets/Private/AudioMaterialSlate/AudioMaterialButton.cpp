// Copyright Epic Games, Inc. All Rights Reserved.


#include "AudioMaterialSlate/AudioMaterialButton.h"
#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "AudioMaterialSlate/SAudioMaterialButton.h"
#include "AudioWidgetsStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioMaterialButton)

#define LOCTEXT_NAMESPACE "AudioWidgets"

UAudioMaterialButton::UAudioMaterialButton()
{
	//get default style
	WidgetStyle = FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioMaterialButtonStyle>("AudioMaterialButton.Style");
}

#if WITH_EDITOR
const FText UAudioMaterialButton::GetPaletteCategory()
{
	return LOCTEXT("PaletteCategory", "AudioMaterial");
}
#endif // WITH_EDITOR

void UAudioMaterialButton::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!Button.IsValid())
	{
		return;
	}

	Button->SetPressedState(bIsPressed);
	Button->ApplyNewMaterial();
}

void UAudioMaterialButton::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);
	Button.Reset();
}

bool UAudioMaterialButton::GetIsPressed() const
{
	return bIsPressed;
}

void UAudioMaterialButton::SetIsPressed(bool InPressed)
{
	if (!Button.IsValid())
	{
		return;
	}

	if (bIsPressed != InPressed)
	{
		bIsPressed = InPressed;
		OnButtonPressedChangedEvent.Broadcast(InPressed);
	}
}

TSharedRef<SWidget> UAudioMaterialButton::RebuildWidget()
{
	Button = SNew(SAudioMaterialButton)
		.Owner(this)
		.AudioMaterialButtonStyle(&WidgetStyle)
		.bIsPressedAttribute(bIsPressed)
		.OnBooleanValueChanged(BIND_UOBJECT_DELEGATE(FOnBooleanValueChanged, HandleOnPressedValueChanged))
		.OnMouseCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureBegin))
		.OnMouseCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureEnd));

	return Button.ToSharedRef();
}

void UAudioMaterialButton::HandleOnPressedValueChanged(bool InPressedSate)
{
	bIsPressed = InPressedSate;
	OnButtonPressedChangedEvent.Broadcast(InPressedSate);
}

void UAudioMaterialButton::HandleOnMouseCaptureBegin()
{
	OnMouseCaptureBegin.Broadcast();
}

void UAudioMaterialButton::HandleOnMouseCaptureEnd()
{
	OnMouseCaptureEnd.Broadcast();
}

#undef LOCTEXT_NAMESPACE
