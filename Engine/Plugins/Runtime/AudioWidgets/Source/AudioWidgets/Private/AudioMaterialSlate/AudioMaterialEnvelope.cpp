// Copyright Epic Games, Inc. All Rights Reserved.


#include "AudioMaterialSlate/AudioMaterialEnvelope.h"
#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "AudioMaterialSlate/SAudioMaterialEnvelope.h"
#include "AudioWidgetsStyle.h"
#include "Widgets/SWeakWidget.h"
#include "Components/AudioComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioMaterialEnvelope)

#define LOCTEXT_NAMESPACE "AudioWidgets"

UAudioMaterialEnvelope::UAudioMaterialEnvelope()
{
	//get default style
	WidgetStyle = FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioMaterialEnvelopeStyle>("AudioMaterialEnvelope.Style");
}

#if WITH_EDITOR
const FText UAudioMaterialEnvelope::GetPaletteCategory()
{
	return LOCTEXT("PaletteCategory", "AudioMaterial");
}
#endif

void UAudioMaterialEnvelope::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!EnvelopeCurve.IsValid())
	{
		return;
	}

	EnvelopeCurve->ApplyNewMaterial();
}

void UAudioMaterialEnvelope::ReleaseSlateResources(bool bReleaseChildren)
{
	EnvelopeCurve.Reset();
}

TSharedRef<SWidget> UAudioMaterialEnvelope::RebuildWidget()
{
	EnvelopeCurve = SNew(SAudioMaterialEnvelope)
		.Owner(this)
		.AudioMaterialEnvelopeStyle(&WidgetStyle)
		.EnvelopeSettings(&EnvelopeSettings);

	return EnvelopeCurve.ToSharedRef();
}

#undef LOCTEXT_NAMESPACE
