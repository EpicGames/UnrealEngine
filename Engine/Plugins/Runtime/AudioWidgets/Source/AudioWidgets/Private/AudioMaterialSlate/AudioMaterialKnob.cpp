// Copyright Epic Games, Inc. All Rights Reserved.


#include "AudioMaterialSlate/AudioMaterialKnob.h"
#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "AudioMaterialSlate/SAudioMaterialKnob.h"
#include "AudioWidgetsStyle.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioMaterialKnob)

#define LOCTEXT_NAMESPACE "AudioWidgets"
UAudioMaterialKnob::UAudioMaterialKnob(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, TuneSpeed(0.2f)
	, FineTuneSpeed(0.05f)
	, bLocked(false)
	, bMouseUsesStep(false)
	, StepSize(0.01f)
{
	//get default style
	WidgetStyle = FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioMaterialKnobStyle>("AudioMaterialKnob.Style");
}

#if WITH_EDITOR
const FText UAudioMaterialKnob::GetPaletteCategory()
{
	return LOCTEXT("PaletteCategory", "AudioMaterial");
}
#endif // WITH_EDITOR

void UAudioMaterialKnob::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!Knob.IsValid())
	{
		return;
	}

	Knob->SetValue(Value);
	Knob->ApplyNewMaterial();
}

void UAudioMaterialKnob::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);
	Knob.Reset();
}

float UAudioMaterialKnob::GetValue()
{
	return Value;
}

void UAudioMaterialKnob::SetValue(float InValue)
{
	InValue = FMath::Clamp(InValue, 0.f, 1.f);

	if (Knob.IsValid())
	{
		Knob->SetValue(InValue);
		HandleOnKnobValueChanged(InValue);
	}
}

void UAudioMaterialKnob::SetTuneSpeed(float InValue)
{
	TuneSpeed = FMath::Clamp(InValue, 0.f, 1.0f);

	if (Knob.IsValid())
	{
		Knob->SetTuneSpeed(TuneSpeed);
	}
}

float UAudioMaterialKnob::GetTuneSpeed() const
{
	return TuneSpeed;
}

void UAudioMaterialKnob::SetFineTuneSpeed(float InValue)
{
	FineTuneSpeed = FMath::Clamp(InValue, 0.f, 1.0f);

	if (Knob.IsValid())
	{
		Knob->SetFineTuneSpeed(FineTuneSpeed);
	}
}

float UAudioMaterialKnob::GetFineTuneSpeed() const
{
	return FineTuneSpeed;
}

void UAudioMaterialKnob::SetLocked(bool InLocked)
{
	bLocked = InLocked;

	if (Knob.IsValid())
	{
		Knob->SetLocked(InLocked);
	}
}

bool UAudioMaterialKnob::GetIsLocked() const
{
	return bLocked;
}

void UAudioMaterialKnob::SetMouseUsesStep(bool InUsesStep)
{
	bMouseUsesStep = InUsesStep;

	if (Knob.IsValid())
	{
		Knob->SetMouseUsesStep(InUsesStep);
	}
}

bool UAudioMaterialKnob::GetMouseUsesStep() const
{
	return bMouseUsesStep;
}

void UAudioMaterialKnob::SetStepSize(float InValue)
{
	StepSize = InValue;

	if (Knob.IsValid())
	{
		Knob->SetStepSize(InValue);
	}
}

float UAudioMaterialKnob::GetStepSize() const
{
	return StepSize;
}

TSharedRef<SWidget> UAudioMaterialKnob::RebuildWidget()
{
	Knob = SNew(SAudioMaterialKnob)
		.Owner(this)
		.AudioMaterialKnobStyle(&WidgetStyle)
		.Value(Value)
		.TuneSpeed(TuneSpeed)
		.Locked(bLocked)
		.FineTuneSpeed(FineTuneSpeed)
		.MouseUsesStep(bMouseUsesStep)
		.StepSize(StepSize)
		.OnFloatValueChanged(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnKnobValueChanged))
		.OnMouseCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnKnobCaptureBegin))
		.OnMouseCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnKnobCaptureEnd));

	return Knob.ToSharedRef();
}

void UAudioMaterialKnob::HandleOnKnobCaptureBegin()
{
	OnMouseCaptureBegin.Broadcast();
}

void UAudioMaterialKnob::HandleOnKnobCaptureEnd()
{
	OnMouseCaptureEnd.Broadcast();
}

void UAudioMaterialKnob::HandleOnKnobValueChanged(float InValue)
{
	if (Value != InValue)
	{
		Value = InValue;
		OnKnobValueChanged.Broadcast(InValue);
	}
}

#undef LOCTEXT_NAMESPACE
