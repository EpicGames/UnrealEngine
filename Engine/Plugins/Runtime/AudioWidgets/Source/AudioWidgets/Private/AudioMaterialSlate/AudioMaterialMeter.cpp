// Copyright Epic Games, Inc. All Rights Reserved.


#include "AudioMaterialSlate/AudioMaterialMeter.h"
#include "AudioBusSubsystem.h"
#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "AudioMaterialSlate/SAudioMaterialMeter.h"
#include "AudioMixerDevice.h"
#include "AudioWidgetsStyle.h"
#include "Components/AudioComponent.h"
#include "SAudioMeter.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/SWeakWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioMaterialMeter)

#define LOCTEXT_NAMESPACE "AudioWidgets"

UAudioMaterialMeter::UAudioMaterialMeter()
{
	Orientation = EOrientation::Orient_Vertical;

	//get default style
	WidgetStyle = FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioMaterialMeterStyle>("AudioMaterialMeter.Style");

	// Add a single channel as a default just so it can be seen when somebody makes one
	FMeterChannelInfo DefaultInfo;
	DefaultInfo.MeterValue = -6.0f;
	DefaultInfo.PeakValue = -3.0f;
	MeterChannelInfo.Add(DefaultInfo);
}

#if WITH_EDITOR
const FText UAudioMaterialMeter::GetPaletteCategory()
{
	return LOCTEXT("PaletteCategory", "AudioMaterial");
}
#endif

void UAudioMaterialMeter::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!Meter.IsValid())
	{
		return;
	}

	Meter->ApplyNewMaterial();
	Meter->SetOrientation(Orientation);

	TAttribute<TArray<FMeterChannelInfo>> MeterChannelInfoBinding = PROPERTY_BINDING(TArray<FMeterChannelInfo>, MeterChannelInfo);
	Meter->SetMeterChannelInfo(MeterChannelInfoBinding);
}

void UAudioMaterialMeter::ReleaseSlateResources(bool bReleaseChildren)
{
	Meter.Reset();
}

TArray<FMeterChannelInfo> UAudioMaterialMeter::GetMeterChannelInfo() const
{
	if (Meter.IsValid())
	{
		return Meter->GetMeterChannelInfo();
	}
	return TArray<FMeterChannelInfo>();
}

void UAudioMaterialMeter::SetMeterChannelInfo(const TArray<FMeterChannelInfo>& InMeterChannelInfo)
{
	if (Meter.IsValid())
	{
		Meter->SetMeterChannelInfo(InMeterChannelInfo);
	}
}

TSharedRef<SWidget> UAudioMaterialMeter::RebuildWidget()
{
	Meter = SNew(SAudioMaterialMeter)
		.Owner(this)
		.AudioMaterialMeterStyle(&WidgetStyle);

	return Meter.ToSharedRef();
}

#undef LOCTEXT_NAMESPACE
