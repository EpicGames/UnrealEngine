// Copyright Epic Games, Inc. All Rights Reserved.


#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "Styling/StyleDefaults.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioMaterialSlateTypes)

namespace AudioMaterialSlateTypesPrivate
{
	#define PLUGIN_BASE_DIR FString("/AudioWidgets/AudioMaterialSlate/")
}

using namespace AudioMaterialSlateTypesPrivate;

namespace AudioWidgets
{
	namespace SlateTypesPrivate
	{
		// Button
		static const FLinearColor ButtonMainColor(0.5f, 0.5f, 0.5f, 1.f);
		static const FLinearColor ButtonAccentColor(0.96f, 0.96f, 0.96f, 1.f);
		static const FLinearColor ButtonShadowColor(0.5f, 0.5f, 0.5f, 1.f);
		static const FLinearColor ButtonUnpressedOutlineColor(0.0625f, 0.0625f, 0.0625f, 1.f);
		static const FLinearColor ButtonPressedOutlineColor(0.96f, 0.96f, 0.96f, 1.f);
		static const FLinearColor ButtonMainColorTint1(0.96f, 0.96f, 0.96f, 1.f);
		static const FLinearColor ButtonMainColorTint2(0.06f, 0.06f, 0.06f, 1.f);
		// End Button

		// Slider
		static const FLinearColor SliderBackgroundColor(0.008f, 0.008f, 0.008f, 1.f);
		static const FLinearColor SliderBackgroundAccentColor(0.005f, 0.005f, 0.005f, 1.f);
		static const FLinearColor SliderHandleMainColor(1.f, 1.f, 1.f, 1.f);
		static const FLinearColor SliderHandleOutlineColor(0.15f, 0.15f, 0.15f, 1.f);
		static const FLinearColor SliderValueMainColor(0.008f, 0.008f, 0.008f, 1.f);
		// End Slider

		// Knob
		static const FLinearColor KnobMainColor(0.140625f, 0.140625f, 0.140625f, 1.f);
		static const FLinearColor KnobAccentColor(0.06f, 0.06f, 0.06f, 1.f);
		static const FLinearColor KnobShadowColor(0.06f, 0.06f, 0.06f, 1.f);
		static const FLinearColor KnobSmoothBevelColor(0.041667f, 0.041667f, 0.041667f, 1.f);
		static const FLinearColor KnobIndicatorDotColor(1.0f, 0.0f, 0.0f, 1.f);
		static const FLinearColor KnobEdgeFillColor(0.015625f, 0.015625f, 0.015625f, 1.f);
		static const FLinearColor KnobBarColor(0.067f, 0.067f, 0.067f, 1.f);
		static const FLinearColor KnobBarShadowColor(0.067f, 0.067f, 0.067f, 1.f);
		static const FLinearColor KnobBarFillMinColor(0.96f, 0.96f, 0.96f, 1.f);
		static const FLinearColor KnobBarFillMidColor(0.96f, 0.96f, 0.96f, 1.f);
		static const FLinearColor KnobBarFillMaxColor(0.96f, 0.96f, 0.96f, 1.f);
		static const FLinearColor KnobBarFillTintColor(0.96f, 0.96f, 0.96f, 1.f);
		// End Knob

		// Meter
		static const FLinearColor MeterFillMinColor(0.96f, 0.96f, 0.96f, 1.f);
		static const FLinearColor MeterFillMidColor(0.96f, 0.96f, 0.96f, 1.f);
		static const FLinearColor MeterFillMaxColor(0.96f, 0.96f, 0.96f, 1.f);
		static const FLinearColor MeterFillBackgroundColor(0.06f, 0.06f, 0.06f, 1.f);
		static const FVector2D MeterPadding(FVector2D(10.0f, 5.0f));
		static const FVector2D MeterValueRangeDb(FVector2D(-60, 10));
		static const bool bShowMeterScale(true);
		static const bool bScaleMeterSide(true);
		static const float MeterScaleHashOffset(5.0f);
		static const float MeterScaleHashWidth(10.0f);
		static const float MeterScaleHashHeight(1.0f);
		static const int32 MeterDecibelsPerHash(5);
		// End Meter
	}
}

using namespace AudioWidgets;

FAudioMaterialWidgetStyle::FAudioMaterialWidgetStyle()
	:DesiredSize(32.f, 32.f)
{
}

UMaterialInstanceDynamic* FAudioMaterialWidgetStyle::CreateDynamicMaterial(UObject* InOuter) const
{
	return UMaterialInstanceDynamic::Create(Material, InOuter);
}

FAudioMaterialButtonStyle::FAudioMaterialButtonStyle()
	: ButtonMainColor(SlateTypesPrivate::ButtonMainColor)
	, ButtonMainColorTint_1(SlateTypesPrivate::ButtonMainColorTint1)
	, ButtonMainColorTint_2(SlateTypesPrivate::ButtonMainColorTint2)
	, ButtonAccentColor(SlateTypesPrivate::ButtonAccentColor)
	, ButtonShadowColor(SlateTypesPrivate::ButtonShadowColor)
	, ButtonUnpressedOutlineColor(SlateTypesPrivate::ButtonUnpressedOutlineColor)
	, ButtonPressedOutlineColor(SlateTypesPrivate::ButtonPressedOutlineColor)
{
	DesiredSize = FVector2f(128.f, 128.f);	
}

const FName FAudioMaterialButtonStyle::TypeName(TEXT("FAudioMaterialButtonStyle"));

void FAudioMaterialButtonStyle::GetResources(TArray<const FSlateBrush*>& OutBrushes) const
{
}

const FAudioMaterialButtonStyle& FAudioMaterialButtonStyle::GetDefault()
{
	static FAudioMaterialButtonStyle Default;
	return Default;
}

UMaterialInstanceDynamic* FAudioMaterialButtonStyle::CreateDynamicMaterial(UObject* InOuter) const
{
	// Use default material if none provided
	if (!Material)
	{
		const FString Path = PLUGIN_BASE_DIR + "MI_AudioMaterialToggleButton.MI_AudioMaterialToggleButton";
		return UMaterialInstanceDynamic::Create(LoadObject<UMaterialInterface>(nullptr, *Path), InOuter);
	}
	return Super::CreateDynamicMaterial(InOuter);
}

FAudioMaterialSliderStyle::FAudioMaterialSliderStyle()
	: SliderBackgroundColor(SlateTypesPrivate::SliderBackgroundColor)
	, SliderBackgroundAccentColor(SlateTypesPrivate::SliderBackgroundAccentColor)
	, SliderValueMainColor(SlateTypesPrivate::SliderValueMainColor)
	, SliderHandleMainColor(SlateTypesPrivate::SliderHandleMainColor)
	, SliderHandleOutlineColor(SlateTypesPrivate::SliderHandleOutlineColor)
	, TextBoxStyle(FAudioTextBoxStyle::GetDefault())
{
	DesiredSize = FVector2f(25.f, 250.f);
}

const FName FAudioMaterialSliderStyle::TypeName(TEXT("FAudioMaterialSliderStyle"));

void FAudioMaterialSliderStyle::GetResources(TArray<const FSlateBrush*>& OutBrushes) const
{
	// Add any brush resources here so that Slate can correctly atlas and reference them
}

const FAudioMaterialSliderStyle& FAudioMaterialSliderStyle::GetDefault()
{
	static FAudioMaterialSliderStyle Default;
	return Default;
}

UMaterialInstanceDynamic* FAudioMaterialSliderStyle::CreateDynamicMaterial(UObject* InOuter) const
{
	// Use default material if none provided
	if (!Material)
	{
		const FString Path = PLUGIN_BASE_DIR + "MI_AudioMaterialRoundedSlider.MI_AudioMaterialRoundedSlider";
		return UMaterialInstanceDynamic::Create(LoadObject<UMaterialInterface>(nullptr, *Path), InOuter);
	}
	return Super::CreateDynamicMaterial(InOuter);
}

FAudioMaterialKnobStyle::FAudioMaterialKnobStyle()
	: KnobMainColor(SlateTypesPrivate::KnobMainColor)
	, KnobAccentColor(SlateTypesPrivate::KnobAccentColor)
	, KnobShadowColor(SlateTypesPrivate::KnobShadowColor)
	, KnobSmoothBevelColor(SlateTypesPrivate::KnobSmoothBevelColor)
	, KnobIndicatorDotColor(SlateTypesPrivate::KnobIndicatorDotColor)
	, KnobEdgeFillColor(SlateTypesPrivate::KnobEdgeFillColor)
	, KnobBarColor(SlateTypesPrivate::KnobBarColor)
	, KnobBarShadowColor(SlateTypesPrivate::KnobBarShadowColor)
	, KnobBarFillMinColor(SlateTypesPrivate::KnobBarFillMinColor)
	, KnobBarFillMidColor(SlateTypesPrivate::KnobBarFillMidColor)
	, KnobBarFillMaxColor(SlateTypesPrivate::KnobBarFillMaxColor)
	, KnobBarFillTintColor(SlateTypesPrivate::KnobBarFillTintColor)
{
	DesiredSize = FVector2f(128.f,128.f);
}

const FName FAudioMaterialKnobStyle::TypeName(TEXT("FAudioMaterialKnobStyle"));

const FAudioMaterialKnobStyle& FAudioMaterialKnobStyle::GetDefault()
{
	static FAudioMaterialKnobStyle Default;
	return Default;
}

void FAudioMaterialKnobStyle::GetResources(TArray<const FSlateBrush*>& OutBrushes) const
{
	// Add any brush resources here so that Slate can correctly atlas and reference them
}

UMaterialInstanceDynamic* FAudioMaterialKnobStyle::CreateDynamicMaterial(UObject* InOuter) const
{
	// Use default material if none provided
	if (!Material)
	{
		const FString Path = PLUGIN_BASE_DIR + "MI_AudioMaterialKnob.MI_AudioMaterialKnob";
		return UMaterialInstanceDynamic::Create(LoadObject<UMaterialInterface>(nullptr, *Path), InOuter);
	}
	return Super::CreateDynamicMaterial(InOuter);
}

FAudioMaterialMeterStyle::FAudioMaterialMeterStyle()
	: MeterFillMinColor(SlateTypesPrivate::MeterFillMinColor)
	, MeterFillMidColor(SlateTypesPrivate::MeterFillMidColor)
	, MeterFillMaxColor(SlateTypesPrivate::MeterFillMinColor)
	, MeterFillBackgroundColor(SlateTypesPrivate::MeterFillBackgroundColor)
	, MeterPadding(SlateTypesPrivate::MeterPadding)
	, ValueRangeDb(SlateTypesPrivate::MeterValueRangeDb)
	, bShowScale(SlateTypesPrivate::bShowMeterScale)
	, bScaleSide(SlateTypesPrivate::bScaleMeterSide)
	, ScaleHashOffset(SlateTypesPrivate::MeterScaleHashOffset)
	, ScaleHashWidth(SlateTypesPrivate::MeterScaleHashWidth)
	, ScaleHashHeight(SlateTypesPrivate::MeterScaleHashHeight)
	, DecibelsPerHash(SlateTypesPrivate::MeterDecibelsPerHash)
	, Font(FStyleDefaults::GetFontInfo(5))
{
	DesiredSize = FVector2f(25.f, 512.f);
}

const FName FAudioMaterialMeterStyle::TypeName(TEXT("FAudioMaterialMeterStyle"));

const FAudioMaterialMeterStyle& FAudioMaterialMeterStyle::GetDefault()
{
	static FAudioMaterialMeterStyle Default;
	return Default;
}

void FAudioMaterialMeterStyle::GetResources(TArray<const FSlateBrush*>& OutBrushes) const
{
	// Add any brush resources here so that Slate can correctly atlas and reference them
}

UMaterialInstanceDynamic* FAudioMaterialMeterStyle::CreateDynamicMaterial(UObject* InOuter) const
{
	// Use default material if none provided
	if (!Material)
	{
		const FString Path = PLUGIN_BASE_DIR + "MI_AudioMaterialMeter.MI_AudioMaterialMeter";
		return UMaterialInstanceDynamic::Create(LoadObject<UMaterialInterface>(nullptr, *Path), InOuter);
	}
	return Super::CreateDynamicMaterial(InOuter);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FAudioMaterialMeterStyle& FAudioMaterialMeterStyle::SetFont(const FName& InFontName, uint16 InSize) { Font = FSlateFontInfo(InFontName, InSize); return *this; }
FAudioMaterialMeterStyle& FAudioMaterialMeterStyle::SetFont(const FString& InFontName, uint16 InSize) { Font = FSlateFontInfo(*InFontName, InSize); return *this; }
FAudioMaterialMeterStyle& FAudioMaterialMeterStyle::SetFont(const WIDECHAR* InFontName, uint16 InSize) { Font = FSlateFontInfo(InFontName, InSize); return *this; }
FAudioMaterialMeterStyle& FAudioMaterialMeterStyle::SetFont(const ANSICHAR* InFontName, uint16 InSize) { Font = FSlateFontInfo(InFontName, InSize); return *this; }
FAudioMaterialMeterStyle& FAudioMaterialMeterStyle::SetFontName(const FName& InFontName) { Font = FSlateFontInfo(InFontName, Font.Size); return *this; }
FAudioMaterialMeterStyle& FAudioMaterialMeterStyle::SetFontName(const FString& InFontName) { Font = FSlateFontInfo(InFontName, Font.Size); return *this; }
FAudioMaterialMeterStyle& FAudioMaterialMeterStyle::SetFontName(const WIDECHAR* InFontName) { Font = FSlateFontInfo(InFontName, Font.Size); return *this; }
FAudioMaterialMeterStyle& FAudioMaterialMeterStyle::SetFontName(const ANSICHAR* InFontName) { Font = FSlateFontInfo(InFontName, Font.Size); return *this; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FAudioMaterialEnvelopeStyle::FAudioMaterialEnvelopeStyle()
	: CurveColor(FLinearColor::White)
	, BackgroundColor(FLinearColor::Black)
	, OutlineColor(FLinearColor::Gray)
{
	DesiredSize = FVector2f(256.f, 256.f);
}

const FName FAudioMaterialEnvelopeStyle::TypeName(TEXT("FAudioMaterialEnvelopeStyle"));

const FAudioMaterialEnvelopeStyle& FAudioMaterialEnvelopeStyle::GetDefault()
{
	static FAudioMaterialEnvelopeStyle Default;
	return Default;
}

void FAudioMaterialEnvelopeStyle::GetResources(TArray<const FSlateBrush*>& OutBrushes) const
{
	// Add any brush resources here so that Slate can correctly atlas and reference them
}

UMaterialInstanceDynamic* FAudioMaterialEnvelopeStyle::CreateDynamicMaterial(UObject* InOuter) const
{
	// Use default material if none provided
	if (!Material)
	{
		const FString Path = PLUGIN_BASE_DIR + "MI_AudioMaterialEnvelope_ADSR.MI_AudioMaterialEnvelope_ADSR";
		return UMaterialInstanceDynamic::Create(LoadObject<UMaterialInterface>(nullptr, *Path), InOuter);
	}
	return Super::CreateDynamicMaterial(InOuter);
}
