// Copyright Epic Games, Inc. All Rights Reserved.


#include "AudioMaterialSlate/AudioMaterialSlider.h"
#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "AudioMaterialSlate/SAudioMaterialSlider.h"
#include "AudioWidgetsStyle.h"
#include "Widgets/SWeakWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioMaterialSlider)

#define LOCTEXT_NAMESPACE "AudioWidgets"

UAudioMaterialSlider::UAudioMaterialSlider(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, TuneSpeed(0.2f)
	, FineTuneSpeed(0.05f)
	, bLocked(false)
	, bMouseUsesStep(false)
	, StepSize(0.01f)
{
	//get default style
	WidgetStyle = FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioMaterialSliderStyle>("AudioMaterialSlider.Style");
}

#if WITH_EDITOR
const FText UAudioMaterialSlider::GetPaletteCategory()
{
	return LOCTEXT("PaletteCategory", "AudioMaterial");
}
#endif

void UAudioMaterialSlider::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (!Slider.IsValid())
	{
		return;
	}

	Slider->SetValue(Value);
	Slider->SetOrientation(Orientation);
	Slider->ApplyNewMaterial();
}

void UAudioMaterialSlider::ReleaseSlateResources(bool bReleaseChildren)
{
	Slider.Reset();
}

float UAudioMaterialSlider::GetValue() const
{
	return Value;
}

void UAudioMaterialSlider::SetValue(float InValue)
{
	InValue = FMath::Clamp(InValue, 0.f, 1.f);

	if (Slider.IsValid())
	{
		Slider->SetValue(InValue);
		HandleOnValueChanged(InValue);
	}
}

void UAudioMaterialSlider::SetTuneSpeed(const float InValue)
{
	TuneSpeed = FMath::Clamp(InValue, 0.f, 1.f);

	if (Slider.IsValid())
	{
		Slider.Get()->SetTuneSpeed(TuneSpeed);
	}
}

float UAudioMaterialSlider::GetTuneSpeed() const
{
	return TuneSpeed;
}

void UAudioMaterialSlider::SetFineTuneSpeed(const float InValue)
{
	FineTuneSpeed = FMath::Clamp(InValue, 0.f, 1.0f);

	if (Slider.IsValid())
	{
		Slider.Get()->SetFineTuneSpeed(FineTuneSpeed);
	}
}

float UAudioMaterialSlider::GetFineTuneSpeed() const
{
	return FineTuneSpeed;
}

void UAudioMaterialSlider::SetLocked(bool bInLocked)
{
	bLocked = bInLocked;

	if (Slider.IsValid())
	{
		Slider->SetLocked(bInLocked);
	}
}

bool UAudioMaterialSlider::GetIsLocked() const
{
	return bLocked;
}

void UAudioMaterialSlider::SetMouseUsesStep(bool bInUsesStep)
{
	bMouseUsesStep = bInUsesStep;

	if (Slider.IsValid())
	{
		Slider->SetMouseUsesStep(bInUsesStep);
	}
}

bool UAudioMaterialSlider::GetMouseUsesStep() const
{
	return bMouseUsesStep;
}

void UAudioMaterialSlider::SetStepSize(float InValue)
{
	StepSize = InValue;

	if (Slider.IsValid())
	{
		Slider->SetStepSize(InValue);
	}
}

float UAudioMaterialSlider::GetStepSize() const
{
	return StepSize;
}

TSharedRef<SWidget> UAudioMaterialSlider::RebuildWidget()
{
	Slider = SNew(SAudioMaterialSlider)
		.Owner(this)
		.Orientation(Orientation)
		.TuneSpeed(TuneSpeed)
		.FineTuneSpeed(FineTuneSpeed)
		.Locked(bLocked)
		.MouseUsesStep(bMouseUsesStep)
		.StepSize(StepSize)
		.AudioMaterialSliderStyle(&WidgetStyle)
		.OnValueChanged(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnValueChanged))
		.OnMouseCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureBegin))
		.OnMouseCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureEnd));

	return Slider.ToSharedRef();
}

void UAudioMaterialSlider::HandleOnValueChanged(float InValue)
{
	if (Value != InValue)
	{
		Value = InValue;
		OnValueChanged.Broadcast(InValue);
	}
}

void UAudioMaterialSlider::HandleOnMouseCaptureBegin()
{
	OnMouseCaptureBegin.Broadcast();
}

void UAudioMaterialSlider::HandleOnMouseCaptureEnd()
{
	OnMouseCaptureEnd.Broadcast();
}

#undef LOCTEXT_NAMESPACE
