// Copyright Epic Games, Inc. All Rights Reserved.


#include "AudioMaterialSlate/SAudioMaterialButton.h"
#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "Components/Widget.h"
#include "SlateOptMacros.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SAudioMaterialButton::Construct(const FArguments& InArgs)
{
	Owner = InArgs._Owner;
	AudioMaterialButtonStyle = InArgs._AudioMaterialButtonStyle;
	bIsPressedAttribute = InArgs._bIsPressedAttribute;
	OnBooleanValueChanged = InArgs._OnBooleanValueChanged;
	OnMouseCaptureBegin = InArgs._OnMouseCaptureBegin;
	OnMouseCaptureEnd = InArgs._OnMouseCaptureEnd;

	ApplyNewMaterial();

}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SAudioMaterialButton::SetPressedState(bool InPressedState)
{
	CommitNewState(InPressedState);
}

UMaterialInstanceDynamic* SAudioMaterialButton::ApplyNewMaterial()
{
	if (AudioMaterialButtonStyle)
	{
		DynamicMaterial = AudioMaterialButtonStyle->CreateDynamicMaterial(Owner.Get());
	}
	return DynamicMaterial.Get();
}

void SAudioMaterialButton::SetDesiredSizeOverride(const FVector2D InSize)
{
	SetAttribute(DesiredSizeOverride, TAttribute<TOptional<FVector2D>>(InSize),EInvalidateWidgetReason::Layout);
}

int32 SAudioMaterialButton::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	if (AudioMaterialButtonStyle)
	{
		if (DynamicMaterial.IsValid())
		{
			DynamicMaterial.Get()->SetVectorParameterValue(FName("MainColor"), AudioMaterialButtonStyle->ButtonMainColor);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("ShadowColor"), AudioMaterialButtonStyle->ButtonShadowColor);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("SmoothBevelColor"), AudioMaterialButtonStyle->ButtonAccentColor);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("Color_1"), AudioMaterialButtonStyle->ButtonMainColorTint_1);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("Color_2"), AudioMaterialButtonStyle->ButtonMainColorTint_2);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("LedColor"), AudioMaterialButtonStyle->ButtonPressedOutlineColor);
			DynamicMaterial.Get()->SetScalarParameterValue(FName("Click"), bIsPressedAttribute.Get());
			DynamicMaterial.Get()->SetVectorParameterValue(FName("BarColor"), AudioMaterialButtonStyle->ButtonUnpressedOutlineColor);

			DynamicMaterial.Get()->SetScalarParameterValue(FName("LocalWidth"), AllottedGeometry.GetLocalSize().X);
			DynamicMaterial.Get()->SetScalarParameterValue(FName("LocalHeigth"), AllottedGeometry.GetLocalSize().Y);		
			
			const bool bEnabled = ShouldBeEnabled(bParentEnabled);
			const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

			const FLinearColor FinalColorAndOpacity(InWidgetStyle.GetColorAndOpacityTint());

			const float AllotedWidth = AllottedGeometry.GetLocalSize().X;
			const float AllotedHeight = AllottedGeometry.GetLocalSize().Y;

			const float ButtonRadius = FMath::Min(AllotedWidth, AllotedHeight) * 0.5f;
			const FVector2D ButtonMidPoint(AllottedGeometry.GetLocalSize() * 0.5f);
			const FVector2D ButtonDiameter(ButtonRadius * 2);

			FSlateBrush Brush;
			Brush.SetResourceObject(DynamicMaterial.Get());
			FSlateDrawElement::MakeBox(OutDrawElements, LayerId++, AllottedGeometry.ToPaintGeometry(ButtonDiameter , FSlateLayoutTransform(ButtonMidPoint-ButtonRadius)),&Brush, DrawEffects, FinalColorAndOpacity);
		}
		else
		{
			if (AudioMaterialButtonStyle)
			{
				DynamicMaterial = AudioMaterialButtonStyle->CreateDynamicMaterial(Owner.Get());
			}
		}
	}	

	return LayerId;
}

FVector2D SAudioMaterialButton::ComputeDesiredSize(float) const
{
	if (DesiredSizeOverride.Get().IsSet())
	{
		return DesiredSizeOverride.Get().GetValue();
	}

	if (AudioMaterialButtonStyle)
	{
		return FVector2D(AudioMaterialButtonStyle->DesiredSize);
	}

	return FVector2D::ZeroVector;
}

FReply SAudioMaterialButton::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton))
	{
		OnMouseCaptureBegin.ExecuteIfBound();
		CommitNewState(!bIsPressedAttribute.Get());
		FReply Reply = FReply::Handled().CaptureMouse(SharedThis(this));
		return Reply;
	}

	return FReply::Unhandled();
}

FReply SAudioMaterialButton::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && this->HasMouseCapture())
	{
		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

void SAudioMaterialButton::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	OnMouseCaptureEnd.ExecuteIfBound();
	SLeafWidget::OnMouseCaptureLost(CaptureLostEvent);
}

void SAudioMaterialButton::CommitNewState(bool InPressedState)
{
	if (bIsPressedAttribute.Get() != InPressedState)
	{
		if (!bIsPressedAttribute.IsBound())
		{
			bIsPressedAttribute.Set(InPressedState);
		}
		Invalidate(EInvalidateWidgetReason::Paint);
		OnBooleanValueChanged.ExecuteIfBound(InPressedState);
	}
}
