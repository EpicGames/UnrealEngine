// Copyright Epic Games, Inc. All Rights Reserved.


#include "AudioMaterialSlate/SAudioMaterialEnvelope.h"
#include "AudioMaterialSlate/AudioMaterialEnvelope.h"
#include "Components/AudioComponent.h"
#include "SlateOptMacros.h"
#include "Styling/SlateBrush.h"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SAudioMaterialEnvelope::Construct(const FArguments& InArgs)
{
	Owner = InArgs._Owner;
	EnvelopeSettings = InArgs._EnvelopeSettings;
	AudioMaterialEnvelopeStyle = InArgs._AudioMaterialEnvelopeStyle;

	ApplyNewMaterial();
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

int32 SAudioMaterialEnvelope::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	if (AudioMaterialEnvelopeStyle && EnvelopeSettings)
	{
		if (DynamicMaterial.IsValid())
		{
			DynamicMaterial->SetScalarParameterValue(FName("A_Curve"), EnvelopeSettings->AttackCurve);
			DynamicMaterial->SetScalarParameterValue(FName("A_Int"), EnvelopeSettings->AttackValue);
			DynamicMaterial->SetScalarParameterValue(FName("A_Time"), EnvelopeSettings->AttackTime);

			DynamicMaterial->SetScalarParameterValue(FName("D_Curve"), EnvelopeSettings->DecayCurve);
			DynamicMaterial->SetScalarParameterValue(FName("D_Time"), EnvelopeSettings->DecayTime);

			if (EnvelopeSettings->EnvelopeType == EAudioMaterialEnvelopeType::ADSR)
			{
				DynamicMaterial->SetScalarParameterValue(FName("R_Curve"), EnvelopeSettings->ReleaseCurve);
				DynamicMaterial->SetScalarParameterValue(FName("R_Time"), EnvelopeSettings->ReleaseTime);

				DynamicMaterial->SetScalarParameterValue(FName("S_Int"), EnvelopeSettings->SustainValue);
			}

			DynamicMaterial->SetVectorParameterValue(FName("MainColor"), AudioMaterialEnvelopeStyle->CurveColor);
			DynamicMaterial->SetVectorParameterValue(FName("BoxBG"), AudioMaterialEnvelopeStyle->BackgroundColor);
			DynamicMaterial->SetVectorParameterValue(FName("BoxOutline"), AudioMaterialEnvelopeStyle->OutlineColor);

			DynamicMaterial->SetScalarParameterValue(FName("LocalWidth"), AllottedGeometry.GetLocalSize().X);
			DynamicMaterial->SetScalarParameterValue(FName("LocalHeigth"), AllottedGeometry.GetLocalSize().Y);			
			
			const bool bEnabled = ShouldBeEnabled(bParentEnabled);
			const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

			const FLinearColor FinalColorAndOpacity(InWidgetStyle.GetColorAndOpacityTint());

			FSlateBrush Brush;
			Brush.SetResourceObject(DynamicMaterial.Get());
			FSlateDrawElement::MakeBox(OutDrawElements, LayerId++, AllottedGeometry.ToPaintGeometry(), &Brush, DrawEffects, FinalColorAndOpacity);
		}
		else
		{
			if (AudioMaterialEnvelopeStyle)
			{
				DynamicMaterial = AudioMaterialEnvelopeStyle->CreateDynamicMaterial(Owner.Get());
			}
		}
	}

	return LayerId;
}

FVector2D SAudioMaterialEnvelope::ComputeDesiredSize(float) const
{
	if (AudioMaterialEnvelopeStyle)
	{
		return FVector2D(AudioMaterialEnvelopeStyle->DesiredSize);
	}

	return FVector2D::ZeroVector;
}

UMaterialInstanceDynamic* SAudioMaterialEnvelope::ApplyNewMaterial()
{
	if (AudioMaterialEnvelopeStyle)
	{
		DynamicMaterial = AudioMaterialEnvelopeStyle->CreateDynamicMaterial(Owner.Get());
	}

	return DynamicMaterial.Get();
}
