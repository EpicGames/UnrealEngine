// Copyright Epic Games, Inc. All Rights Reserved.


#include "AudioMaterialSlate/SAudioMaterialKnob.h"
#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "Framework/Application/SlateApplication.h"
#include "SlateOptMacros.h"
#include "Components/Widget.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SAudioMaterialKnob::Construct(const FArguments& InArgs)
{
	Owner = InArgs._Owner;

	TuneSpeed = InArgs._TuneSpeed;
	FineTuneSpeed = InArgs._FineTuneSpeed;
	bIsFocusable = InArgs._IsFocusable;
	bLocked = InArgs._Locked;
	bMouseUsesStep = InArgs._MouseUsesStep;
	StepSize = InArgs._StepSize;

	AudioMaterialKnobStyle = InArgs._AudioMaterialKnobStyle;

	OnValueChanged = InArgs._OnFloatValueChanged;
	OnValueCommitted = InArgs._OnValueCommitted;
	OnMouseCaptureBegin = InArgs._OnMouseCaptureBegin;
	OnMouseCaptureEnd = InArgs._OnMouseCaptureEnd;

	ApplyNewMaterial();

	if (InArgs._Value.IsSet())
	{
		CommitValue(InArgs._Value.Get());
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SAudioMaterialKnob::SetValue(const TAttribute<float>& InValueAttribute)
{
	SetAttribute(ValueAttribute, InValueAttribute, EInvalidateWidgetReason::Paint);
}

void SAudioMaterialKnob::SetTuneSpeed(const float InTurnSpeed)
{
	TuneSpeed.Set(InTurnSpeed);
}

void SAudioMaterialKnob::SetFineTuneSpeed(const float InFineTuneTurnSpeed)
{
	FineTuneSpeed.Set(InFineTuneTurnSpeed);
}

void SAudioMaterialKnob::SetLocked(const bool InLocked)
{
	bLocked.Set(InLocked);
}

void SAudioMaterialKnob::SetMouseUsesStep(const bool InUsesStep)
{
	bMouseUsesStep.Set(InUsesStep);
}

void SAudioMaterialKnob::SetStepSize(const float InStepSize)
{
	StepSize.Set(InStepSize);
}

bool SAudioMaterialKnob::IsLocked() const
{
	return bLocked.Get();
}

UMaterialInstanceDynamic* SAudioMaterialKnob::ApplyNewMaterial()
{
	if (AudioMaterialKnobStyle)
	{
		DynamicMaterial = AudioMaterialKnobStyle->CreateDynamicMaterial(Owner.Get());
	}

	return DynamicMaterial.Get();
}
const float SAudioMaterialKnob::GetOutputValue(const float InSliderValue)
{
	return FMath::GetMappedRangeValueClamped(NormalizedLinearSliderRange, OutputRange, InSliderValue);
}

const float SAudioMaterialKnob::GetSliderValue(const float OutputValue)
{
	return FMath::GetMappedRangeValueClamped(OutputRange, NormalizedLinearSliderRange, OutputValue);
}

void SAudioMaterialKnob::SetOutputRange(const FVector2D Range)
{
	OutputRange = Range;
	// if Range.Y < Range.X, set Range.X to Range.Y
	OutputRange.X = FMath::Min(Range.X, Range.Y);

	const float OutputValue = GetOutputValue(ValueAttribute.Get());
	const float ClampedOutputValue = FMath::Clamp(OutputValue, OutputRange.X, OutputRange.Y);
	const float ClampedSliderValue = GetSliderValue(ClampedOutputValue);
	ValueAttribute.Set(ClampedSliderValue);
}

void SAudioMaterialKnob::SetDesiredSizeOverride(const FVector2D Size)
{
	SetAttribute(DesiredSizeOverride, TAttribute<TOptional<FVector2D>>(Size), EInvalidateWidgetReason::Layout);
}

int32 SAudioMaterialKnob::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	if (AudioMaterialKnobStyle)
	{
		if (DynamicMaterial.IsValid())
		{
			const float KnobPercent = ValueAttribute.Get();

			DynamicMaterial.Get()->SetVectorParameterValue(FName("Color_2"), AudioMaterialKnobStyle->KnobAccentColor);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("Color_1"), AudioMaterialKnobStyle->KnobMainColor);

			DynamicMaterial.Get()->SetVectorParameterValue(FName("BarColor"), AudioMaterialKnobStyle->KnobBarColor);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("BarShadowColor"), AudioMaterialKnobStyle->KnobBarShadowColor);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("Led_Max"), AudioMaterialKnobStyle->KnobBarFillMaxColor);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("Led_Med"), AudioMaterialKnobStyle->KnobBarFillMidColor);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("LED_Min"), AudioMaterialKnobStyle->KnobBarFillMinColor);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("DotColor"), AudioMaterialKnobStyle->KnobIndicatorDotColor);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("LedTint"), AudioMaterialKnobStyle->KnobBarFillTintColor);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("EdgeFillColor"), AudioMaterialKnobStyle->KnobEdgeFillColor);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("ShadowColor"), AudioMaterialKnobStyle->KnobShadowColor);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("SmoothBevelColor"), AudioMaterialKnobStyle->KnobSmoothBevelColor);

			DynamicMaterial.Get()->SetScalarParameterValue(FName("VALUE"), FMath::Clamp(KnobPercent, 0.f, 1.f));

			DynamicMaterial.Get()->SetScalarParameterValue(FName("LocalWidth"), AllottedGeometry.GetLocalSize().X);
			DynamicMaterial.Get()->SetScalarParameterValue(FName("LocalHeigth"), AllottedGeometry.GetLocalSize().Y);

			const bool bEnabled = ShouldBeEnabled(bParentEnabled);
			const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

			const FLinearColor FinalColorAndOpacity(InWidgetStyle.GetColorAndOpacityTint());
			
			const float AllottedWidth = AllottedGeometry.GetLocalSize().X;
			const float AllottedHeight = AllottedGeometry.GetLocalSize().Y;
			
			const float SliderRadius = FMath::Min(AllottedWidth, AllottedHeight) * 0.5f;
			const FVector2D SliderMidPoint(AllottedGeometry.GetLocalSize() * 0.5f);
			const FVector2D SliderDiameter(SliderRadius * 2);

			FSlateBrush Brush;
			Brush.SetResourceObject(DynamicMaterial.Get());
			FSlateDrawElement::MakeBox(OutDrawElements, LayerId++, AllottedGeometry.ToPaintGeometry(SliderDiameter, FSlateLayoutTransform(SliderMidPoint-SliderRadius)), &Brush, DrawEffects, FinalColorAndOpacity);
		}
		else
		{
			if (AudioMaterialKnobStyle)
			{
				DynamicMaterial = AudioMaterialKnobStyle->CreateDynamicMaterial(Owner.Get());
			}
		}
	}

	return LayerId;
}

FVector2D SAudioMaterialKnob::ComputeDesiredSize(float) const
{
	if (DesiredSizeOverride.Get().IsSet())
	{
		return DesiredSizeOverride.Get().GetValue();
	}
	
	if (AudioMaterialKnobStyle)
	{
		return FVector2D(AudioMaterialKnobStyle->DesiredSize);
	}

	return FVector2D::ZeroVector;
}

FReply SAudioMaterialKnob::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (this->HasMouseCapture() && !IsLocked())
	{
		SetCursor(EMouseCursor::GrabHandClosed);

		int32 CurrentYValue = MouseEvent.GetLastScreenSpacePosition().Y;
		const float Speed = bIsFineTune ? FineTuneSpeed.Get() : TuneSpeed.Get();

		float ValueDelta = (float)(MouseDownPosition.Y - CurrentYValue) / PixelDelta * Speed;
		float NewValue = FMath::Clamp(MouseDownValue + ValueDelta, 0.0f, 1.0f);

		if (bMouseUsesStep.Get())
		{
			const float SteppedValue = FMath::RoundToInt(NewValue / StepSize.Get()) * StepSize.Get();
			NewValue = SteppedValue;
		}

		CommitValue(NewValue);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SAudioMaterialKnob::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && !IsLocked())
	{
		CachedCursor = GetCursor().Get(EMouseCursor::Default);

		MouseDownPosition = MouseEvent.GetScreenSpacePosition();
		MouseDownValue = ValueAttribute.Get();
		OnMouseCaptureBegin.ExecuteIfBound();

		FReply Reply = FReply::Handled().CaptureMouse(SharedThis(this));
		return Reply;
	}

	return FReply::Unhandled();
}

FReply SAudioMaterialKnob::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && this->HasMouseCapture())
	{
		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

void SAudioMaterialKnob::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	SetCursor(CachedCursor);
	OnValueCommitted.ExecuteIfBound(ValueAttribute.Get());
	OnMouseCaptureEnd.ExecuteIfBound();
	SLeafWidget::OnMouseCaptureLost(CaptureLostEvent);
}

FReply SAudioMaterialKnob::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::LeftShift)
	{	
		MouseDownPosition = FSlateApplication::Get().GetCursorPos();
		MouseDownValue = ValueAttribute.Get();
		bIsFineTune = true;
	}
	
	return FReply::Unhandled();
}

FReply SAudioMaterialKnob::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	bIsFineTune = false;
	MouseDownPosition = FSlateApplication::Get().GetCursorPos();
	MouseDownValue = ValueAttribute.Get();
	
	return FReply::Unhandled();
}

bool SAudioMaterialKnob::SupportsKeyboardFocus() const
{
	return bIsFocusable.Get();
}

bool SAudioMaterialKnob::IsInteractable() const
{
	return IsEnabled() && !IsLocked() && SupportsKeyboardFocus();;
}

void SAudioMaterialKnob::CommitValue(float NewValue)
{
	const float OldValue = ValueAttribute.Get();
	float Val = FMath::Clamp(NewValue, 0.f, 1.f);

	if (NewValue != OldValue)
	{
		if (!ValueAttribute.IsBound())
		{
			ValueAttribute.Set(Val);
		}

		Invalidate(EInvalidateWidgetReason::Paint);
		OnValueChanged.ExecuteIfBound(Val);
	}
}
