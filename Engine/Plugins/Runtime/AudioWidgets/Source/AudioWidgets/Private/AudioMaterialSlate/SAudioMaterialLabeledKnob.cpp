// Copyright Epic Games, Inc. All Rights Reserved.


#include "AudioMaterialSlate/SAudioMaterialLabeledKnob.h"
#include "AudioMaterialSlate/SAudioMaterialKnob.h"
#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "SAudioTextBox.h"
#include "SlateOptMacros.h"
#include "Widgets/SBoxPanel.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SAudioMaterialLabeledKnob::Construct(const FArguments& InArgs)
{
	Style = InArgs._Style;
	
	Owner = InArgs._Owner;
	AudioUnitsValueType = InArgs._AudioUnitsValueType;

	OnValueChanged = InArgs._OnValueChanged;
	OnMouseCaptureEnd = InArgs._OnMouseCaptureEnd;
	OnMouseCaptureBegin = InArgs._OnMouseCaptureBegin;

	if (InArgs._Value.IsSet())
	{
		ValueAttribute = InArgs._Value;
	}

	// Text label
	SAssignNew(Label, SAudioTextBox)
		.Style(&Style->TextBoxStyle)
		.OnValueTextCommitted_Lambda([this](const FText& Text, ETextCommit::Type CommitType)
		{
			const float OutputValue = FCString::Atof(*Text.ToString());
			const float NewSliderValue = GetSliderValueForText(OutputValue);
			if (!FMath::IsNearlyEqual(NewSliderValue, ValueAttribute.Get()))
			{
				ValueAttribute.Set(NewSliderValue);
				Knob->SetValue(NewSliderValue);
				OnValueChanged.ExecuteIfBound(NewSliderValue);
				OnMouseCaptureEnd.ExecuteIfBound();
				OnValueTextCommitted.ExecuteIfBound(NewSliderValue);
			}
		});

	// Underlying Knob widget
	SAssignNew(Knob, SAudioMaterialKnob)
		.Value(ValueAttribute)
		.Owner(InArgs._Owner)
		.AudioMaterialKnobStyle(Style)
		.OnFloatValueChanged_Lambda([this](float Value)
		{
			ValueAttribute.Set(Value);
			OnValueChanged.ExecuteIfBound(Value);
			const float OutputValue = GetOutputValueForText(Value);
			Label->SetValueText(OutputValue);
		})
		.OnMouseCaptureBegin(OnMouseCaptureBegin)
		.OnMouseCaptureEnd(OnMouseCaptureEnd);

	//Create layout
	SAssignNew(VerticalLayotWidget, SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			Knob.ToSharedRef()
		]
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Top)
		.AutoHeight()
		[
			Label.ToSharedRef()
		];

	//Set the Unit processor from unit value type
	switch (AudioUnitsValueType.Get())
	{
	case EAudioUnitsValueType::Linear:
		if (Label.IsValid())
		{
			AudioUnitProcessor = MakeShared<FAudioUnitProcessor>();
			Label.Get()->SetShowUnitsText(false);
		}
		break;
	case EAudioUnitsValueType::Frequency:
		AudioUnitProcessor = MakeShared<FFrequencyProcessor>();
		break;
	case EAudioUnitsValueType::Volume:
		AudioUnitProcessor = MakeShared<FVolumeProcessor>(InArgs._bUseLinearOutput);
		break;
	}

	if (AudioUnitProcessor.IsValid())
	{
		SetOutputRange(AudioUnitProcessor->GetDefaultOutputRange());
		if (Label.IsValid())
		{
			Label.Get()->SetUnitsText(AudioUnitProcessor->GetUnitsText());
		}
	}

	ChildSlot
	[
		VerticalLayotWidget.ToSharedRef()
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SAudioMaterialLabeledKnob::SetValue(float InValue)
{
	ValueAttribute.Set(InValue);
	const float OutputValueForText = GetOutputValueForText(InValue);
	Label->SetValueText(OutputValueForText);
	Knob->SetValue(InValue);
}

const float SAudioMaterialLabeledKnob::GetOutputValue(const float InSliderValue)
{
	return AudioUnitProcessor.IsValid() ? AudioUnitProcessor->GetOutputValue(OutputRange, InSliderValue) : 0.0f;
}

const float SAudioMaterialLabeledKnob::GetSliderValue(const float OutputValue)
{
	return AudioUnitProcessor.IsValid() ? AudioUnitProcessor->GetSliderValue(OutputRange, OutputValue) : 0.0f;
}

const float SAudioMaterialLabeledKnob::GetOutputValueForText(const float InSliderValue)
{
	return AudioUnitProcessor.IsValid() ? AudioUnitProcessor->GetOutputValueForText(OutputRange, InSliderValue) : 0.0f;
}

const float SAudioMaterialLabeledKnob::GetSliderValueForText(const float OutputValue)
{
	return AudioUnitProcessor.IsValid() ? AudioUnitProcessor->GetSliderValueForText(OutputRange, OutputValue) : 0.0f;
}

void SAudioMaterialLabeledKnob::SetSliderValue(float InSliderValue)
{
	SetValue(InSliderValue);
}

void SAudioMaterialLabeledKnob::SetOutputRange(const FVector2D InRange)
{
	//Check the valid range from the processor
	FVector2D Range = AudioUnitProcessor.IsValid() ? AudioUnitProcessor->GetOutputRange(InRange) : InRange;
	OutputRange = Range;

	// if Range.Y < Range.X, set Range.X to Range.Y
	OutputRange.X = FMath::Min(Range.X, Range.Y);

	const float OutputValue = GetOutputValue(ValueAttribute.Get());
	const float ClampedOutputValue = FMath::Clamp(OutputValue, OutputRange.X, OutputRange.Y);
	const float ClampedSliderValue = GetSliderValue(ClampedOutputValue);
	SetSliderValue(ClampedSliderValue);

	Label->UpdateValueTextWidth(OutputRange);
}

FVector2D SAudioMaterialLabeledKnob::ComputeDesiredSize(float) const
{
	if (DesiredSizeOverride.Get().IsSet())
	{
		return DesiredSizeOverride.Get().GetValue();
	}
	
	if (Style)
	{
		return FVector2D(Style->DesiredSize);
	}

	return FVector2D::ZeroVector;
}

void SAudioMaterialLabeledKnob::SetDesiredSizeOverride(const FVector2D Size)
{
	SetAttribute(DesiredSizeOverride, TAttribute<TOptional<FVector2D>>(Size), EInvalidateWidgetReason::Layout);
}

void SAudioMaterialLabeledKnob::SetLabelBackgroundColor(FSlateColor InColor)
{
	Label->SetLabelBackgroundColor(InColor);
}

void SAudioMaterialLabeledKnob::SetUnitsText(const FText Units)
{
	Label->SetUnitsText(Units);
}

void SAudioMaterialLabeledKnob::SetUnitsTextReadOnly(const bool bIsReadOnly)
{
	Label->SetUnitsTextReadOnly(bIsReadOnly);
}

void SAudioMaterialLabeledKnob::SetShowUnitsText(const bool bShowUnitsText)
{
	Label->SetShowUnitsText(bShowUnitsText);
}
