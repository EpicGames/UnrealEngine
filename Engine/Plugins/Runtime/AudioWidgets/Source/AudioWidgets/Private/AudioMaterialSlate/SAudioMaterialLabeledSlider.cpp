// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMaterialSlate/SAudioMaterialLabeledSlider.h"
#include "AudioMaterialSlate/SAudioMaterialSlider.h"
#include "SAudioTextBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SOverlay.h"

void SAudioMaterialLabeledSlider::Construct(const SAudioMaterialLabeledSlider::FArguments& InArgs)
{
	Style = InArgs._Style;
	OnValueChanged = InArgs._OnValueChanged;
	OnValueCommitted = InArgs._OnValueCommitted;
	Orientation = InArgs._Orientation;
	AudioUnitsValueType = InArgs._AudioUnitsValueType;
	DesiredSizeOverride = InArgs._DesiredSizeOverride;

	if (InArgs._SliderValue.IsSet())
	{
		SliderValueAttribute = InArgs._SliderValue;
	}

	// Text label
	SAssignNew(Label, SAudioTextBox)
		.Style(&Style->TextBoxStyle)
		.OnValueTextCommitted_Lambda([this](const FText& Text, ETextCommit::Type CommitType)
		{
			const float OutputValue = FCString::Atof(*Text.ToString());
			const float NewSliderValue = GetSliderValueForText(OutputValue);
			if (!FMath::IsNearlyEqual(NewSliderValue, SliderValueAttribute.Get()))
			{
				SliderValueAttribute.Set(NewSliderValue);
				Slider->SetValue(NewSliderValue);
				OnValueChanged.ExecuteIfBound(NewSliderValue);
				OnValueCommitted.ExecuteIfBound(NewSliderValue);
			}
		});

	// Underlying slider widget
	SAssignNew(Slider, SAudioMaterialSlider)
		.ValueAttribute(SliderValueAttribute.Get())
		.Owner(InArgs._Owner)
		.AudioMaterialSliderStyle(Style)
		.Orientation(Orientation.Get())
		.OnValueChanged_Lambda([this](float Value)
		{
			SliderValueAttribute.Set(Value);
			OnValueChanged.ExecuteIfBound(Value);
			const float OutputValue = GetOutputValueForText(Value);
			Label->SetValueText(OutputValue);
		})
		.OnValueCommitted_Lambda([this](float Value)
		{
			OnValueCommitted.ExecuteIfBound(SliderValueAttribute.Get());
		});

	//Set the Unit processor from Units Type
	switch (AudioUnitsValueType.Get())
	{
		case EAudioUnitsValueType::Linear:
			if (Label.IsValid())
			{
				AudioUnitProcessor = MakeShared<FAudioUnitProcessor>();
				Label.Get()->SetShowUnitsText(false);
			}
			break;
		case EAudioUnitsValueType::Frequency:
			AudioUnitProcessor = MakeShared<FFrequencyProcessor>();
			break;
		case EAudioUnitsValueType::Volume:
			AudioUnitProcessor = MakeShared<FVolumeProcessor>(InArgs._bUseLinearOutput);
			break;
	}

	if (AudioUnitProcessor.IsValid())
	{
		SetOutputRange(AudioUnitProcessor->GetDefaultOutputRange());
		if (Label.IsValid())
		{
			Label.Get()->SetUnitsText(AudioUnitProcessor->GetUnitsText());
		}
	}

	ChildSlot
	[
		CreateWidgetLayout()
	];
}

void SAudioMaterialLabeledSlider::SetSliderValue(float InSliderValue)
{
	SliderValueAttribute.Set(InSliderValue);
	const float OutputValueForText = GetOutputValueForText(InSliderValue);
	Label->SetValueText(OutputValueForText);
	Slider->SetValue(InSliderValue);
}

void SAudioMaterialLabeledSlider::SetOrientation(EOrientation InOrientation)
{
	SetAttribute(Orientation, TAttribute<EOrientation>(InOrientation), EInvalidateWidgetReason::Layout);
	Slider->SetOrientation(InOrientation);
	LayoutWidgetSwitcher->SetActiveWidgetIndex(Orientation.Get());
}

FVector2D SAudioMaterialLabeledSlider::ComputeDesiredSize(float) const
{
	if (DesiredSizeOverride.Get().IsSet())
	{
		return DesiredSizeOverride.Get().GetValue();
	}

	if (Style && Label.IsValid())
	{
		const float Width = Orientation.Get() == Orient_Vertical ? Label.Get()->GetDesiredSize().X + 6.f : Style->DesiredSize.Y + Label.Get()->GetDesiredSize().X;
		const float Heigth = Orientation.Get() == Orient_Vertical ? Style->DesiredSize.Y + Label.Get()->GetDesiredSize().Y + 3.f: Style->DesiredSize.X;
		return FVector2D(Width, Heigth);
	}

	return FVector2D::ZeroVector;
}

void SAudioMaterialLabeledSlider::SetDesiredSizeOverride(const FVector2D Size)
{
	SetAttribute(DesiredSizeOverride, TAttribute<TOptional<FVector2D>>(Size), EInvalidateWidgetReason::Layout);
}

const float SAudioMaterialLabeledSlider::GetOutputValue(const float InSliderValue)
{	
	return AudioUnitProcessor.IsValid() == true ? AudioUnitProcessor->GetOutputValue(OutputRange, InSliderValue) : 0.0f;
}

const float SAudioMaterialLabeledSlider::GetOutputValueForText(const float InSliderValue)
{
	return AudioUnitProcessor.IsValid() == true ? AudioUnitProcessor->GetOutputValueForText(OutputRange, InSliderValue) : 0.0f;
}

const float SAudioMaterialLabeledSlider::GetSliderValueForText(const float OutputValue)
{
	return AudioUnitProcessor.IsValid() == true ? AudioUnitProcessor->GetSliderValueForText(OutputRange, OutputValue) : 0.0f;
}

const float SAudioMaterialLabeledSlider::GetSliderValue(const float OutputValue)
{
	return AudioUnitProcessor.IsValid() == true ? AudioUnitProcessor->GetSliderValue(OutputRange, OutputValue) : 0.0f;
}

void SAudioMaterialLabeledSlider::SetOutputRange(const FVector2D InRange)
{	
	//Check the valid range from the processor
	FVector2D Range = AudioUnitProcessor.IsValid() == true ? AudioUnitProcessor->GetOutputRange(InRange) : InRange;
	OutputRange = Range;

	// if Range.Y < Range.X, set Range.X to Range.Y
	OutputRange.X = FMath::Min(Range.X, Range.Y);

	const float OutputValue = GetOutputValue(SliderValueAttribute.Get());
	const float ClampedOutputValue = FMath::Clamp(OutputValue, OutputRange.X, OutputRange.Y);
	const float ClampedSliderValue = GetSliderValue(ClampedOutputValue);
	SetSliderValue(ClampedSliderValue);

	Label->UpdateValueTextWidth(OutputRange);
}

void SAudioMaterialLabeledSlider::SetLabelBackgroundColor(FSlateColor InColor)
{
	Label->SetLabelBackgroundColor(InColor.GetSpecifiedColor());
}

void SAudioMaterialLabeledSlider::SetUnitsText(const FText Units)
{
	Label->SetUnitsText(Units);
}

void SAudioMaterialLabeledSlider::SetUnitsTextReadOnly(const bool bIsReadOnly)
{
	Label->SetUnitsTextReadOnly(bIsReadOnly);
}

void SAudioMaterialLabeledSlider::SetValueTextReadOnly(const bool bIsReadOnly)
{
	Label->SetValueTextReadOnly(bIsReadOnly);
}

void SAudioMaterialLabeledSlider::SetShowLabelOnlyOnHover(const bool bShowLabelOnlyOnHover)
{
	Label->SetShowLabelOnlyOnHover(bShowLabelOnlyOnHover);
}

void SAudioMaterialLabeledSlider::SetShowUnitsText(const bool bShowUnitsText)
{
	Label->SetShowUnitsText(bShowUnitsText);
}

TSharedRef<SWidgetSwitcher> SAudioMaterialLabeledSlider::CreateWidgetLayout()
{
	SAssignNew(LayoutWidgetSwitcher, SWidgetSwitcher);
	// Create overall layout
	// Horizontal orientation
	LayoutWidgetSwitcher->AddSlot(EOrientation::Orient_Horizontal)
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				// SSlider
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(3.0f, 0.0f)
				[
					Slider.ToSharedRef()
				]
			]
			// Text Label
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(3.0f, 0.0f, 0.0f, 0.0f)
			[
				Label.ToSharedRef()
			]
		]	
	];
	// Vertical orientation
	LayoutWidgetSwitcher->AddSlot(EOrientation::Orient_Vertical)
	[
		SNew(SOverlay)
		+ SOverlay::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Fill)
		[
			SNew(SVerticalBox)
			// Text Label
			+ SVerticalBox::Slot()
			.Padding(0.0f, 0.0f, 0.0f, 3.0f)
			.AutoHeight()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				Label.ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Fill)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				// Actual SSlider
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Fill)
				.Padding(0.0f, 3.0f)
				[
					Slider.ToSharedRef()
				]
			]
		]		
	];
	LayoutWidgetSwitcher->SetActiveWidgetIndex(Orientation.Get());
	SetOrientation(Orientation.Get());

	return LayoutWidgetSwitcher.ToSharedRef();
}
