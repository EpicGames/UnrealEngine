// Copyright Epic Games, Inc. All Rights Reserved.


#include "AudioMaterialSlate/SAudioMaterialMeter.h"
#include "AudioMaterialSlate/AudioMaterialMeter.h"
#include "Components/AudioComponent.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "SlateOptMacros.h"
#include "Styling/SlateBrush.h"
#include "Styling/StyleDefaults.h"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SAudioMaterialMeter::Construct(const FArguments& InArgs)
{
	Owner = InArgs._Owner;
	Orientation = InArgs._Orientation;

	Style = InArgs._AudioMaterialMeterStyle;
	MeterChannelInfoAttribute = InArgs._MeterChannelInfo;

	ApplyNewMaterial();
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

int32 SAudioMaterialMeter::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	if (Style && !DynamicMaterials.IsEmpty())
	{
		const float AllottedWidth = Orientation == Orient_Vertical ? AllottedGeometry.GetLocalSize().X : AllottedGeometry.GetLocalSize().Y;
		const float AllottedHeight = Orientation == Orient_Vertical ? AllottedGeometry.GetLocalSize().Y : AllottedGeometry.GetLocalSize().X;

		FGeometry MeterGeometry = AllottedGeometry;

		// Get the scale hash offset
		float ScaleOffset = 0.0f;
		if (Style->bShowScale && Style->bScaleSide)
		{
			ScaleOffset = GetScaleWidth();
		}

		TArray<FMeterChannelInfo> ChannelInfos = MeterChannelInfoAttribute.Get();
		int32 NumChannels = ChannelInfos.Num();

		const FLinearColor FinalColorAndOpacity(InWidgetStyle.GetColorAndOpacityTint());

		//Draw Meter for every channel
		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
		{	
			TWeakObjectPtr<UMaterialInstanceDynamic> DynamicMaterial = DynamicMaterials[ChannelIndex];

			if (DynamicMaterial.IsValid())
			{
				DynamicMaterial->SetVectorParameterValue(FName("A (V3)"), Style->MeterFillMinColor);
				DynamicMaterial->SetVectorParameterValue(FName("B (V3)"), Style->MeterFillMidColor);
				DynamicMaterial->SetVectorParameterValue(FName("C (V3)"), Style->MeterFillMaxColor);
				DynamicMaterial->SetVectorParameterValue(FName("OffColor"), Style->MeterFillBackgroundColor);
				DynamicMaterial->SetVectorParameterValue(FName("DotsOffColor"), Style->MeterFillBackgroundColor);

				float ChannelMeterValueDb = FMath::GetMappedRangeValueClamped(Style->ValueRangeDb,FVector2D(0.f,1.f),  ChannelInfos[ChannelIndex].MeterValue);
				DynamicMaterial->SetScalarParameterValue(FName("VALUE"), ChannelMeterValueDb);

				DynamicMaterial->SetScalarParameterValue(FName("LocalWidth"), AllottedGeometry.GetLocalSize().X);
				DynamicMaterial->SetScalarParameterValue(FName("LocalHeigth"), AllottedGeometry.GetLocalSize().Y);			

				const bool bEnabled = ShouldBeEnabled(bParentEnabled);
				const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
			
				FSlateBrush Brush;
				Brush.SetResourceObject(DynamicMaterial.Get());

				// rotate the meter 90deg if should be horizontal.
				if (Orientation == Orient_Horizontal)
				{
					//rotate
					FSlateRenderTransform SlateRenderTransform = TransformCast<FSlateRenderTransform>(Concatenate(Inverse(FVector2D(0, AllottedHeight)), FQuat2D(FMath::DegreesToRadians(90.0f))));
					// create a child geometry matching this one, but with the render transform that will be passed to the drawed Meter.
					MeterGeometry = AllottedGeometry.MakeChild(
						FVector2D(AllottedWidth, AllottedHeight),
						FSlateLayoutTransform(),
						SlateRenderTransform, FVector2D::ZeroVector);
				}
	
				FVector2D MeterTopLeft = FVector2D(Style->MeterPadding.X + ScaleOffset + ChannelIndex * (Style->DesiredSize.X + Style->MeterPadding.X), Style->MeterPadding.Y);
				FVector2D MeterSize = FVector2D(Style->DesiredSize.X, AllottedHeight - Style->MeterPadding.Y);

				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId,
					MeterGeometry.ToPaintGeometry(MeterSize, FSlateLayoutTransform(MeterTopLeft)),	
					&Brush,
					DrawEffects,
					FinalColorAndOpacity);
			}
			else
			{
				if (Style)
				{
					DynamicMaterials[ChannelIndex] = Style->CreateDynamicMaterial(Owner.Get());
				}
			}
		}

		// Draw the scale hash
		if (Style->bShowScale)
		{

			int32 DecibelsPerHash = Style->DecibelsPerHash;
			int32 MinValueDb = FMath::Min((int32)Style->ValueRangeDb.X, (int32)Style->ValueRangeDb.Y);
			int32 MaxValueDb = FMath::Max((int32)Style->ValueRangeDb.Y, (int32)Style->ValueRangeDb.Y);

			// Snap the min/max values to the nearest hash 
			MaxValueDb -= MaxValueDb % DecibelsPerHash;
			MinValueDb -= MinValueDb % DecibelsPerHash;

			float ScaleHashHalfHeight = 0.5f * Style->ScaleHashHeight;
			FVector2D HashSize = FVector2D(Style->ScaleHashWidth, Style->ScaleHashHeight);

			const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

			// Measure the min value label size so we can right-justify if needed
			FVector2D MinValueLabelSize = FontMeasureService->Measure(FString::FromInt(MinValueDb), Style->Font);

			// Get the size of the negative sign to use to offset the label text in horizontal mode
			FVector2D NegativeSignSize = FVector2D::ZeroVector;
			if (Orientation == Orient_Horizontal)
			{
				NegativeSignSize = FontMeasureService->Measure(TEXT("-"), Style->Font);
			}

			int32 ValueDelta = MinValueDb - MaxValueDb;
			int32 CurrentHashValue = MaxValueDb;
			while (CurrentHashValue >= MinValueDb)
			{
				// Get the fractional value for hash mark
				const float CurrentHashMeterValuePercent = FMath::Clamp(((float)CurrentHashValue - (float)MaxValueDb) / ValueDelta, 0.0f, 1.0f);
			
				// Get the Y location for the hash
				float HashPixelCenter = CurrentHashMeterValuePercent * (AllottedHeight - 2.0f * Style->MeterPadding.Y);

				FVector2D HashTopLeft;
				HashTopLeft.Y = Style->MeterPadding.Y + HashPixelCenter - ScaleHashHalfHeight;
				if (Style->bScaleSide)
				{
					HashTopLeft.X = Style->MeterPadding.X + ScaleOffset - Style->ScaleHashOffset - Style->ScaleHashWidth;
				}
				else
				{
					HashTopLeft.X = (Style->DesiredSize.X + Style->MeterPadding.X) * NumChannels + Style->ScaleHashOffset;
				}

				// Draw hash
				FSlateBrush HashBrush;
				FSlateDrawElement::MakeBox(
					OutDrawElements,
					LayerId,
					MeterGeometry.ToPaintGeometry(HashSize, FSlateLayoutTransform(HashTopLeft)),
					&HashBrush,
					ESlateDrawEffect::None,
					FinalColorAndOpacity);

				FString LabelString = FString::FromInt(CurrentHashValue);

				bool bIsNegative = (CurrentHashValue < 0);

				FVector2D LabelSize;

				if (Orientation == Orient_Horizontal && bIsNegative)
				{
					// We want to center the text on just the positive portion of the number
					FString LabelStringPositive = FString::FromInt(FMath::Abs(CurrentHashValue));
					LabelSize = FontMeasureService->Measure(LabelStringPositive, Style->Font);
				}
				else
				{
					LabelSize = FontMeasureService->Measure(LabelString, Style->Font);
				}

				FText LabelText = FText::FromString(LabelString);

				FGeometry TextGeometry = MeterGeometry;
				FVector2D LabelTopLeft;

				if (Orientation == Orient_Horizontal)
				{
					//Calculate offset from TopLeft
					LabelTopLeft.Y = Style->MeterPadding.Y + HashPixelCenter + 0.5f * LabelSize.X;

					if (bIsNegative)
					{
						LabelTopLeft.Y += NegativeSignSize.X;
					}

					if (Style->bScaleSide)
					{
						LabelTopLeft.X = Style->MeterPadding.X - 2.0f + (MinValueLabelSize.Y - LabelSize.Y);
					}
					else
					{
						LabelTopLeft.X = (Style->DesiredSize.X + Style->MeterPadding.X) * NumChannels + Style->ScaleHashOffset + Style->ScaleHashWidth + 2.0f;
					}

					// Undo the rotation for horizontal before we do the rendering of the scale value 
					FSlateRenderTransform RotationTransform = FSlateRenderTransform(FQuat2D(FMath::DegreesToRadians(-90.0f)));

					FSlateLayoutTransform ChildLayoutTransform(1.0f, TransformPoint(1.0f, LabelTopLeft));

					TextGeometry = MeterGeometry.MakeChild(
						LabelSize,
						ChildLayoutTransform,
						RotationTransform,
						FVector2D(0.0f, 0.0f));
				}
				else//Vertical
				{
					//Calculate offset from TopLeft
					LabelTopLeft.Y = Style->MeterPadding.Y + HashPixelCenter - 0.5f * LabelSize.Y;

					if (Style->bScaleSide)
					{
						LabelTopLeft.X = Style->MeterPadding.X + ScaleOffset - 2.0f - LabelSize.X - Style->ScaleHashOffset - Style->ScaleHashWidth;
					}
					else
					{
						LabelTopLeft.X = (Style->DesiredSize.X + Style->MeterPadding.X) * NumChannels + Style->ScaleHashOffset + Style->ScaleHashWidth + 2.0f;
					}

					FSlateRenderTransform RotationTransform = FSlateRenderTransform(FQuat2D(FMath::DegreesToRadians(0.0f)));

					FSlateLayoutTransform ChildLayoutTransform(1.0f, TransformPoint(1.0f, LabelTopLeft));

					TextGeometry = MeterGeometry.MakeChild(
						LabelSize,
						ChildLayoutTransform,
						RotationTransform,
						FVector2D(0.0f, 0.0f));
				}

				// Draw text label
				FSlateDrawElement::MakeText(
					OutDrawElements,
					LayerId,
					TextGeometry.ToPaintGeometry(),
					LabelText,
					Style->Font,
					ESlateDrawEffect::None,
					FinalColorAndOpacity);

				CurrentHashValue -= Style->DecibelsPerHash;
			}
		}
	}

	return LayerId;
}

FVector2D SAudioMaterialMeter::ComputeDesiredSize(float) const
{
	static const FVector2D DefaultMeterDesiredSize(50.0f, 50.0f);

	if (Style == nullptr)
	{
		return DefaultMeterDesiredSize;
	}

	TArray<FMeterChannelInfo> ChannelInfo = MeterChannelInfoAttribute.Get();
	int32 NumChannels = FMath::Max(ChannelInfo.Num(), 1);

	FVector2D Size = FVector2D((Style->DesiredSize.X + Style->MeterPadding.X) * NumChannels, Style->DesiredSize.Y+ Style->MeterPadding.Y);

	// remove the end padding
	Size.X += Style->MeterPadding.X;
	Size.Y += Style->MeterPadding.Y;

	// Add the width for the scale if it's been set to show
	if (Style->bShowScale)
	{
		Size.X += GetScaleWidth();
	}

	if (Orientation == Orient_Horizontal)
	{
		return FVector2D(Size.Y, Size.X);
	}

	return Size;
}

void SAudioMaterialMeter::SetOrientation(EOrientation InOrientation)
{
	if (Orientation != InOrientation)
	{
		Orientation = InOrientation;
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

TArray<TWeakObjectPtr<UMaterialInstanceDynamic>> SAudioMaterialMeter::ApplyNewMaterial()
{
	int32 NumChannels = MeterChannelInfoAttribute.Get().Num();

	DynamicMaterials.SetNum(NumChannels);
	DynamicMaterials.Empty();

	if (Style)
	{
		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
		{
			UMaterialInstanceDynamic* MaterialInstance = Style->CreateDynamicMaterial(Owner.Get());
			DynamicMaterials.Add(MaterialInstance);
		}
	}

	return DynamicMaterials;
}

void SAudioMaterialMeter::SetMeterChannelInfo(const TAttribute<TArray<FMeterChannelInfo>>& InMeterChannelInfo)
{
	SetAttribute(MeterChannelInfoAttribute, InMeterChannelInfo, EInvalidateWidgetReason::Paint);
	ApplyNewMaterial();
}

TArray<FMeterChannelInfo> SAudioMaterialMeter::GetMeterChannelInfo() const
{
	return MeterChannelInfoAttribute.Get();
}

float SAudioMaterialMeter::GetScaleWidth() const
{
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	FVector2D LabelSize = FontMeasureService->Measure(FString::FromInt(-60), Style->Font);

	float ScaleWidth = Style->ScaleHashWidth + Style->ScaleHashOffset;
	if (Orientation == Orient_Horizontal)
	{
		return ScaleWidth + LabelSize.Y;
	}
	return ScaleWidth + LabelSize.X;
}
