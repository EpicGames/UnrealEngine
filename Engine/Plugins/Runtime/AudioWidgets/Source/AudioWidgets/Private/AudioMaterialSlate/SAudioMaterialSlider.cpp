// Copyright Epic Games, Inc. All Rights Reserved.


#include "AudioMaterialSlate/SAudioMaterialSlider.h"
#include "AudioMaterialSlate/AudioMaterialSlider.h"
#include "AudioWidgetsStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "SlateOptMacros.h"
#include "Styling/SlateBrush.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SAudioMaterialSlider::Construct(const FArguments& InArgs)
{
	Owner = InArgs._Owner;

	TuneSpeed = InArgs._TuneSpeed;
	FineTuneSpeed = InArgs._FineTuneSpeed;
	bIsFocusable = InArgs._IsFocusable;
	Orientation = InArgs._Orientation;
	bMouseUsesStep = InArgs._MouseUsesStep;
	bLocked = InArgs._Locked;
	StepSize = InArgs._StepSize;

	AudioMaterialSliderStyle = InArgs._AudioMaterialSliderStyle;

	OnValueChanged = InArgs._OnValueChanged;
	OnValueCommitted = InArgs._OnValueCommitted;
	OnMouseCaptureBegin = InArgs._OnMouseCaptureBegin;
	OnMouseCaptureEnd = InArgs._OnMouseCaptureEnd;

	ApplyNewMaterial();

	if (InArgs._ValueAttribute.IsSet())
	{
		CommitValue(InArgs._ValueAttribute.Get());
	}
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

int32 SAudioMaterialSlider::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	if (AudioMaterialSliderStyle)
	{
		if (DynamicMaterial.IsValid())
		{
			const float Value = ValueAttribute.Get();
			DynamicMaterial.Get()->SetVectorParameterValue(FName("BarColor"), AudioMaterialSliderStyle->SliderBackgroundColor);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("LedColor"), AudioMaterialSliderStyle->SliderValueMainColor);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("BarInnerShadow"), AudioMaterialSliderStyle->SliderBackgroundAccentColor);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("ValueColor"), AudioMaterialSliderStyle->SliderHandleMainColor);
			DynamicMaterial.Get()->SetVectorParameterValue(FName("ValueColorEdge"), AudioMaterialSliderStyle->SliderHandleOutlineColor);
			DynamicMaterial.Get()->SetScalarParameterValue(FName("VALUE"), FMath::Clamp(Value, 0.f, 1.f));
			DynamicMaterial.Get()->SetScalarParameterValue(FName("LedInt"), FMath::GetMappedRangeValueClamped(FVector2D(0.f,1.f),FVector2D(0.7f,4.f), Value));

			DynamicMaterial.Get()->SetScalarParameterValue(FName("LocalWidth"), AllottedGeometry.GetLocalSize().X);
			DynamicMaterial.Get()->SetScalarParameterValue(FName("LocalHeigth"), AllottedGeometry.GetLocalSize().Y);

			// We draw the slider as a vertical slider regardless of the orientation and apply a render transform to make it display in the correct orientation.
			FGeometry SliderGeometry = AllottedGeometry;
			const float AllottedWidth = Orientation == Orient_Vertical ? AllottedGeometry.GetLocalSize().X : AllottedGeometry.GetLocalSize().Y;
			const float AllottedHeight = Orientation == Orient_Vertical ? AllottedGeometry.GetLocalSize().Y : AllottedGeometry.GetLocalSize().X;

			// rotate the slider 90deg if should be horizontal.
			if (Orientation == Orient_Horizontal)
			{
				// rotate
				FSlateRenderTransform SlateRenderTransform = TransformCast<FSlateRenderTransform>(Concatenate(Inverse(FVector2f(0, AllottedHeight)), FQuat2D(FMath::DegreesToRadians(90.0f))));
				// create a child geometry matching this one, but with the rotated render transform that will be passed to the drawed slider.
				SliderGeometry = AllottedGeometry.MakeChild(
					FVector2f(AllottedWidth, AllottedHeight),
					FSlateLayoutTransform(),
					SlateRenderTransform, FVector2f::ZeroVector);
			}

			const bool bEnabled = ShouldBeEnabled(bParentEnabled);
			const ESlateDrawEffect DrawEffects = bEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
				
			const FLinearColor FinalColorAndOpacity(InWidgetStyle.GetColorAndOpacityTint());

			FSlateBrush Brush;
			Brush.SetResourceObject(DynamicMaterial.Get());
			FSlateDrawElement::MakeBox(OutDrawElements, LayerId++, SliderGeometry.ToPaintGeometry(FVector2D(AllottedWidth, AllottedHeight), FSlateLayoutTransform()), &Brush, DrawEffects, FinalColorAndOpacity);

		}
		else
		{
			if (AudioMaterialSliderStyle)
			{
				DynamicMaterial = AudioMaterialSliderStyle->CreateDynamicMaterial(Owner.Get());
			}
		}
	}

	return LayerId;
}

FVector2D SAudioMaterialSlider::ComputeDesiredSize(float) const
{
	if (DesiredSizeOverride.Get().IsSet())
	{
		return DesiredSizeOverride.Get().GetValue();
	}

	if (AudioMaterialSliderStyle)
	{
		const float Width = Orientation == Orient_Vertical ? AudioMaterialSliderStyle->DesiredSize.X : AudioMaterialSliderStyle->DesiredSize.Y;
		const float Heigth = Orientation == Orient_Vertical ? AudioMaterialSliderStyle->DesiredSize.Y : AudioMaterialSliderStyle->DesiredSize.X;
		return FVector2D(Width, Heigth);
	}

	return FVector2D::ZeroVector;
}

void SAudioMaterialSlider::SetValue(TAttribute<float> InValueAttribute)
{
	ValueAttribute = InValueAttribute;
}

void SAudioMaterialSlider::SetTuneSpeed(const float InMouseTuneSpeed)
{
	TuneSpeed.Set(InMouseTuneSpeed);
}

void SAudioMaterialSlider::SetFineTuneSpeed(const float InMouseFineTuneSpeed)
{
	FineTuneSpeed.Set(InMouseFineTuneSpeed);
}

void SAudioMaterialSlider::SetMouseUsesStep(const bool InUsesStep)
{
	bMouseUsesStep.Set(InUsesStep);
}

void SAudioMaterialSlider::SetStepSize(const float InStepSize)
{
	StepSize.Set(InStepSize);
}

void SAudioMaterialSlider::SetLocked(const bool bInLocked)
{
	bLocked.Set(bInLocked);
}

bool SAudioMaterialSlider::IsLocked() const
{
	return bLocked.Get();
}

void SAudioMaterialSlider::ApplyNewMaterial()
{
	if (AudioMaterialSliderStyle)
	{
		DynamicMaterial = AudioMaterialSliderStyle->CreateDynamicMaterial(Owner.Get());
	}
}

void SAudioMaterialSlider::SetOrientation(EOrientation InOrientation)
{
	if (Orientation != InOrientation)
	{
		Orientation = InOrientation;
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

FReply SAudioMaterialSlider::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && !IsLocked())
	{	
		CachedCursor = GetCursor().Get(EMouseCursor::Default);

		MouseDownStartPosition = MouseEvent.GetScreenSpacePosition();
		MouseDownValue = ValueAttribute.Get();
		OnMouseCaptureBegin.ExecuteIfBound();
		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}

FReply SAudioMaterialSlider::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ((MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton) && HasMouseCapture())
	{
		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

FReply SAudioMaterialSlider::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (this->HasMouseCapture() && !IsLocked())
	{
		SetCursor(EMouseCursor::GrabHandClosed);

		int32 MouseCurrentPosition = Orientation == Orient_Horizontal ? MouseEvent.GetLastScreenSpacePosition().X : MouseEvent.GetLastScreenSpacePosition().Y;
		float MouseDownPosition = Orientation == Orient_Horizontal ? MouseDownStartPosition.X : MouseDownStartPosition.Y;
		const float Speed = bIsFineTune ? FineTuneSpeed.Get() : TuneSpeed.Get();

		float ValueDelta = (float)(MouseDownPosition - MouseCurrentPosition) / PixelDelta * Speed;

		float NewValue = Orientation == Orient_Horizontal ? FMath::Clamp(MouseDownValue - ValueDelta, 0.0f, 1.0f) : FMath::Clamp(MouseDownValue + ValueDelta, 0.0f, 1.0f);

		if (bMouseUsesStep.Get())
		{
			const float SteppedValue = FMath::RoundToInt(NewValue / StepSize.Get()) * StepSize.Get();
			NewValue = SteppedValue;
		}

		CommitValue(NewValue);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SAudioMaterialSlider::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	SetCursor(CachedCursor);
	OnValueCommitted.ExecuteIfBound(ValueAttribute.Get());
	OnMouseCaptureEnd.ExecuteIfBound();
	SLeafWidget::OnMouseCaptureLost(CaptureLostEvent);
}

FReply SAudioMaterialSlider::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::LeftShift)
	{
		MouseDownStartPosition = FSlateApplication::Get().GetCursorPos();
		MouseDownValue = ValueAttribute.Get();
		bIsFineTune = true;
	}

	return FReply::Unhandled();
}

FReply SAudioMaterialSlider::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	bIsFineTune = false;
	MouseDownStartPosition = FSlateApplication::Get().GetCursorPos();
	MouseDownValue = ValueAttribute.Get();

	return FReply::Unhandled();
}

bool SAudioMaterialSlider::SupportsKeyboardFocus() const
{
	return bIsFocusable.Get();
}

bool SAudioMaterialSlider::IsInteractable() const
{
	return IsEnabled() && !IsLocked() && SupportsKeyboardFocus();
}

void SAudioMaterialSlider::CommitValue(float NewValue)
{
	const float OldValue = ValueAttribute.Get();
	float Val = FMath::Clamp(NewValue, 0.f , 1.f);

	if (NewValue != OldValue)
	{
		if (!ValueAttribute.IsBound())
		{
			ValueAttribute.Set(Val);
		}

		Invalidate(EInvalidateWidgetReason::Paint);
		OnValueChanged.ExecuteIfBound(Val);
	}
}
