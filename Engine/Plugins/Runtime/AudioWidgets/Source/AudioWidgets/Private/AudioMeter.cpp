// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMeter.h"

#include "AudioBusSubsystem.h"
#include "AudioMaterialSlate/SAudioMaterialMeter.h"
#include "AudioMixerDevice.h"
#include "AudioWidgetsStyle.h"
#include "Engine/Engine.h"
#include "SAudioMeter.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Docking/SDockTab.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioMeter)

#define LOCTEXT_NAMESPACE "AUDIO_UMG"
UAudioMeter::UAudioMeter(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Orientation = EOrientation::Orient_Vertical;

	// TODO: Move to style set
	BackgroundColor = FLinearColor(0.0075f, 0.0075f, 0.0075f, 1.0f);
	MeterBackgroundColor = FLinearColor(0.031f, 0.031f, 0.031f, 1.0f);
	MeterValueColor = FLinearColor(0.025719f, 0.208333f, 0.069907f, 1.0f);
	MeterPeakColor = FLinearColor(0.24349f, 0.708333f, 0.357002f, 1.0f);
	MeterClippingColor = FLinearColor::Transparent;
	MeterScaleColor = FLinearColor(0.017642f, 0.017642f, 0.017642f, 1.0f);
	MeterScaleLabelColor = FLinearColor(0.442708f, 0.442708f, 0.442708f, 1.0f);

	// Add a single channel as a default just so it can be seen when somebody makes one
	FMeterChannelInfo DefaultInfo;
	DefaultInfo.MeterValue = -6.0f;
	DefaultInfo.PeakValue = -3.0f;
	MeterChannelInfo.Add(DefaultInfo);

	WidgetStyle = FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioMeterStyle>("AudioMeter.Style");

#if WITH_EDITORONLY_DATA
	AccessibleBehavior = ESlateAccessibleBehavior::NotAccessible;
	bCanChildrenBeAccessible = false;
#endif
}

TSharedRef<SWidget> UAudioMeter::RebuildWidget()
{
	MyAudioMeter = SNew(SAudioMeter)
		.Style(&WidgetStyle);

	return MyAudioMeter.ToSharedRef();
}

void UAudioMeter::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	MyAudioMeter->SetOrientation(Orientation);

	MyAudioMeter->SetBackgroundColor(BackgroundColor);
	MyAudioMeter->SetMeterBackgroundColor(MeterBackgroundColor);
	MyAudioMeter->SetMeterValueColor(MeterValueColor);
	MyAudioMeter->SetMeterPeakColor(MeterPeakColor);
	MyAudioMeter->SetMeterClippingColor(MeterClippingColor);
	MyAudioMeter->SetMeterScaleColor(MeterScaleColor);
	MyAudioMeter->SetMeterScaleLabelColor(MeterScaleLabelColor);

	TAttribute<TArray<FMeterChannelInfo>> MeterChannelInfoBinding = PROPERTY_BINDING(TArray<FMeterChannelInfo>, MeterChannelInfo);
	MyAudioMeter->SetMeterChannelInfo(MeterChannelInfoBinding);
}

void UAudioMeter::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyAudioMeter.Reset();
}

TArray<FMeterChannelInfo> UAudioMeter::GetMeterChannelInfo() const
{
	if (MyAudioMeter.IsValid())
	{
		return MyAudioMeter->GetMeterChannelInfo();
	}
	return TArray<FMeterChannelInfo>();
}

void UAudioMeter::SetMeterChannelInfo(const TArray<FMeterChannelInfo>& InMeterChannelInfo)
{
	if (MyAudioMeter.IsValid())
	{
		MyAudioMeter->SetMeterChannelInfo(InMeterChannelInfo);
	}
}

void UAudioMeter::SetBackgroundColor(FLinearColor InValue)
{
	BackgroundColor = InValue;
	if (MyAudioMeter.IsValid())
	{
		MyAudioMeter->SetBackgroundColor(InValue);
	}
}

void UAudioMeter::SetMeterBackgroundColor(FLinearColor InValue)
{
	MeterBackgroundColor = InValue;
	if (MyAudioMeter.IsValid())
	{
		MyAudioMeter->SetMeterBackgroundColor(InValue);
	}
}

void UAudioMeter::SetMeterValueColor(FLinearColor InValue)
{
	MeterValueColor = InValue;
	if (MyAudioMeter.IsValid())
	{
		MyAudioMeter->SetMeterValueColor(InValue);
	}
}

void UAudioMeter::SetMeterPeakColor(FLinearColor InValue)
{
	MeterPeakColor = InValue;
	if (MyAudioMeter.IsValid())
	{
		MyAudioMeter->SetMeterPeakColor(InValue);
	}
}

void UAudioMeter::SetMeterClippingColor(FLinearColor InValue)
{
	MeterClippingColor = InValue;
	if (MyAudioMeter.IsValid())
	{
		MyAudioMeter->SetMeterClippingColor(InValue);
	}
}

void UAudioMeter::SetMeterScaleColor(FLinearColor InValue)
{
	MeterScaleColor = InValue;
	if (MyAudioMeter.IsValid())
	{
		MyAudioMeter->SetMeterScaleColor(InValue);
	}
}

void UAudioMeter::SetMeterScaleLabelColor(FLinearColor InValue)
{
	MeterScaleLabelColor = InValue;
	if (MyAudioMeter.IsValid())
	{
		MyAudioMeter->SetMeterScaleLabelColor(InValue);
	}
}

#if WITH_EDITOR
const FText UAudioMeter::GetPaletteCategory()
{
	return LOCTEXT("Audio", "Audio");
}
#endif

const FAudioMeterDefaultColorStyle& FAudioMeterDefaultColorStyle::GetDefault()
{
	static FAudioMeterDefaultColorStyle Style;
	return Style;
}

namespace AudioWidgets
{
	const FAudioAnalyzerRackUnitTypeInfo FAudioMeter::RackUnitTypeInfo
	{
		.TypeName = TEXT("FAudioMeter"),
		.DisplayName = LOCTEXT("AudioMeterDisplayName", "Meter"),
		.OnMakeAudioAnalyzerRackUnit = FOnMakeAudioAnalyzerRackUnit::CreateStatic(&MakeRackUnit),
		.VerticalSizeCoefficient = 0.5f,
	};

	FAudioMeter::FAudioMeter(int32 InNumChannels, UWorld& InWorld, TObjectPtr<UAudioBus> InExternalAudioBus)
		: FAudioMeter(InNumChannels, InWorld.GetAudioDevice().GetDeviceID(), InExternalAudioBus)
	{
	}

	FAudioMeter::FAudioMeter(const int32 InNumChannels, const Audio::FDeviceId InAudioDeviceId, const TObjectPtr<UAudioBus> InExternalAudioBus, const FAudioMeterDefaultColorStyle* AudioMeterWidgetStyle)
	{
		const FAudioMeterDefaultColorStyle* WidgetStyle = AudioMeterWidgetStyle ? AudioMeterWidgetStyle : &FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioMeterDefaultColorStyle>("AudioMeter.DefaultColorStyle");
		Widget = SNew(SAudioMeter)
			.Orientation(EOrientation::Orient_Vertical)
			.BackgroundColor(FLinearColor::Transparent)
			.MeterBackgroundColor(WidgetStyle->MeterBackgroundColor)
			.MeterValueColor(WidgetStyle->MeterValueColor)
			.MeterPeakColor(WidgetStyle->MeterPeakColor)
			.MeterClippingColor(WidgetStyle->MeterClippingColor)
			.MeterScaleColor(WidgetStyle->MeterScaleColor)
			.MeterScaleLabelColor(WidgetStyle->MeterScaleLabelColor);

		Init(InNumChannels, InAudioDeviceId, InExternalAudioBus);
	}

	FAudioMeter::FAudioMeter(const int32 InNumChannels, const Audio::FDeviceId InAudioDeviceId, const FAudioMaterialMeterStyle& AudioMaterialMeterStyle, TObjectPtr<UAudioBus> InExternalAudioBus)
	{
		Widget = SNew(SAudioMaterialMeter)
			.AudioMaterialMeterStyle(&AudioMaterialMeterStyle);

		Init(InNumChannels, InAudioDeviceId, InExternalAudioBus);
	}

	FAudioMeter::~FAudioMeter()
	{
		Teardown();
	}

	UAudioBus* FAudioMeter::GetAudioBus() const
	{
		return AudioBus.Get();
	}

	TSharedRef<SAudioMeter> FAudioMeter::GetWidget() const
	{
		return StaticCastSharedRef<SAudioMeter>(Widget->AsShared());
	}

	void FAudioMeter::Init(int32 InNumChannels, UWorld& InWorld, TObjectPtr<UAudioBus> InExternalAudioBus)
	{
		const FAudioDeviceHandle AudioDevice = InWorld.GetAudioDevice();
		if (!AudioDevice.IsValid())
		{
			return;
		}

		Init(InNumChannels, AudioDevice.GetDeviceID(), InExternalAudioBus);
	}

	void FAudioMeter::Init(const int32 InNumChannels, const Audio::FDeviceId InAudioDeviceId, const TObjectPtr<UAudioBus> InExternalAudioBus)
	{
		Teardown();

		Settings = TStrongObjectPtr(NewObject<UMeterSettings>());
		Settings->PeakHoldTime = 4000.0f;

		// Only create analyzers etc if we have an audio device:
		if (InAudioDeviceId != FAudioBusInfo::InvalidAudioDeviceId)
		{
			check(InNumChannels > 0);

			Analyzer = TStrongObjectPtr(NewObject<UMeterAnalyzer>());
			Analyzer->Settings = Settings.Get();

			bUseExternalAudioBus = InExternalAudioBus != nullptr;

			AudioBus = bUseExternalAudioBus ? TStrongObjectPtr(InExternalAudioBus.Get()) : TStrongObjectPtr(NewObject<UAudioBus>());
			AudioBus->AudioBusChannels = EAudioBusChannels(InNumChannels - 1);

			ResultsDelegateHandle = Analyzer->OnLatestPerChannelMeterResultsNative.AddRaw(this, &FAudioMeter::OnMeterOutput);

			Analyzer->StartAnalyzing(InAudioDeviceId, AudioBus.Get());
		}

		constexpr float DefaultMeterValue = MIN_VOLUME_DECIBELS;
		constexpr float DefaultPeakValue  = MIN_VOLUME_DECIBELS;
		ChannelInfo.Init(FMeterChannelInfo{ DefaultMeterValue, DefaultPeakValue }, InNumChannels);

		if (Widget.IsValid())
		{
			Widget->SetMeterChannelInfo(ChannelInfo);
		}
	}

	void FAudioMeter::OnMeterOutput(UMeterAnalyzer* InMeterAnalyzer, int32 ChannelIndex, const FMeterResults& InMeterResults)
	{
		if (InMeterAnalyzer == Analyzer.Get())
		{
			FMeterChannelInfo NewChannelInfo;
			NewChannelInfo.MeterValue = InMeterResults.MeterValue;
			NewChannelInfo.PeakValue = InMeterResults.PeakValue;

			if (ChannelIndex < ChannelInfo.Num())
			{
				ChannelInfo[ChannelIndex] = MoveTemp(NewChannelInfo);
			}

			// Update the widget if this is the last channel
			if (ChannelIndex == ChannelInfo.Num() - 1)
			{
				if (Widget.IsValid())
				{
					Widget->SetMeterChannelInfo(ChannelInfo);
				}
			}
		}
	}

	void FAudioMeter::Teardown()
	{
		if (UObjectInitialized() && Analyzer.IsValid() && Analyzer->IsValidLowLevel())
		{
			Analyzer->StopAnalyzing();
			if (ResultsDelegateHandle.IsValid())
			{
				Analyzer->OnLatestPerChannelMeterResultsNative.Remove(ResultsDelegateHandle);
			}
		}

		Analyzer.Reset();
		ResultsDelegateHandle.Reset();
		AudioBus.Reset();
		ChannelInfo.Reset();
		Settings.Reset();

		bUseExternalAudioBus = false;
	}

	TSharedRef<SDockTab> FAudioMeter::SpawnTab(const FSpawnTabArgs& Args) const
	{
		return SNew(SDockTab)
			.Clipping(EWidgetClipping::ClipToBounds)
			.Label(RackUnitTypeInfo.DisplayName)
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Fill)
					[
						GetWidget()
					]
			];
	}

	TSharedRef<IAudioAnalyzerRackUnit> FAudioMeter::MakeRackUnit(const FAudioAnalyzerRackUnitConstructParams& Params)
	{
		// If we have an AudioMaterialMeterStyle create as an AudioMaterialMeter, otherwise create as a standard Slate meter:
		const FName MaterialMeterStyleName("AudioMaterialMeter.Style");
		if (Params.StyleSet->HasWidgetStyle<FAudioMaterialMeterStyle>(MaterialMeterStyleName))
		{
			const FAudioMaterialMeterStyle& AudioMaterialMeterStyle = Params.StyleSet->GetWidgetStyle<FAudioMaterialMeterStyle>(MaterialMeterStyleName);
			return MakeShared<FAudioMeter>(Params.AudioBusInfo.GetNumChannels(), Params.AudioBusInfo.AudioDeviceId, AudioMaterialMeterStyle, Params.AudioBusInfo.AudioBus);
		}
		else
		{
			const FAudioMeterDefaultColorStyle& MeterDefaultColorStyle = Params.StyleSet->GetWidgetStyle<FAudioMeterDefaultColorStyle>("AudioMeter.DefaultColorStyle");
			return MakeShared<FAudioMeter>(Params.AudioBusInfo.GetNumChannels(), Params.AudioBusInfo.AudioDeviceId, Params.AudioBusInfo.AudioBus, &MeterDefaultColorStyle);
		}
	}
} // namespace AudioWidgets
#undef LOCTEXT_NAMESPACE
