// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMeterStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioMeterStyle)

const FAudioMeterStyle& FAudioMeterStyle::GetDefault()
{
	static FAudioMeterStyle Default;
	return Default;
}

void FAudioMeterStyle::GetResources(TArray<const FSlateBrush*>& OutBrushes) const
{
	FAudioMeterWidgetStyle::GetResources(OutBrushes);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FAudioMeterStyle& FAudioMeterStyle::SetFont(const FName& InFontName, uint16 InSize) { Font = FSlateFontInfo(InFontName, InSize); return *this; }
FAudioMeterStyle& FAudioMeterStyle::SetFont(const FString& InFontName, uint16 InSize) { Font = FSlateFontInfo(*InFontName, InSize); return *this; }
FAudioMeterStyle& FAudioMeterStyle::SetFont(const WIDECHAR* InFontName, uint16 InSize) { Font = FSlateFontInfo(InFontName, InSize); return *this; }
FAudioMeterStyle& FAudioMeterStyle::SetFont(const ANSICHAR* InFontName, uint16 InSize) { Font = FSlateFontInfo(InFontName, InSize); return *this; }
FAudioMeterStyle& FAudioMeterStyle::SetFontName(const FName& InFontName) { Font = FSlateFontInfo(InFontName, Font.Size); return *this; }
FAudioMeterStyle& FAudioMeterStyle::SetFontName(const FString& InFontName) { Font = FSlateFontInfo(InFontName, Font.Size); return *this; }
FAudioMeterStyle& FAudioMeterStyle::SetFontName(const WIDECHAR* InFontName) { Font = FSlateFontInfo(InFontName, Font.Size); return *this; }
FAudioMeterStyle& FAudioMeterStyle::SetFontName(const ANSICHAR* InFontName) { Font = FSlateFontInfo(InFontName, Font.Size); return *this; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS
