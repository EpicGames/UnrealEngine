// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioOscilloscope.h"

#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "FAudioOscilloscope"

namespace AudioWidgets
{
	const FAudioAnalyzerRackUnitTypeInfo FAudioOscilloscope::RackUnitTypeInfo
	{
		.TypeName = TEXT("FAudioOscilloscope"),
		.DisplayName = LOCTEXT("AudioOscilloscopeDisplayName", "Oscilloscope"),
		.OnMakeAudioAnalyzerRackUnit = FOnMakeAudioAnalyzerRackUnit::CreateStatic(&MakeRackUnit),
		.VerticalSizeCoefficient = 0.25f,
	};

	FAudioOscilloscope::FAudioOscilloscope(Audio::FDeviceId InAudioDeviceId,
		const uint32 InNumChannels, 
		const float InTimeWindowMs, 
		const float InMaxTimeWindowMs, 
		const float InAnalysisPeriodMs, 
		const EAudioPanelLayoutType InPanelLayoutType,
		const FAudioOscilloscopePanelStyle* InOscilloscopePanelStyle,
		TObjectPtr<UAudioBus> InExternalAudioBus)
	{
		OscilloscopePanelStyle = InOscilloscopePanelStyle ? *InOscilloscopePanelStyle : FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioOscilloscopePanelStyle>("AudioOscilloscope.PanelStyle");
		
		if (InExternalAudioBus != nullptr)
		{
			ensure(InExternalAudioBus->GetNumChannels() == InNumChannels);
			AudioBus = TStrongObjectPtr(InExternalAudioBus.Get());
		}
		else if (InNumChannels > 0)
		{
			CreateAudioBus(InNumChannels);
		}

		CreateDataProvider(InAudioDeviceId, InTimeWindowMs, InMaxTimeWindowMs, InAnalysisPeriodMs, InPanelLayoutType);
		CreateOscilloscopeWidget(InNumChannels, InPanelLayoutType);
	}

	void FAudioOscilloscope::CreateAudioBus(const uint32 InNumChannels)
	{
		AudioBus = TStrongObjectPtr(NewObject<UAudioBus>());
		AudioBus->AudioBusChannels = AudioBusUtils::ConvertIntToEAudioBusChannels(InNumChannels);
	}

	void FAudioOscilloscope::CreateDataProvider(Audio::FDeviceId InAudioDeviceId,
		const float InTimeWindowMs,
		const float InMaxTimeWindowMs,
		const float InAnalysisPeriodMs,
		const EAudioPanelLayoutType InPanelLayoutType)
	{
		if (InAudioDeviceId == FAudioBusInfo::InvalidAudioDeviceId)
		{
			// Can't have a data provider without a valid audio device.
			AudioSamplesDataProvider.Reset();
			return;
		}

		check(AudioBus);

		const uint32 NumChannelsToProvide = (InPanelLayoutType == EAudioPanelLayoutType::Advanced) ? 1 : AudioBus->GetNumChannels(); // Advanced mode waveform display is based on channel selection
		AudioSamplesDataProvider = MakeShared<FWaveformAudioSamplesDataProvider>(InAudioDeviceId, AudioBus.Get(), NumChannelsToProvide, InTimeWindowMs, InMaxTimeWindowMs, InAnalysisPeriodMs);
	}

	void FAudioOscilloscope::CreateOscilloscopeWidget(const uint32 InNumChannels, const EAudioPanelLayoutType InPanelLayoutType, const FAudioOscilloscopePanelStyle* InOscilloscopePanelStyle)
	{
		FFixedSampledSequenceView SequenceView{ .NumDimensions = FMath::Max(InNumChannels, 1U), .SampleRate = 48000}; // Initialize with usable fallback values (but no sample data in the view).

		if (AudioSamplesDataProvider.IsValid())
		{
			// Get the actual sequence view from the data provider:
			SequenceView = AudioSamplesDataProvider->GetDataView();
		}

		if (InOscilloscopePanelStyle)
		{
			OscilloscopePanelStyle = *InOscilloscopePanelStyle;
		}

		if (!OscilloscopePanelWidget.IsValid())
		{
			OscilloscopePanelWidget = SNew(SAudioOscilloscopePanelWidget, SequenceView, InNumChannels)
			.PanelLayoutType(InPanelLayoutType)
			.PanelStyle(&OscilloscopePanelStyle);
		}
		else
		{
			OscilloscopePanelWidget->BuildWidget(SequenceView, InNumChannels, InPanelLayoutType);
		}

		if (AudioSamplesDataProvider.IsValid())
		{
			// Interconnect data provider and widget
			AudioSamplesDataProvider->OnDataViewGenerated.AddSP(OscilloscopePanelWidget.Get(), &SAudioOscilloscopePanelWidget::ReceiveSequenceView);

			if (InPanelLayoutType == EAudioPanelLayoutType::Advanced)
			{
				OscilloscopePanelWidget->OnSelectedChannelChanged.AddSP(AudioSamplesDataProvider.Get(), &FWaveformAudioSamplesDataProvider::SetChannelToAnalyze);
				OscilloscopePanelWidget->OnTriggerModeChanged.AddSP(AudioSamplesDataProvider.Get(), &FWaveformAudioSamplesDataProvider::SetTriggerMode);
				OscilloscopePanelWidget->OnTriggerThresholdChanged.AddSP(AudioSamplesDataProvider.Get(), &FWaveformAudioSamplesDataProvider::SetTriggerThreshold);
				OscilloscopePanelWidget->OnTimeWindowValueChanged.AddSP(AudioSamplesDataProvider.Get(), &FWaveformAudioSamplesDataProvider::SetTimeWindow);
				OscilloscopePanelWidget->OnAnalysisPeriodChanged.AddSP(AudioSamplesDataProvider.Get(), &FWaveformAudioSamplesDataProvider::SetAnalysisPeriod);
			}
		}
	}

	void FAudioOscilloscope::StartProcessing()
	{
		if (AudioSamplesDataProvider.IsValid())
		{
			AudioSamplesDataProvider->StartProcessing();
		}
	}

	void FAudioOscilloscope::StopProcessing()
	{
		if (AudioSamplesDataProvider.IsValid())
		{
			AudioSamplesDataProvider->StopProcessing();
		}
	}

	UAudioBus* FAudioOscilloscope::GetAudioBus() const
	{
		return AudioBus.Get();
	}

	TSharedRef<SWidget> FAudioOscilloscope::GetPanelWidget() const
	{
		return OscilloscopePanelWidget.ToSharedRef();
	}

	void FAudioOscilloscope::SetAudioBusInfo(const FAudioBusInfo& AudioBusInfo)
	{
		AudioBus = TStrongObjectPtr(AudioBusInfo.AudioBus.Get());
		CreateDataProvider(AudioBusInfo.AudioDeviceId, RackUnitTimeWindowMs, RackUnitMaxTimeWindowMs, RackUnitAnalysisPeriodMs, RackUnitPanelLayoutType);
		CreateOscilloscopeWidget(AudioBusInfo.AudioBus->GetNumChannels(), RackUnitPanelLayoutType);
	}

	TSharedRef<SDockTab> FAudioOscilloscope::SpawnTab(const FSpawnTabArgs& Args) const
	{
		return SNew(SDockTab)
			.Clipping(EWidgetClipping::ClipToBounds)
			.Label(RackUnitTypeInfo.DisplayName)
			[
				GetPanelWidget()
			];
	}

	TSharedRef<IAudioAnalyzerRackUnit> FAudioOscilloscope::MakeRackUnit(const FAudioAnalyzerRackUnitConstructParams& Params)
	{
		return MakeShared<FAudioOscilloscope>(
			Params.AudioBusInfo.AudioDeviceId,
			Params.AudioBusInfo.GetNumChannels(),
			RackUnitTimeWindowMs,
			RackUnitMaxTimeWindowMs,
			RackUnitAnalysisPeriodMs,
			RackUnitPanelLayoutType,
			&Params.StyleSet->GetWidgetStyle<FAudioOscilloscopePanelStyle>("AudioOscilloscope.PanelStyle"),
			Params.AudioBusInfo.AudioBus);
	}
}

#undef LOCTEXT_NAMESPACE
