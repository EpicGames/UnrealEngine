// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioOscilloscopeUMG.h"

#include "AudioMixerDevice.h"
#include "Engine/Engine.h"
#include "Sound/AudioBus.h"
#include "WaveformAudioSamplesDataProvider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioOscilloscopeUMG)

#define LOCTEXT_NAMESPACE "AudioOscilloscopeUMG"

UAudioOscilloscope::UAudioOscilloscope(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, OscilloscopeStyle(FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioOscilloscopePanelStyle>("AudioOscilloscope.PanelStyle"))
{
#if WITH_EDITORONLY_DATA
	AccessibleBehavior       = ESlateAccessibleBehavior::NotAccessible;
	bCanChildrenBeAccessible = false;
#endif

#if !UE_SERVER
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		DummyAudioSamples.Init(0.0f, DummyMaxNumSamples);
		DummyDataView = FFixedSampledSequenceView{ MakeArrayView(DummyAudioSamples.GetData(), DummyAudioSamples.Num()), DummyNumChannels, DummySampleRate };
	}
#endif
}

void UAudioOscilloscope::CreateDummyOscilloscopeWidget()
{
	OscilloscopePanelWidget = SNew(SAudioOscilloscopePanelWidget, DummyDataView, DummyNumChannels)
	.PanelLayoutType(PanelLayoutType)
	.PanelStyle(&OscilloscopeStyle);
}

void UAudioOscilloscope::CreateDataProvider()
{
	UWorld* World = GetWorld();
	if (!World)
	{
		return;
	}

	const FAudioDeviceHandle AudioDevice = World->GetAudioDevice();
	if (!AudioDevice.IsValid())
	{
		return;
	}

	const uint32 NumChannelsToProvide = (PanelLayoutType == EAudioPanelLayoutType::Advanced) ? 1 : AudioBus->GetNumChannels(); // Advanced mode waveform display is based on channel selection

	AudioSamplesDataProvider = MakeShared<AudioWidgets::FWaveformAudioSamplesDataProvider>(AudioDevice.GetDeviceID(), AudioBus, NumChannelsToProvide, TimeWindowMs, MaxTimeWindowMs, AnalysisPeriodMs);
}

void UAudioOscilloscope::CreateOscilloscopeWidget()
{
	check(AudioSamplesDataProvider);

	using namespace AudioWidgets;

	const FFixedSampledSequenceView SequenceView = AudioSamplesDataProvider->GetDataView();

	OscilloscopePanelWidget = SNew(SAudioOscilloscopePanelWidget, SequenceView, AudioSamplesDataProvider->GetNumChannels())
	.PanelLayoutType(PanelLayoutType)
	.PanelStyle(&OscilloscopeStyle);

	// Interconnect data provider and widget
	AudioSamplesDataProvider->OnDataViewGenerated.AddSP(OscilloscopePanelWidget.Get(), &SAudioOscilloscopePanelWidget::ReceiveSequenceView);

	if (PanelLayoutType == EAudioPanelLayoutType::Advanced)
	{
		OscilloscopePanelWidget->OnSelectedChannelChanged.AddSP(AudioSamplesDataProvider.Get(),  &FWaveformAudioSamplesDataProvider::SetChannelToAnalyze);
		OscilloscopePanelWidget->OnTriggerModeChanged.AddSP(AudioSamplesDataProvider.Get(),      &FWaveformAudioSamplesDataProvider::SetTriggerMode);
		OscilloscopePanelWidget->OnTriggerThresholdChanged.AddSP(AudioSamplesDataProvider.Get(), &FWaveformAudioSamplesDataProvider::SetTriggerThreshold);
		OscilloscopePanelWidget->OnTimeWindowValueChanged.AddSP(AudioSamplesDataProvider.Get(),  &FWaveformAudioSamplesDataProvider::SetTimeWindow);
		OscilloscopePanelWidget->OnAnalysisPeriodChanged.AddSP(AudioSamplesDataProvider.Get(),   &FWaveformAudioSamplesDataProvider::SetAnalysisPeriod);
	}
}

TSharedRef<SWidget> UAudioOscilloscope::RebuildWidget()
{
	TimeWindowMs = FMath::Clamp(TimeWindowMs, 10.0f, MaxTimeWindowMs);

	if (!AudioBus)
	{
		NumChannels = DummyNumChannels;

		CreateDummyOscilloscopeWidget();
	}
	else
	{
		NumChannels = AudioBus->GetNumChannels();

		CreateDataProvider();
		CreateOscilloscopeWidget();
	}

	return OscilloscopePanelWidget.ToSharedRef();
}

void UAudioOscilloscope::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	TimeWindowMs = FMath::Clamp(TimeWindowMs, 10.0f, MaxTimeWindowMs);

	if (!AudioBus)
	{
		NumChannels = DummyNumChannels;

		if (AudioSamplesDataProvider.IsValid())
		{
			AudioSamplesDataProvider.Reset();
			CreateDummyOscilloscopeWidget();
		}
	}
	else
	{
		if (!AudioSamplesDataProvider.IsValid() || AudioBus != AudioSamplesDataProvider->GetAudioBus() || AudioBus->GetNumChannels() != NumChannels)
		{
			NumChannels = AudioBus->GetNumChannels();

			CreateDataProvider();
			CreateOscilloscopeWidget();
		}
	}

	if (PanelLayoutType == EAudioPanelLayoutType::Advanced)
	{
		ChannelToAnalyze = FMath::Clamp(ChannelToAnalyze, 1, NumChannels);
	}

	if (AudioSamplesDataProvider.IsValid())
	{
		if (MaxTimeWindowMs != AudioSamplesDataProvider->GetMaxTimeWindowMs())
		{
			AudioSamplesDataProvider->SetMaxTimeWindowMs(MaxTimeWindowMs);
		}

		if (PanelLayoutType == EAudioPanelLayoutType::Advanced)
		{
			AudioSamplesDataProvider->SetChannelToAnalyze(ChannelToAnalyze);
		}

		if (CanTriggeringBeSet())
		{
			AudioSamplesDataProvider->SetTriggerMode(TriggerMode);
			AudioSamplesDataProvider->SetTriggerThreshold(TriggerThreshold);
		}
		else
		{
			AudioSamplesDataProvider->SetTriggerMode(EAudioOscilloscopeTriggerMode::None);
		}

		AudioSamplesDataProvider->SetTimeWindow(TimeWindowMs);
		AudioSamplesDataProvider->SetAnalysisPeriod(AnalysisPeriodMs);

		AudioSamplesDataProvider->RequestSequenceView(TRange<double>::Inclusive(0, 1));
	}

	if (OscilloscopePanelWidget.IsValid())
	{
		if (PanelLayoutType != OscilloscopePanelWidget->GetPanelLayoutType() && AudioSamplesDataProvider.IsValid())
		{
			CreateOscilloscopeWidget();
		}

		if (!AudioBus)
		{
			DummyDataView = FFixedSampledSequenceView{ MakeArrayView(DummyAudioSamples.GetData(), static_cast<int32>((TimeWindowMs / 1000.0f) * DummySampleRate)), DummyNumChannels, DummySampleRate};
			OscilloscopePanelWidget->ReceiveSequenceView(DummyDataView, 0);
		}

		OscilloscopePanelWidget->UpdateSequenceRulerStyle(OscilloscopeStyle.TimeRulerStyle);
		OscilloscopePanelWidget->UpdateValueGridOverlayStyle(OscilloscopeStyle.ValueGridStyle);
		OscilloscopePanelWidget->UpdateSequenceViewerStyle(OscilloscopeStyle.WaveViewerStyle);

		OscilloscopePanelWidget->SetXAxisGridVisibility(bShowTimeGrid);
		OscilloscopePanelWidget->SetSequenceRulerDisplayUnit(TimeGridLabelsUnit);

		OscilloscopePanelWidget->SetYAxisGridVisibility(bShowAmplitudeGrid);
		OscilloscopePanelWidget->SetYAxisLabelsVisibility(bShowAmplitudeLabels);
		OscilloscopePanelWidget->SetValueGridOverlayDisplayUnit(AmplitudeGridLabelsUnit);

		OscilloscopePanelWidget->SetMaxTimeWindowMs(MaxTimeWindowMs);

		if (PanelLayoutType == EAudioPanelLayoutType::Advanced)
		{
			OscilloscopePanelWidget->SetChannelToAnalyze(ChannelToAnalyze);
		}

		if (CanTriggeringBeSet())
		{
			OscilloscopePanelWidget->SetTriggerMode(TriggerMode);
			OscilloscopePanelWidget->SetTriggerThreshold(TriggerThreshold);
		}
		else
		{
			OscilloscopePanelWidget->SetTriggerMode(EAudioOscilloscopeTriggerMode::None);
		}

		OscilloscopePanelWidget->SetTimeWindow(TimeWindowMs);
		OscilloscopePanelWidget->SetAnalysisPeriod(AnalysisPeriodMs);
	}
}

void UAudioOscilloscope::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);
	
	OscilloscopePanelWidget.Reset();
}

#if WITH_EDITOR
const FText UAudioOscilloscope::GetPaletteCategory()
{
	return LOCTEXT("Audio", "Audio");
}
#endif

bool UAudioOscilloscope::CanTriggeringBeSet()
{
	return PanelLayoutType == EAudioPanelLayoutType::Advanced || (PanelLayoutType == EAudioPanelLayoutType::Basic && NumChannels == 1);
}

void UAudioOscilloscope::StartProcessing()
{
	if (AudioSamplesDataProvider.IsValid())
	{
		AudioSamplesDataProvider->StartProcessing();
	}
}

void UAudioOscilloscope::StopProcessing()
{
	if (AudioSamplesDataProvider.IsValid())
	{
		AudioSamplesDataProvider->StopProcessing();
	}
}

#undef LOCTEXT_NAMESPACE
