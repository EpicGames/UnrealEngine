// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioRadialSlider.h"
#include "AudioDefines.h"
#include "AudioWidgets.h"
#include "Modules/ModuleManager.h"
#include "SAudioRadialSlider.h"
#include "Styling/StyleColors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioRadialSlider)

#define LOCTEXT_NAMESPACE "AUDIO_UMG"

static FAudioRadialSliderStyle* DefaultAudioRadialSliderStyle = nullptr;
#if WITH_EDITOR 
static FAudioRadialSliderStyle* EditorAudioRadialSliderStyle = nullptr;
#endif 

// UAudioRadialSlider
UAudioRadialSlider::UAudioRadialSlider(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (DefaultAudioRadialSliderStyle == nullptr)
	{
		DefaultAudioRadialSliderStyle = new FAudioRadialSliderStyle(FAudioRadialSliderStyle::GetDefault());
	}

	WidgetStyle = *DefaultAudioRadialSliderStyle;

#if WITH_EDITOR 
	if (EditorAudioRadialSliderStyle == nullptr)
	{
		FModuleManager::LoadModuleChecked<FAudioWidgetsModule>("AudioWidgets");
		EditorAudioRadialSliderStyle = new FAudioRadialSliderStyle(FSlateStyleRegistry::FindSlateStyle("AudioWidgetsStyle")->GetWidgetStyle<FAudioRadialSliderStyle>("AudioRadialSlider.Style"));
	}

	if (IsEditorWidget())
	{
		WidgetStyle = *EditorAudioRadialSliderStyle;
	}
#endif // WITH_EDITOR

	Value = 0.0f;
	WidgetLayout = EAudioRadialSliderLayout::Layout_LabelBottom;
	CenterBackgroundColor = FStyleColors::Recessed.GetSpecifiedColor();
	SliderProgressColor = FStyleColors::White.GetSpecifiedColor();
	SliderBarColor = FStyleColors::AccentGray.GetSpecifiedColor();
	TextLabelBackgroundColor = FStyleColors::Recessed.GetSpecifiedColor();
	UnitsText = FText::FromString("units");
	HandStartEndRatio = FVector2D(0.0f, 1.0f);
	ShowLabelOnlyOnHover = false;
	ShowUnitsText = true;
	IsUnitsTextReadOnly = true;
	IsValueTextReadOnly = false;
	SliderThickness = 5.0f;
	OutputRange = FVector2D(0.0f, 1.0f);

#if WITH_EDITORONLY_DATA
	AccessibleBehavior = ESlateAccessibleBehavior::NotAccessible;
	bCanChildrenBeAccessible = false;
#endif
}

float UAudioRadialSlider::GetOutputValue(const float InSliderValue)
{
	return MyAudioRadialSlider->GetOutputValue(InSliderValue);
}

float UAudioRadialSlider::GetSliderValue(const float OutputValue)
{
	return MyAudioRadialSlider->GetSliderValue(OutputValue);
}

void UAudioRadialSlider::SetWidgetLayout(EAudioRadialSliderLayout InLayout)
{
	WidgetLayout = InLayout;
	if (MyAudioRadialSlider.IsValid())
	{
		MyAudioRadialSlider->SetWidgetLayout(InLayout);
	}
}

void UAudioRadialSlider::SetCenterBackgroundColor(FLinearColor InValue)
{
	CenterBackgroundColor = InValue;
	if (MyAudioRadialSlider.IsValid())
	{
		MyAudioRadialSlider->SetCenterBackgroundColor(InValue);
	}
}

void UAudioRadialSlider::SetSliderProgressColor(FLinearColor InValue)
{
	SliderProgressColor = InValue;
	if (MyAudioRadialSlider.IsValid())
	{
		MyAudioRadialSlider->SetSliderProgressColor(InValue);
	}
}

void UAudioRadialSlider::SetSliderBarColor(FLinearColor InValue)
{
	SliderBarColor = InValue;
	if (MyAudioRadialSlider.IsValid())
	{
		MyAudioRadialSlider->SetSliderBarColor(InValue);
	}
}

void UAudioRadialSlider::SetHandStartEndRatio(const FVector2D InHandStartEndRatio)
{
	HandStartEndRatio = InHandStartEndRatio;
	if (MyAudioRadialSlider.IsValid())
	{
		MyAudioRadialSlider->SetHandStartEndRatio(InHandStartEndRatio);
	}
}

void UAudioRadialSlider::SetTextLabelBackgroundColor(FSlateColor InColor)
{
	TextLabelBackgroundColor = InColor.GetSpecifiedColor();
	if (MyAudioRadialSlider.IsValid())
	{
		MyAudioRadialSlider->SetLabelBackgroundColor(InColor);
	}
}

void UAudioRadialSlider::SetUnitsText(const FText Units)
{
	UnitsText = Units;
	if (MyAudioRadialSlider.IsValid())
	{
		MyAudioRadialSlider->SetUnitsText(Units);
	}
}

void UAudioRadialSlider::SetUnitsTextReadOnly(const bool bIsReadOnly)
{
	IsUnitsTextReadOnly = bIsReadOnly;
	if (MyAudioRadialSlider.IsValid())
	{
		MyAudioRadialSlider->SetUnitsTextReadOnly(bIsReadOnly);
	}
}

void UAudioRadialSlider::SetValueTextReadOnly(const bool bIsReadOnly)
{
	IsValueTextReadOnly = bIsReadOnly;
	if (MyAudioRadialSlider.IsValid())
	{
		MyAudioRadialSlider->SetValueTextReadOnly(bIsReadOnly);
	}
}

void UAudioRadialSlider::SetShowLabelOnlyOnHover(const bool bShowLabelOnlyOnHover)
{
	ShowLabelOnlyOnHover = bShowLabelOnlyOnHover;
	if (MyAudioRadialSlider.IsValid())
	{
		MyAudioRadialSlider->SetShowLabelOnlyOnHover(bShowLabelOnlyOnHover);
	}
}

void UAudioRadialSlider::SetShowUnitsText(const bool bShowUnitsText)
{
	ShowUnitsText = bShowUnitsText;
	if (MyAudioRadialSlider.IsValid())
	{
		MyAudioRadialSlider->SetShowUnitsText(bShowUnitsText);
	}
}

void UAudioRadialSlider::SetSliderThickness(const float InThickness)
{
	SliderThickness = InThickness;
	if (MyAudioRadialSlider.IsValid())
	{
		MyAudioRadialSlider->SetSliderThickness(InThickness);
	}
}

void UAudioRadialSlider::SetOutputRange(const FVector2D InOutputRange)
{
	OutputRange = InOutputRange;
	if (MyAudioRadialSlider.IsValid())
	{
		MyAudioRadialSlider->SetOutputRange(InOutputRange);
	}
}

void UAudioRadialSlider::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	MyAudioRadialSlider->SetCenterBackgroundColor(CenterBackgroundColor);
	MyAudioRadialSlider->SetWidgetLayout(WidgetLayout);
	MyAudioRadialSlider->SetLabelBackgroundColor(TextLabelBackgroundColor);
	MyAudioRadialSlider->SetSliderProgressColor(SliderProgressColor);
	MyAudioRadialSlider->SetSliderBarColor(SliderBarColor);
	MyAudioRadialSlider->SetHandStartEndRatio(HandStartEndRatio);
	MyAudioRadialSlider->SetUnitsText(UnitsText);
	MyAudioRadialSlider->SetUnitsTextReadOnly(IsUnitsTextReadOnly);
	MyAudioRadialSlider->SetValueTextReadOnly(IsValueTextReadOnly);
	MyAudioRadialSlider->SetShowLabelOnlyOnHover(ShowLabelOnlyOnHover);
	MyAudioRadialSlider->SetShowUnitsText(ShowUnitsText);
	MyAudioRadialSlider->SetSliderThickness(SliderThickness);
	MyAudioRadialSlider->SetOutputRange(OutputRange);
}

void UAudioRadialSlider::HandleOnValueChanged(float InValue)
{
	OnValueChanged.Broadcast(InValue);
}

void UAudioRadialSlider::HandleOnMouseCaptureBegin()
{
	OnMouseCaptureBegin.Broadcast();
}

void UAudioRadialSlider::HandleOnMouseCaptureEnd()
{
	OnMouseCaptureEnd.Broadcast();
}

#if WITH_EDITOR
const FText UAudioRadialSlider::GetPaletteCategory()
{
	return LOCTEXT("Audio", "Audio");
}
#endif

void UAudioRadialSlider::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyAudioRadialSlider.Reset();
}

TSharedRef<SWidget> UAudioRadialSlider::RebuildWidget()
{
	MyAudioRadialSlider = SNew(SAudioRadialSlider)
		.Style(&WidgetStyle)
		.OnValueChanged(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnValueChanged))
		.OnMouseCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureBegin))
		.OnMouseCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureEnd));

	return MyAudioRadialSlider.ToSharedRef();
}

// UAudioVolumeRadialSlider
UAudioVolumeRadialSlider::UAudioVolumeRadialSlider(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Super::SetUnitsText(FText::FromString("dB"));
	OutputRange = FVector2D(-12.0f, 0.0f);
}

TSharedRef<SWidget> UAudioVolumeRadialSlider::RebuildWidget()
{
	MyAudioRadialSlider = SNew(SAudioVolumeRadialSlider)
		.Style(&WidgetStyle)
		.OnValueChanged(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnValueChanged))
		.OnMouseCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureBegin))
		.OnMouseCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureEnd));

	return MyAudioRadialSlider.ToSharedRef();
}

// UAudioFrequencyRadialSlider
UAudioFrequencyRadialSlider::UAudioFrequencyRadialSlider(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Super::SetUnitsText(FText::FromString("Hz"));
	OutputRange = FVector2D(MIN_FILTER_FREQUENCY, MAX_FILTER_FREQUENCY);
}

TSharedRef<SWidget> UAudioFrequencyRadialSlider::RebuildWidget()
{
	MyAudioRadialSlider = SNew(SAudioFrequencyRadialSlider)
		.Style(&WidgetStyle)
		.OnValueChanged(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnValueChanged))
		.OnMouseCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureBegin))
		.OnMouseCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureEnd));

	return MyAudioRadialSlider.ToSharedRef();
}

#undef LOCTEXT_NAMESPACE
