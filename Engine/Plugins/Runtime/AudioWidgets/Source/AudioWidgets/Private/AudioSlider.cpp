// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSlider.h"
#include "AudioWidgets.h"
#include "Modules/ModuleManager.h"
#include "SAudioSlider.h"
#include "Styling/StyleColors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioSlider)

#define LOCTEXT_NAMESPACE "AUDIO_UMG"

static FAudioSliderStyle* DefaultAudioSliderStyle = nullptr;
static FAudioSliderStyle* EditorAudioSliderStyle = nullptr;

// UAudioSliderBase
UAudioSliderBase::UAudioSliderBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (DefaultAudioSliderStyle == nullptr)
	{
		DefaultAudioSliderStyle = new FAudioSliderStyle(FAudioSliderStyle::GetDefault());

		// Unlink UMG default colors.
		DefaultAudioSliderStyle->UnlinkColors();
	}

	WidgetStyle = *DefaultAudioSliderStyle;

#if WITH_EDITOR 
	if (EditorAudioSliderStyle == nullptr)
	{
		FModuleManager::LoadModuleChecked<FAudioWidgetsModule>("AudioWidgets");
		EditorAudioSliderStyle = new FAudioSliderStyle(FSlateStyleRegistry::FindSlateStyle("AudioWidgetsStyle")->GetWidgetStyle<FAudioSliderStyle>("AudioSlider.Style"));

		// Unlink UMG Editor colors from the editor settings colors.
		EditorAudioSliderStyle->UnlinkColors();
	}

	if (IsEditorWidget())
	{
		WidgetStyle = *EditorAudioSliderStyle;
	}
#endif // WITH_EDITOR


	Value = 0.0f;
	UnitsText = FText::FromString("units");
	ShowLabelOnlyOnHover = false;
	ShowUnitsText = true;
	IsUnitsTextReadOnly = true;
	IsValueTextReadOnly = false;
	Orientation = Orient_Vertical;
	TextLabelBackgroundColor = FStyleColors::Recessed.GetSpecifiedColor();
	SliderBackgroundColor = FStyleColors::Recessed.GetSpecifiedColor();
	SliderBarColor = FStyleColors::Black.GetSpecifiedColor();
	SliderThumbColor = FStyleColors::AccentGray.GetSpecifiedColor();
	WidgetBackgroundColor = FStyleColors::Transparent.GetSpecifiedColor();

#if WITH_EDITORONLY_DATA
	AccessibleBehavior = ESlateAccessibleBehavior::NotAccessible;
	bCanChildrenBeAccessible = false;
#endif
}

void UAudioSliderBase::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	if (MyAudioSlider.IsValid())
	{
		MyAudioSlider->SetLabelBackgroundColor(TextLabelBackgroundColor);
		MyAudioSlider->SetUnitsText(UnitsText);
		MyAudioSlider->SetUnitsTextReadOnly(IsUnitsTextReadOnly);
		MyAudioSlider->SetValueTextReadOnly(IsValueTextReadOnly);
		MyAudioSlider->SetShowLabelOnlyOnHover(ShowLabelOnlyOnHover);
		MyAudioSlider->SetShowUnitsText(ShowUnitsText);
		MyAudioSlider->SetOrientation(Orientation);
		MyAudioSlider->SetSliderBackgroundColor(SliderBackgroundColor);
		MyAudioSlider->SetSliderBarColor(SliderBarColor);
		MyAudioSlider->SetSliderThumbColor(SliderThumbColor);
		MyAudioSlider->SetWidgetBackgroundColor(WidgetBackgroundColor);

		TAttribute<float> ValueBinding = PROPERTY_BINDING(float, Value);
		if (ValueBinding.IsBound())
		{
			MyAudioSlider->SetSliderValue(ValueBinding);
		}
		else
		{
			MyAudioSlider->SetSliderValue(Value);
		}
	}
}

void UAudioSliderBase::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);

	MyAudioSlider.Reset();
}

void UAudioSliderBase::HandleOnValueChanged(float InValue)
{
	Value = InValue;
	OnValueChanged.Broadcast(InValue);
}

void UAudioSliderBase::HandleOnMouseCaptureBegin()
{
	OnMouseCaptureBegin.Broadcast();
}

void UAudioSliderBase::HandleOnMouseCaptureEnd()
{
	OnMouseCaptureEnd.Broadcast();
}

void UAudioSliderBase::SetTextLabelBackgroundColor(FSlateColor InColor)
{
	TextLabelBackgroundColor = InColor.GetSpecifiedColor();
	if (MyAudioSlider.IsValid())
	{
		MyAudioSlider->SetLabelBackgroundColor(InColor);
	}
}

void UAudioSliderBase::SetUnitsText(const FText Units)
{
	UnitsText = Units;
	if (MyAudioSlider.IsValid())
	{
		MyAudioSlider->SetUnitsText(Units);
	}
}

void UAudioSliderBase::SetUnitsTextReadOnly(const bool bIsReadOnly)
{
	IsUnitsTextReadOnly = bIsReadOnly;
	if (MyAudioSlider.IsValid())
	{
		MyAudioSlider->SetUnitsTextReadOnly(bIsReadOnly);
	}
}

void UAudioSliderBase::SetValueTextReadOnly(const bool bIsReadOnly)
{
	IsValueTextReadOnly = bIsReadOnly;
	if (MyAudioSlider.IsValid())
	{
		MyAudioSlider->SetValueTextReadOnly(bIsReadOnly);
	}
}

void UAudioSliderBase::SetShowLabelOnlyOnHover(const bool bShowLabelOnlyOnHover)
{
	ShowLabelOnlyOnHover = bShowLabelOnlyOnHover;
	if (MyAudioSlider.IsValid())
	{
		MyAudioSlider->SetShowLabelOnlyOnHover(bShowLabelOnlyOnHover);
	}
}

void UAudioSliderBase::SetShowUnitsText(const bool bShowUnitsText)
{
	ShowUnitsText = bShowUnitsText;
	if (MyAudioSlider.IsValid())
	{
		MyAudioSlider->SetShowUnitsText(bShowUnitsText);
	}
}

float UAudioSliderBase::GetOutputValue(const float InSliderValue)
{
	return MyAudioSlider->GetOutputValue(InSliderValue);
}

float UAudioSliderBase::GetLinValue(const float OutputValue)
{
	return GetSliderValue(OutputValue);
}

float UAudioSliderBase::GetSliderValue(const float OutputValue)
{
	return MyAudioSlider->GetSliderValue(OutputValue);
}

void UAudioSliderBase::SetSliderBackgroundColor(FLinearColor InValue)
{
	SliderBackgroundColor = InValue;
	if (MyAudioSlider.IsValid())
	{
		MyAudioSlider->SetSliderBackgroundColor(InValue);
	}
}

void UAudioSliderBase::SetSliderBarColor(FLinearColor InValue)
{
	SliderBarColor = InValue;
	if (MyAudioSlider.IsValid())
	{
		MyAudioSlider->SetSliderBarColor(InValue);
	}
}

void UAudioSliderBase::SetSliderThumbColor(FLinearColor InValue)
{
	SliderThumbColor = InValue;
	if (MyAudioSlider.IsValid())
	{
		MyAudioSlider->SetSliderThumbColor(InValue);
	}
}

void UAudioSliderBase::SetWidgetBackgroundColor(FLinearColor InValue)
{
	WidgetBackgroundColor = InValue;
	if (MyAudioSlider.IsValid())
	{
		MyAudioSlider->SetWidgetBackgroundColor(InValue);
	}
}

TSharedRef<SWidget> UAudioSliderBase::RebuildWidget()
{
	MyAudioSlider = SNew(SAudioSliderBase)
		.Style(&WidgetStyle)
		.OnValueChanged(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnValueChanged))
		.OnMouseCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureBegin))
		.OnMouseCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureEnd));

	return MyAudioSlider.ToSharedRef();
}

#if WITH_EDITOR

const FText UAudioSliderBase::GetPaletteCategory()
{
	return LOCTEXT("Audio", "Audio");
}

#endif

// UAudioSlider
UAudioSlider::UAudioSlider(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAudioSlider::SynchronizeProperties()
{
	UAudioSliderBase::SynchronizeProperties();

	StaticCastSharedPtr<SAudioSlider>(MyAudioSlider)->SetLinToOutputCurve(LinToOutputCurve);
	StaticCastSharedPtr<SAudioSlider>(MyAudioSlider)->SetOutputToLinCurve(OutputToLinCurve);
}

TSharedRef<SWidget> UAudioSlider::RebuildWidget()
{
	MyAudioSlider = SNew(SAudioSlider)
		.Style(&WidgetStyle)
		.OnValueChanged(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnValueChanged))
		.OnMouseCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureBegin))
		.OnMouseCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureEnd));

	return MyAudioSlider.ToSharedRef();
}

// UAudioVolumeSlider
UAudioVolumeSlider::UAudioVolumeSlider(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Super::SetUnitsText(FText::FromString("dB"));
}

TSharedRef<SWidget> UAudioVolumeSlider::RebuildWidget()
{
	MyAudioSlider = SNew(SAudioVolumeSlider)
		.OnValueChanged(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnValueChanged))
		.OnMouseCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureBegin))
		.OnMouseCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureEnd));
	LinToOutputCurve = StaticCastSharedPtr<SAudioSlider>(MyAudioSlider)->GetLinToOutputCurve();
	OutputToLinCurve = StaticCastSharedPtr<SAudioSlider>(MyAudioSlider)->GetOutputToLinCurve();

	return MyAudioSlider.ToSharedRef();
}

// UAudioFrequencySlider
UAudioFrequencySlider::UAudioFrequencySlider(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, OutputRange(FVector2D(MIN_FILTER_FREQUENCY, MAX_FILTER_FREQUENCY))
{
	Super::SetUnitsText(FText::FromString("Hz"));
}

TSharedRef<SWidget> UAudioFrequencySlider::RebuildWidget()
{
	MyAudioSlider = SNew(SAudioFrequencySlider)
		.Style(&WidgetStyle)
		.OnValueChanged(BIND_UOBJECT_DELEGATE(FOnFloatValueChanged, HandleOnValueChanged))
		.OnMouseCaptureBegin(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureBegin))
		.OnMouseCaptureEnd(BIND_UOBJECT_DELEGATE(FSimpleDelegate, HandleOnMouseCaptureEnd));
	StaticCastSharedPtr<SAudioFrequencySlider>(MyAudioSlider)->SetOutputRange(OutputRange);

	return MyAudioSlider.ToSharedRef();
}

#undef LOCTEXT_NAMESPACE
