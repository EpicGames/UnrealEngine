// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSpectrogram.h"

#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Docking/SDockTab.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioSpectrogram)

#define LOCTEXT_NAMESPACE "FAudioSpectrogram"

namespace AudioWidgets
{
	namespace AudioSpectrogramPrivate
	{
		/**
		 * Light wrapper for accessing settings for the analyzer rack unit. Can be passed by value.
		 */
		class FRackUnitSettingsHelper
		{
		public:
			FRackUnitSettingsHelper(const FProperty& InSettingsProperty)
				: SettingsProperty(InSettingsProperty)
			{
				//
			}

			FSpectrogramRackUnitSettings* GetRackUnitSettings() const
			{
				UObject* EditorSettingsObject = GetEditorSettingsObject();
				return SettingsProperty.ContainerPtrToValuePtr<FSpectrogramRackUnitSettings>(EditorSettingsObject);
			}

			void SaveConfig() const
			{
				GetEditorSettingsObject()->SaveConfig();
			}

		private:
			UObject* GetEditorSettingsObject() const
			{
				return SettingsProperty.GetOwnerClass()->GetDefaultObject();
			}

			const FProperty& SettingsProperty;
		};
	}

	const FAudioAnalyzerRackUnitTypeInfo FAudioSpectrogram::RackUnitTypeInfo
	{
		.TypeName = TEXT("FAudioSpectrogram"),
		.DisplayName = LOCTEXT("AudioSpectrogramDisplayName", "Spectrogram"),
		.OnMakeAudioAnalyzerRackUnit = FOnMakeAudioAnalyzerRackUnit::CreateStatic(&MakeRackUnit),
		.VerticalSizeCoefficient = 0.25f,
	};

	FAudioSpectrogram::FAudioSpectrogram(const FAudioSpectrogramParams& Params)
		: SpectrumAnalysisSettings(NewObject<USynesthesiaSpectrumAnalysisSettings>())
		, ConstantQSettings(NewObject<UConstantQSettings>())
		, Widget(SNew(SAudioSpectrogram)
			.Clipping(EWidgetClipping::ClipToBounds)
			.FrequencyAxisScale(Params.FrequencyAxisScale)
			.FrequencyAxisPixelBucketMode(Params.FrequencyAxisPixelBucketMode)
			.ColorMap(Params.ColorMap)
			.Orientation(Params.Orientation)
			.FillBackground(true)
			.OnFrequencyAxisPixelBucketModeMenuEntryClicked(Params.OnFrequencyAxisPixelBucketModeMenuEntryClicked)
			.OnFrequencyAxisScaleMenuEntryClicked(Params.OnFrequencyAxisScaleMenuEntryClicked)
			.OnColorMapMenuEntryClicked(Params.OnColorMapMenuEntryClicked)
			.OnOrientationMenuEntryClicked(Params.OnOrientationMenuEntryClicked)
		)
		, AnalyzerType(Params.AnalyzerType)
		, FFTAnalyzerFFTSize(Params.FFTAnalyzerFFTSize)
		, CQTAnalyzerFFTSize(Params.CQTAnalyzerFFTSize)
		, OnAnalyzerTypeMenuEntryClicked(Params.OnAnalyzerTypeMenuEntryClicked)
		, OnFFTAnalyzerFFTSizeMenuEntryClicked(Params.OnFFTAnalyzerFFTSizeMenuEntryClicked)
		, OnCQTAnalyzerFFTSizeMenuEntryClicked(Params.OnCQTAnalyzerFFTSizeMenuEntryClicked)
	{
		SpectrumAnalysisSettings->SpectrumType = EAudioSpectrumType::PowerSpectrum;
		SpectrumAnalysisSettings->FFTSize = FFTAnalyzerFFTSize.Get();
		SpectrumAnalysisSettings->WindowType = EFFTWindowType::Blackman;
		SpectrumAnalysisSettings->bDownmixToMono = true;

		ConstantQSettings->SpectrumType = EAudioSpectrumType::PowerSpectrum;
		ConstantQSettings->NumBandsPerOctave = 6.0f;
		ConstantQSettings->NumBands = 61;
		ConstantQSettings->StartingFrequencyHz = 20000.0f * FMath::Pow(0.5f, (ConstantQSettings->NumBands - 1) / ConstantQSettings->NumBandsPerOctave);
		ConstantQSettings->FFTSize = CQTAnalyzerFFTSize.Get();
		ConstantQSettings->WindowType = EFFTWindowType::Blackman;
		ConstantQSettings->bDownmixToMono = true;
		ConstantQSettings->BandWidthStretch = 2.0f;

		ContextMenuExtension = Widget->AddContextMenuExtension(EExtensionHook::Before, nullptr, FMenuExtensionDelegate::CreateRaw(this, &FAudioSpectrogram::ExtendSpectrumPlotContextMenu));

		ActiveTimer = Widget->RegisterActiveTimer(0.0f, FWidgetActiveTimerDelegate::CreateRaw(this, &FAudioSpectrogram::Update));

		Init(Params.NumChannels, Params.AudioDeviceId, Params.ExternalAudioBus);
	}

	FAudioSpectrogram::~FAudioSpectrogram()
	{
		Teardown();

		if (ContextMenuExtension.IsValid())
		{
			Widget->RemoveContextMenuExtension(ContextMenuExtension.ToSharedRef());
		}

		if (ActiveTimer.IsValid())
		{
			Widget->UnRegisterActiveTimer(ActiveTimer.ToSharedRef());
		}
	}

	UAudioBus* FAudioSpectrogram::GetAudioBus() const
	{
		return AudioBus.Get();
	}

	TSharedRef<SWidget> FAudioSpectrogram::GetWidget() const
	{
		return Widget->AsShared();
	}

	void FAudioSpectrogram::Init(int32 InNumChannels, Audio::FDeviceId InAudioDeviceId, TObjectPtr<UAudioBus> InExternalAudioBus)
	{
		Teardown();

		AudioDeviceId = InAudioDeviceId;

		// Only create analyzers etc if we have an audio device:
		if (AudioDeviceId != FAudioBusInfo::InvalidAudioDeviceId)
		{
			check(InNumChannels > 0);

			bUseExternalAudioBus = InExternalAudioBus != nullptr;
			AudioBus = bUseExternalAudioBus ? TStrongObjectPtr(InExternalAudioBus.Get()) : TStrongObjectPtr(NewObject<UAudioBus>());
			AudioBus->AudioBusChannels = EAudioBusChannels(InNumChannels - 1);

			CreateSynesthesiaSpectrumAnalyzer();
			CreateConstantQAnalyzer();

			StartAnalyzing(AnalyzerType.Get());
		}
	}

	void FAudioSpectrogram::StartAnalyzing(const EAudioSpectrumAnalyzerType InAnalyzerType)
	{
		ensure(!ActiveAnalyzerType.IsSet());

		switch (InAnalyzerType)
		{
		case EAudioSpectrumAnalyzerType::FFT:
			SpectrumAnalyzer->StartAnalyzing(AudioDeviceId, AudioBus.Get());
			break;
		case EAudioSpectrumAnalyzerType::CQT:
			ConstantQAnalyzer->StartAnalyzing(AudioDeviceId, AudioBus.Get());
			break;
		default:
			break;
		}

		ActiveAnalyzerType = InAnalyzerType;
	}

	void FAudioSpectrogram::StopAnalyzing()
	{
		ensure(ActiveAnalyzerType.IsSet());

		switch (ActiveAnalyzerType.GetValue())
		{
		case EAudioSpectrumAnalyzerType::FFT:
			SpectrumAnalyzer->StopAnalyzing();
			break;
		case EAudioSpectrumAnalyzerType::CQT:
			ConstantQAnalyzer->StopAnalyzing();
			break;
		default:
			break;
		}

		ActiveAnalyzerType.Reset();
	}

	void FAudioSpectrogram::OnSpectrumResults(USynesthesiaSpectrumAnalyzer* InSpectrumAnalyzer, int32 ChannelIndex, const TArray<FSynesthesiaSpectrumResults>& InSpectrumResultsArray)
	{
		if (ActiveAnalyzerType == EAudioSpectrumAnalyzerType::FFT && InSpectrumAnalyzer == SpectrumAnalyzer.Get())
		{
			for (const FSynesthesiaSpectrumResults& SpectrumResults : InSpectrumResultsArray)
			{
				// Find samplerate:
				float SampleRate = 48000.0f;
				if (const FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
				{
					if (const FAudioDevice* AudioDevice = AudioDeviceManager->GetAudioDeviceRaw(AudioDeviceId))
					{
						SampleRate = AudioDevice->GetSampleRate();
					}
				}

				Widget->AddFrame(SpectrumResults, SpectrumAnalysisSettings->SpectrumType, SampleRate);
			}
		}
	}

	void FAudioSpectrogram::OnConstantQResults(UConstantQAnalyzer* InSpectrumAnalyzer, int32 ChannelIndex, const TArray<FConstantQResults>& InSpectrumResultsArray)
	{
		if (ActiveAnalyzerType == EAudioSpectrumAnalyzerType::CQT && InSpectrumAnalyzer == ConstantQAnalyzer.Get())
		{
			for (const FConstantQResults& SpectrumResults : InSpectrumResultsArray)
			{
				Widget->AddFrame(SpectrumResults, ConstantQSettings->StartingFrequencyHz, ConstantQSettings->NumBandsPerOctave, ConstantQSettings->SpectrumType);
			}
		}
	}

	void FAudioSpectrogram::Teardown()
	{
		if (UObjectInitialized())
		{
			if (SpectrumAnalyzer.IsValid() && SpectrumAnalyzer->IsValidLowLevel())
			{
				if (ActiveAnalyzerType == EAudioSpectrumAnalyzerType::FFT)
				{
					SpectrumAnalyzer->StopAnalyzing();
				}

				ReleaseSynesthesiaSpectrumAnalyzer();
			}
			
			if (ConstantQAnalyzer.IsValid() && ConstantQAnalyzer->IsValidLowLevel())
			{
				if (ActiveAnalyzerType == EAudioSpectrumAnalyzerType::CQT)
				{
					ConstantQAnalyzer->StopAnalyzing();
				}

				ReleaseConstantQAnalyzer();
			}
		}

		SpectrumResultsDelegateHandle.Reset();
		SpectrumAnalyzer.Reset();

		ConstantQResultsDelegateHandle.Reset();
		ConstantQAnalyzer.Reset();

		ActiveAnalyzerType.Reset();

		AudioBus.Reset();
		bUseExternalAudioBus = false;
	}

	void FAudioSpectrogram::ExtendSpectrumPlotContextMenu(FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.BeginSection("AnalyzerSettings", LOCTEXT("AnalyzerSettings", "Analyzer Settings"));
		MenuBuilder.AddSubMenu(
			LOCTEXT("AnalyzerType", "Analyzer Type"),
			FText(),
			FNewMenuDelegate::CreateSP(this, &FAudioSpectrogram::BuildAnalyzerTypeSubMenu));
		MenuBuilder.AddSubMenu(
			LOCTEXT("FFTSize", "FFT Size"),
			FText(),
			FNewMenuDelegate::CreateSP(this, &FAudioSpectrogram::BuildFFTSizeSubMenu));
		MenuBuilder.EndSection();
	}

	void FAudioSpectrogram::BuildAnalyzerTypeSubMenu(FMenuBuilder& SubMenu)
	{
		const UEnum* EnumClass = StaticEnum<EAudioSpectrumAnalyzerType>();
		const int32 NumEnumValues = EnumClass->NumEnums() - 1; // Exclude 'MAX' enum value.
		for (int32 Index = 0; Index < NumEnumValues; Index++)
		{
			const auto EnumValue = static_cast<EAudioSpectrumAnalyzerType>(EnumClass->GetValueByIndex(Index));

			SubMenu.AddMenuEntry(
				EnumClass->GetDisplayNameTextByIndex(Index),
#if WITH_EDITOR
				EnumClass->GetToolTipTextByIndex(Index),
#else
				FText(),
#endif
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSPLambda(this, [this, EnumValue]()
						{
							if (!AnalyzerType.IsBound())
							{
								AnalyzerType = EnumValue;
							}

							OnAnalyzerTypeMenuEntryClicked.ExecuteIfBound(EnumValue);
						}),
					FCanExecuteAction(),
					FIsActionChecked::CreateSPLambda(this, [this, EnumValue]() { return (AnalyzerType.Get() == EnumValue); })
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);
		}
	}

	void FAudioSpectrogram::BuildFFTSizeSubMenu(FMenuBuilder& SubMenu)
	{
		// There is a different FFTSize enum depending on the analyzer type.

		if (AnalyzerType.Get() == EAudioSpectrumAnalyzerType::FFT)
		{
			const UEnum* EnumClass = StaticEnum<EFFTSize>();
			const int32 NumEnumValues = EnumClass->NumEnums() - 1; // Exclude 'MAX' enum value.
			for (int32 Index = 0; Index < NumEnumValues; Index++)
			{
				const auto EnumValue = static_cast<EFFTSize>(EnumClass->GetValueByIndex(Index));
				if (EnumValue == EFFTSize::DefaultSize)
				{
					// Skip the duplicate 512 enum value 'DefaultSize'.
					continue;
				}

				SubMenu.AddMenuEntry(
					EnumClass->GetDisplayNameTextByIndex(Index),
#if WITH_EDITOR
					EnumClass->GetToolTipTextByIndex(Index),
#else
					FText(),
#endif
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSPLambda(this, [this, EnumValue]()
							{
								if (!FFTAnalyzerFFTSize.IsBound())
								{
									FFTAnalyzerFFTSize = EnumValue;
								}

								OnFFTAnalyzerFFTSizeMenuEntryClicked.ExecuteIfBound(EnumValue);
							}),
						FCanExecuteAction(),
						FIsActionChecked::CreateSPLambda(this, [this, EnumValue]() { return (FFTAnalyzerFFTSize.Get() == EnumValue); })
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton);
			}
		}
		else if (AnalyzerType.Get() == EAudioSpectrumAnalyzerType::CQT)
		{
			const UEnum* EnumClass = StaticEnum<EConstantQFFTSizeEnum>();
			const int32 NumEnumValues = EnumClass->NumEnums() - 1; // Exclude 'MAX' enum value.
			for (int32 Index = 0; Index < NumEnumValues; Index++)
			{
				const auto EnumValue = static_cast<EConstantQFFTSizeEnum>(EnumClass->GetValueByIndex(Index));

				SubMenu.AddMenuEntry(
					EnumClass->GetDisplayNameTextByIndex(Index),
#if WITH_EDITOR
					EnumClass->GetToolTipTextByIndex(Index),
#else
					FText(),
#endif
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSPLambda(this, [this, EnumValue]()
							{
								if (!CQTAnalyzerFFTSize.IsBound())
								{
									CQTAnalyzerFFTSize = EnumValue;
								}

								OnCQTAnalyzerFFTSizeMenuEntryClicked.ExecuteIfBound(EnumValue);
							}),
						FCanExecuteAction(),
						FIsActionChecked::CreateSPLambda(this, [this, EnumValue]() { return (CQTAnalyzerFFTSize.Get() == EnumValue); })
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton);
			}
		}
	}

	EActiveTimerReturnType FAudioSpectrogram::Update(double InCurrentTime, float InDeltaTime)
	{
		if (AudioDeviceId == FAudioBusInfo::InvalidAudioDeviceId)
		{
			// No analyzers available if no valid audio device.
			ensure(!ActiveAnalyzerType.IsSet());
			return EActiveTimerReturnType::Continue;
		}

		const EAudioSpectrumAnalyzerType RequiredAnalyzerType = AnalyzerType.Get();
		const EFFTSize FFTAnalyzerRequiredFFTSize = FFTAnalyzerFFTSize.Get();
		const EConstantQFFTSizeEnum CQTAnalyzerRequiredFFTSize = CQTAnalyzerFFTSize.Get();

		const bool bRequiredAnalyzerTypeChanged = (ActiveAnalyzerType != RequiredAnalyzerType);
		const bool bFFTAnalyzerRequiredFFTSizeChanged = (SpectrumAnalysisSettings->FFTSize != FFTAnalyzerRequiredFFTSize);
		const bool bCQTAnalyzerRequiredFFTSizeChanged = (ConstantQSettings->FFTSize != CQTAnalyzerRequiredFFTSize);
		if (bRequiredAnalyzerTypeChanged || bFFTAnalyzerRequiredFFTSizeChanged || bCQTAnalyzerRequiredFFTSizeChanged)
		{
			StopAnalyzing();

			if (bFFTAnalyzerRequiredFFTSizeChanged)
			{
				ReleaseSynesthesiaSpectrumAnalyzer();
				SpectrumAnalysisSettings->FFTSize = FFTAnalyzerRequiredFFTSize;
				CreateSynesthesiaSpectrumAnalyzer();
			}

			if (bCQTAnalyzerRequiredFFTSizeChanged)
			{
				ReleaseConstantQAnalyzer();
				ConstantQSettings->FFTSize = CQTAnalyzerRequiredFFTSize;
				CreateConstantQAnalyzer();
			}

			StartAnalyzing(RequiredAnalyzerType);
		}

		return EActiveTimerReturnType::Continue;
	}

	void FAudioSpectrogram::CreateSynesthesiaSpectrumAnalyzer()
	{
		ensure(!SpectrumAnalyzer.IsValid());
		ensure(!SpectrumResultsDelegateHandle.IsValid());

		SpectrumAnalyzer = TStrongObjectPtr(NewObject<USynesthesiaSpectrumAnalyzer>());
		SpectrumAnalyzer->Settings = SpectrumAnalysisSettings.Get();
		SpectrumResultsDelegateHandle = SpectrumAnalyzer->OnSpectrumResultsNative.AddRaw(this, &FAudioSpectrogram::OnSpectrumResults);
	}

	void FAudioSpectrogram::ReleaseSynesthesiaSpectrumAnalyzer()
	{
		if (ensure(SpectrumAnalyzer.IsValid() && SpectrumResultsDelegateHandle.IsValid()))
		{
			SpectrumAnalyzer->OnSpectrumResultsNative.Remove(SpectrumResultsDelegateHandle);
		}

		SpectrumResultsDelegateHandle.Reset();
		SpectrumAnalyzer.Reset();
	}

	void FAudioSpectrogram::CreateConstantQAnalyzer()
	{
		ConstantQAnalyzer = TStrongObjectPtr(NewObject<UConstantQAnalyzer>());
		ConstantQAnalyzer->Settings = ConstantQSettings.Get();
		ConstantQResultsDelegateHandle = ConstantQAnalyzer->OnConstantQResultsNative.AddRaw(this, &FAudioSpectrogram::OnConstantQResults);
	}

	void FAudioSpectrogram::ReleaseConstantQAnalyzer()
	{
		if (ensure(ConstantQAnalyzer.IsValid() && ConstantQResultsDelegateHandle.IsValid()))
		{
			ConstantQAnalyzer->OnConstantQResultsNative.Remove(ConstantQResultsDelegateHandle);
		}

		ConstantQResultsDelegateHandle.Reset();
		ConstantQAnalyzer.Reset();
	}

	TSharedRef<SDockTab> FAudioSpectrogram::SpawnTab(const FSpawnTabArgs& Args) const
	{
		return SNew(SDockTab)
			.Clipping(EWidgetClipping::ClipToBounds)
			.Label(RackUnitTypeInfo.DisplayName)
			[
				GetWidget()
			];
	}

	TSharedRef<IAudioAnalyzerRackUnit> FAudioSpectrogram::MakeRackUnit(const FAudioAnalyzerRackUnitConstructParams& Params)
	{
		using namespace AudioSpectrogramPrivate;

		FAudioSpectrogramParams AnalyzerParams;
		AnalyzerParams.NumChannels = Params.AudioBusInfo.GetNumChannels();
		AnalyzerParams.AudioDeviceId = Params.AudioBusInfo.AudioDeviceId;
		AnalyzerParams.ExternalAudioBus = Params.AudioBusInfo.AudioBus;

		if (Params.EditorSettingsClass != nullptr)
		{
			// If we have been given a valid editor settings class, bind analyzer options to the settings:
			const FProperty* SpectrogramSettingsProperty = Params.EditorSettingsClass->FindPropertyByName("SpectrogramSettings");
			if (SpectrogramSettingsProperty != nullptr)
			{
				const FRackUnitSettingsHelper SettingsHelper(*SpectrogramSettingsProperty);

				AnalyzerParams.AnalyzerType.BindLambda([=]()
					{
						return SettingsHelper.GetRackUnitSettings()->AnalyzerType;
					});
				AnalyzerParams.FFTAnalyzerFFTSize.BindLambda([=]()
					{
						return SettingsHelper.GetRackUnitSettings()->FFTAnalyzerFFTSize;
					});
				AnalyzerParams.CQTAnalyzerFFTSize.BindLambda([=]()
					{
						return SettingsHelper.GetRackUnitSettings()->CQTAnalyzerFFTSize;
					});
				AnalyzerParams.FrequencyAxisPixelBucketMode.BindLambda([=]()
					{
						return SettingsHelper.GetRackUnitSettings()->PixelPlotMode;
					});
				AnalyzerParams.FrequencyAxisScale.BindLambda([=]()
					{
						return SettingsHelper.GetRackUnitSettings()->FrequencyScale;
					});
				AnalyzerParams.ColorMap.BindLambda([=]()
					{
						return SettingsHelper.GetRackUnitSettings()->ColorMap;
					});
				AnalyzerParams.Orientation.BindLambda([=]()
					{
						return SettingsHelper.GetRackUnitSettings()->Orientation;
					});

				AnalyzerParams.OnAnalyzerTypeMenuEntryClicked.BindLambda([=](EAudioSpectrumAnalyzerType SelectedValue)
					{
						SettingsHelper.GetRackUnitSettings()->AnalyzerType = SelectedValue;
						SettingsHelper.SaveConfig();
					});
				AnalyzerParams.OnFFTAnalyzerFFTSizeMenuEntryClicked.BindLambda([=](EFFTSize SelectedValue)
					{
						SettingsHelper.GetRackUnitSettings()->FFTAnalyzerFFTSize = SelectedValue;
						SettingsHelper.SaveConfig();
					});
				AnalyzerParams.OnCQTAnalyzerFFTSizeMenuEntryClicked.BindLambda([=](EConstantQFFTSizeEnum SelectedValue)
					{
						SettingsHelper.GetRackUnitSettings()->CQTAnalyzerFFTSize = SelectedValue;
						SettingsHelper.SaveConfig();
					});
				AnalyzerParams.OnFrequencyAxisPixelBucketModeMenuEntryClicked.BindLambda([=](EAudioSpectrogramFrequencyAxisPixelBucketMode SelectedValue)
					{
						SettingsHelper.GetRackUnitSettings()->PixelPlotMode = SelectedValue;
						SettingsHelper.SaveConfig();
					});
				AnalyzerParams.OnFrequencyAxisScaleMenuEntryClicked.BindLambda([=](EAudioSpectrogramFrequencyAxisScale SelectedValue)
					{
						SettingsHelper.GetRackUnitSettings()->FrequencyScale = SelectedValue;
						SettingsHelper.SaveConfig();
					});
				AnalyzerParams.OnColorMapMenuEntryClicked.BindLambda([=](EAudioColorGradient SelectedValue)
					{
						SettingsHelper.GetRackUnitSettings()->ColorMap = SelectedValue;
						SettingsHelper.SaveConfig();
					});
				AnalyzerParams.OnOrientationMenuEntryClicked.BindLambda([=](EOrientation SelectedValue)
					{
						SettingsHelper.GetRackUnitSettings()->Orientation = SelectedValue;
						SettingsHelper.SaveConfig();
					});
			}
		}

		return MakeShared<FAudioSpectrogram>(AnalyzerParams);
	}
} // namespace AudioWidgets

#undef LOCTEXT_NAMESPACE
