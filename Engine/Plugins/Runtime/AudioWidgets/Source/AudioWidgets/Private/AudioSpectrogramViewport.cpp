// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSpectrogramViewport.h"

#include "Framework/Application/SlateApplication.h"
#include "Rendering/SlateRenderer.h"
#include "Slate/SlateTextures.h"

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioSpectrogramViewport)

/**
 * Helper class for getting the normalized read position in the source spectrum data for a given target pixel to be rendered.
 */
class FAudioSpectrogramRenderTransformHelper
{
public:
	FAudioSpectrogramRenderTransformHelper(const FAudioSpectrogramViewportRenderParams& RenderParams, const bool bSpectrumDataHasLogSpacedFreqencies, const float SpectrumDataMinFrequency, const float SpectrumDataMaxFrequency)
		: PixelToFreqTransform(RenderParams.FrequencyAxisScale, RenderParams.NumPixelsPerRow, RenderParams.ViewMinFrequency, RenderParams.ViewMaxFrequency)
		, FreqToDataPosTransform(bSpectrumDataHasLogSpacedFreqencies, SpectrumDataMinFrequency, SpectrumDataMaxFrequency)
	{
		//
	}

	// Given a pixel index, get the normalized position to sample the data at.
	float PixelToNormalizedDataPos(const float Pixel) const
	{
		const float Frequency = PixelToFreqTransform.PixelToFrequency(Pixel);
		return FreqToDataPosTransform.FrequencyToNormalizedDataPos(Frequency);
	}

	// Given a pixel index, get the ideal mip size that would best resolve the data for the position the data would be sampled at.
	float GetDesiredMipSize(const int Pixel) const
	{
		const float NormalizedDataPosLo = PixelToNormalizedDataPos(float(Pixel) - 0.5f);
		const float NormalizedDataPosHi = PixelToNormalizedDataPos(float(Pixel) + 0.5f);
		const float NormalizedDataPosDelta = FMath::Abs(NormalizedDataPosHi - NormalizedDataPosLo);
		constexpr float MinDelta = 1.0f / float(1 << 24);
		return 1.0f / FMath::Max(NormalizedDataPosDelta, MinDelta);
	}

private:
	/**
	 * Internal helper class for getting the frequency for a given target pixel to be rendered.
	 */
	class FPixelToFreqTransform
	{
	public:
		FPixelToFreqTransform(const EAudioSpectrogramFrequencyAxisScale InFrequencyAxisScale, const int NumPixelsPerRow, const float InViewMinFrequency, const float InViewMaxFrequency)
			: FrequencyAxisScale(InFrequencyAxisScale)
			, TransformedViewMinFrequency(ForwardTransformFrequency(InViewMinFrequency))
			, ScalingValue((ForwardTransformFrequency(InViewMaxFrequency) - TransformedViewMinFrequency) / float(NumPixelsPerRow - 1))
		{
			check(NumPixelsPerRow >= 2);
		}

		float PixelToFrequency(const float Index) const
		{
			const float TransformedFrequency = TransformedViewMinFrequency + (Index * ScalingValue);
			return InverseTransformFrequency(TransformedFrequency);
		}

	private:
		float ForwardTransformFrequency(const float Frequency) const
		{
			return (FrequencyAxisScale == EAudioSpectrogramFrequencyAxisScale::Logarithmic) ? FMath::Loge(Frequency) : Frequency;
		}

		float InverseTransformFrequency(const float TransformedFrequency) const
		{
			return (FrequencyAxisScale == EAudioSpectrogramFrequencyAxisScale::Logarithmic) ? FMath::Exp(TransformedFrequency) : TransformedFrequency;
		}

		const EAudioSpectrogramFrequencyAxisScale FrequencyAxisScale;
		const float TransformedViewMinFrequency;
		const float ScalingValue;
	};

	/**
	 * Internal helper class for getting the normalized read position in the source spectrum data for a given frequency.
	 */
	class FFreqToDataPosTransform
	{
	public:
		FFreqToDataPosTransform(const bool bInSpectrumDataHasLogSpacedFreqencies, const float InSpectrumDataMinFrequency, const float InSpectrumDataMaxFrequency)
			: bSpectrumDataHasLogSpacedFreqencies(bInSpectrumDataHasLogSpacedFreqencies)
			, TransformedSpectrumDataMinFrequency(TransformFrequency(InSpectrumDataMinFrequency))
			, ScalingValue(1.0f / (TransformFrequency(InSpectrumDataMaxFrequency) - TransformedSpectrumDataMinFrequency))
		{
			check(InSpectrumDataMaxFrequency != InSpectrumDataMinFrequency);
		}

		float FrequencyToNormalizedDataPos(const float Frequency) const
		{
			return ScalingValue * (TransformFrequency(Frequency) - TransformedSpectrumDataMinFrequency);
		}

	private:
		float TransformFrequency(const float Frequency) const
		{
			return (bSpectrumDataHasLogSpacedFreqencies) ? FMath::Loge(Frequency) : Frequency;
		}

		const bool bSpectrumDataHasLogSpacedFreqencies;
		const float TransformedSpectrumDataMinFrequency;
		const float ScalingValue;
	};

	const FPixelToFreqTransform PixelToFreqTransform;
	const FFreqToDataPosTransform FreqToDataPosTransform;
};

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FAudioSpectrogramViewport::FAudioSpectrogramViewport()
{
	// Max history size should default to zero (we shall only start collecting spectrogram frames once render params have been set):
	ensure(RenderParams.NumRows == 0);
}

FAudioSpectrogramViewport::~FAudioSpectrogramViewport()
{
	// Release texture if we have one:
	if (UpdatableTexture != nullptr)
	{
		if (FSlateApplication::IsInitialized())
		{
			if (FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer())
			{
				Renderer->ReleaseUpdatableTexture(UpdatableTexture);
			}
		}

		UpdatableTexture = nullptr;
		UpdatableTexturePixelFormat = PF_Unknown;
	}
}

FIntPoint FAudioSpectrogramViewport::GetSize() const
{
	const FSlateShaderResource* SlateResource = GetViewportRenderTargetTexture();
	if (SlateResource != nullptr)
	{
		return FIntPoint(SlateResource->GetWidth(), SlateResource->GetHeight());
	}

	return FIntPoint::ZeroValue;
}

void FAudioSpectrogramViewport::SetRenderParams(const FAudioSpectrogramViewportRenderParams& InRenderParams)
{
	if (RenderParams.NumRows != InRenderParams.NumRows ||
		RenderParams.NumPixelsPerRow != InRenderParams.NumPixelsPerRow ||
		RenderParams.ViewMinFrequency != InRenderParams.ViewMinFrequency ||
		RenderParams.ViewMaxFrequency != InRenderParams.ViewMaxFrequency ||
		RenderParams.ColorMapMinSoundLevel != InRenderParams.ColorMapMinSoundLevel ||
		RenderParams.ColorMapMaxSoundLevel != InRenderParams.ColorMapMaxSoundLevel ||
		RenderParams.ColorMap != InRenderParams.ColorMap ||
		RenderParams.FrequencyAxisScale != InRenderParams.FrequencyAxisScale ||
		RenderParams.FrequencyAxisPixelBucketMode != InRenderParams.FrequencyAxisPixelBucketMode)
	{
		RenderParams = InRenderParams;

		if (History.Num() > RenderParams.NumRows)
		{
			// Reduce size of history by removing frames from the start:
			const int ExcessFrames = History.Num() - RenderParams.NumRows;
			History.PopFront(ExcessFrames);
		}

		// Render params changed, invalidate all cached pixel data:
		UpdateTextureData(true);
	}
}

void FAudioSpectrogramViewport::AddFrame(const FAudioSpectrogramFrameData& SpectrogramFrameData)
{
	if (RenderParams.NumRows == 0)
	{
		return;
	}

	if (History.Num() == RenderParams.NumRows)
	{
		// History is full. Make space for new frame by removing oldest from the start of the history: 
		History.PopFront();
	}

	// Add the new frame to the end of the history buffer:
	History.Emplace(SpectrogramFrameData, RenderParams.FrequencyAxisPixelBucketMode);

	// Spectrogram data changed, update texture data:
	UpdateTextureData();
}

void FAudioSpectrogramViewport::UpdateTextureData(const bool bInvalidateCachedPixelData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAudioSpectrogramViewport::UpdateTextureData);

	// Figure out required texture size:
	const int32 Width = RenderParams.NumPixelsPerRow;
	const int32 Height = RenderParams.NumRows;

	if (UpdatableTexture == nullptr)
	{
		// Try to create a texture of the required size:
		if (FSlateRenderer* Renderer = FSlateApplication::Get().GetRenderer())
		{
			UpdatableTexture = Renderer->CreateUpdatableTexture(Width, Height);
			UpdatableTexturePixelFormat = Renderer->GetSlateRecommendedColorFormat();
		}
	}

	if (UpdatableTexture != nullptr)
	{
		// We shall write rows of pixels to RawData:
		TArray<uint8> RawData;

		// Set RawData to the size requried for the texture:
		check(UpdatableTexturePixelFormat != PF_Unknown);
		const int32 BytesPerPixel = GPixelFormats[UpdatableTexturePixelFormat].BlockBytes;
		const int32 BytesPerRow = Width * BytesPerPixel;
		const int32 SizeInBytes = Height * BytesPerRow;
		RawData.AddZeroed(SizeInBytes);

		// Ensure the texture is of the expected format:
		if (ensure(BytesPerPixel == sizeof(FColor) && UpdatableTexturePixelFormat == PF_B8G8R8A8))
		{
			// Go over each frame in the History buffer:
			check(History.Num() <= Height);
			const int32 NumSkipRows = Height - History.Num(); // If History size is smaller than texture height, skip over rows in the texture to fill right to the bottom.
			for (int32 RowIndex = NumSkipRows; RowIndex < Height; RowIndex++)
			{
				FHistoryFrameData& HistoryFrameData = History[RowIndex - NumSkipRows];

				// Generate/regenerate pixel data for this frame if required:
				if (HistoryFrameData.GetCachedPixels().Num() != Width || bInvalidateCachedPixelData)
				{
					HistoryFrameData.UpdateCachedPixels(RenderParams);
				}

				// Copy generated pixel data to RawData:
				const TConstArrayView<FColor> CachedPixels = HistoryFrameData.GetCachedPixels();
				check(CachedPixels.Num() == Width);
				memcpy(&RawData[RowIndex * BytesPerRow], CachedPixels.GetData(), BytesPerRow);
			}
		}

		// Apply RawData to the texture:
		FSlateTextureData* BulkData = new FSlateTextureData(Width, Height, BytesPerPixel, MoveTemp(RawData));
		UpdatableTexture->UpdateTextureThreadSafeWithTextureData(BulkData); // Ownership of BulkData is transferred to the render system here.
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FAudioSpectrogramViewport::FHistoryFrameData::FHistoryFrameData(const FAudioSpectrogramFrameData& SpectrogramFrameData, const EAudioSpectrogramFrequencyAxisPixelBucketMode InFrequencyAxisPixelBucketMode)
	: NumSoundLevels(SpectrogramFrameData.SpectrumValues.Num())
	, SpectrumDataMinFrequency(SpectrogramFrameData.MinFrequency)
	, SpectrumDataMaxFrequency(SpectrogramFrameData.MaxFrequency)
	, bSpectrumDataHasLogSpacedFreqencies(SpectrogramFrameData.bLogSpacedFreqencies)
	, SoundLevelsMipChainPixelBucketMode(InFrequencyAxisPixelBucketMode)
{
	// Allocate required space for mip chain:
	SetMipChainLengthUninitialized(InFrequencyAxisPixelBucketMode);

	// Copy spectrum data to mip zero, converting to dB if necessary:
	switch (SpectrogramFrameData.SpectrumType)
	{
		case EAudioSpectrumType::MagnitudeSpectrum:
			ArrayMagnitudeToDecibel(SpectrogramFrameData.SpectrumValues, SoundLevelsMipChain, -200.0f);
			break;
		case EAudioSpectrumType::PowerSpectrum:
			ArrayPowerToDecibel(SpectrogramFrameData.SpectrumValues, SoundLevelsMipChain, -200.0f);
			break;
		default:
		case EAudioSpectrumType::Decibel:
			FMemory::Memcpy(SoundLevelsMipChain.GetData(), SpectrogramFrameData.SpectrumValues.GetData(), NumSoundLevels * sizeof(float));
			break;
	}

	// Generate remaining mips if required:
	UpdateMipChain(InFrequencyAxisPixelBucketMode);
}

FAudioSpectrogramViewport::FHistoryFrameData::FHistoryFrameData(FHistoryFrameData&& Other)
	: NumSoundLevels(Other.NumSoundLevels)
	, SpectrumDataMinFrequency(Other.SpectrumDataMinFrequency)
	, SpectrumDataMaxFrequency(Other.SpectrumDataMaxFrequency)
	, bSpectrumDataHasLogSpacedFreqencies(Other.bSpectrumDataHasLogSpacedFreqencies)
	, SoundLevelsMipChain(MoveTemp(Other.SoundLevelsMipChain))
	, SoundLevelsMipChainPixelBucketMode(Other.SoundLevelsMipChainPixelBucketMode)
	, CachedPixels(MoveTemp(Other.CachedPixels))
{
	//
}

void FAudioSpectrogramViewport::FHistoryFrameData::UpdateCachedPixels(const FAudioSpectrogramViewportRenderParams& InRenderParams)
{
	// Create local helpers:
	const FAudioSpectrogramRenderTransformHelper RenderTransformHelper(InRenderParams, bSpectrumDataHasLogSpacedFreqencies, SpectrumDataMinFrequency, SpectrumDataMaxFrequency);
	const FAudioColorMapper ColorMapper(InRenderParams.ColorMapMinSoundLevel, InRenderParams.ColorMapMaxSoundLevel, InRenderParams.ColorMap);

	// Regenerate mip chain if FrequencyAxisPixelBucketMode has changed:
	if (SoundLevelsMipChainPixelBucketMode != InRenderParams.FrequencyAxisPixelBucketMode)
	{
		UpdateMipChain(InRenderParams.FrequencyAxisPixelBucketMode);
		SoundLevelsMipChainPixelBucketMode = InRenderParams.FrequencyAxisPixelBucketMode;
	}

	// Ensure pixel data array is of the required size before writing to it:
	CachedPixels.SetNumUninitialized(InRenderParams.NumPixelsPerRow);

	if (HasMips())
	{
		for (int Index = 0; Index < InRenderParams.NumPixelsPerRow; Index++)
		{
			// Calculate mip LOD:
			const float DesiredMipSize = RenderTransformHelper.GetDesiredMipSize(Index);

			// Sample from mips with interpolation:
			const float NormalizedDataPos = RenderTransformHelper.PixelToNormalizedDataPos(Index);
			const float InterpolatedSoundLevel = GetMipInterpolatedSoundLevel(DesiredMipSize, NormalizedDataPos);

			// Apply colormap and write pixel:
			CachedPixels[Index] = ColorMapper.GetColorFromValue(InterpolatedSoundLevel);
		}
	}
	else
	{
		// Render using mip zero only (which is all we have):
		const TConstArrayView<float> Mip0(SoundLevelsMipChain);
		for (int Index = 0; Index < InRenderParams.NumPixelsPerRow; Index++)
		{
			// Sample data with interpolation:
			const float NormalizedDataPos = RenderTransformHelper.PixelToNormalizedDataPos(Index);
			const float InterpolatedSoundLevel = GetInterpolatedSoundLevel(Mip0, NormalizedDataPos);

			// Apply colormap and write pixel:
			CachedPixels[Index] = ColorMapper.GetColorFromValue(InterpolatedSoundLevel);
		}
	}
}

void FAudioSpectrogramViewport::FHistoryFrameData::SetMipChainLengthUninitialized(const EAudioSpectrogramFrequencyAxisPixelBucketMode InFrequencyAxisPixelBucketMode)
{
	// Only supporting mip maps for power of two sizes:
	const bool bUseMipMaps = FMath::IsPowerOfTwo(NumSoundLevels) && (InFrequencyAxisPixelBucketMode != EAudioSpectrogramFrequencyAxisPixelBucketMode::Sample) && !bSpectrumDataHasLogSpacedFreqencies;

	const int SoundLevelsMipChainLength = (bUseMipMaps) ? 2 * NumSoundLevels - 1 : NumSoundLevels;
	SoundLevelsMipChain.SetNumUninitialized(SoundLevelsMipChainLength);
}

void FAudioSpectrogramViewport::FHistoryFrameData::UpdateMipChain(const EAudioSpectrogramFrequencyAxisPixelBucketMode InFrequencyAxisPixelBucketMode)
{
	// Mip zero should already exist:
	check(SoundLevelsMipChain.Num() >= NumSoundLevels);

	// Resize SoundLevelsMipChain if necessary for the current pixel bucket mode:
	SetMipChainLengthUninitialized(InFrequencyAxisPixelBucketMode);

	// Mip zero should still exist:
	check(SoundLevelsMipChain.Num() >= NumSoundLevels);

	if (HasMips())
	{
		// Only these two modes use mip maps:
		check(InFrequencyAxisPixelBucketMode == EAudioSpectrogramFrequencyAxisPixelBucketMode::Peak || InFrequencyAxisPixelBucketMode == EAudioSpectrogramFrequencyAxisPixelBucketMode::Average);

		// Start reading from start of mip zero:
		int ReadIndex = 0;

		// Start writing data directly after mip zero:
		int WriteIndex = NumSoundLevels;

		// Generate mips:
		for (int CurrentMipSize = NumSoundLevels / 2; CurrentMipSize >= 1; CurrentMipSize /= 2)
		{
			for (int Count = 0; Count < CurrentMipSize; Count++)
			{
				const float ReadA = SoundLevelsMipChain[ReadIndex++];
				const float ReadB = SoundLevelsMipChain[ReadIndex++];
				if (InFrequencyAxisPixelBucketMode == EAudioSpectrogramFrequencyAxisPixelBucketMode::Peak)
				{
					SoundLevelsMipChain[WriteIndex++] = FMath::Max(ReadA, ReadB);
				}
				else// if (InFrequencyAxisPixelBucketMode == EAudioSpectrogramFrequencyAxisPixelBucketMode::Average)
				{
					SoundLevelsMipChain[WriteIndex++] = 0.5f * (ReadA + ReadB);
				}
			}
		}

		// We have written the full mip chain:
		check(WriteIndex == SoundLevelsMipChain.Num());
	}
}

TConstArrayView<float> FAudioSpectrogramViewport::FHistoryFrameData::GetMipFromMipSize(const int MipSize) const
{
	check(FMath::IsPowerOfTwo(MipSize));
	check(FMath::IsPowerOfTwo(NumSoundLevels));
	check(MipSize <= NumSoundLevels);

	const int StartIndex = 2 * (NumSoundLevels - MipSize);
	return TConstArrayView<float>(SoundLevelsMipChain).Slice(StartIndex, MipSize);
}

float FAudioSpectrogramViewport::FHistoryFrameData::GetMipInterpolatedSoundLevel(const float DesiredMipSize, const float NormalizedDataPos) const
{
	// Get mips for mip LOD:
	const int MipHiSize = FMath::Clamp(FMath::RoundUpToPowerOfTwo(FMath::CeilToInt(DesiredMipSize)), 2, NumSoundLevels);
	const int MipLoSize = MipHiSize / 2;
	const TConstArrayView<float> MipHi = GetMipFromMipSize(MipHiSize);
	const TConstArrayView<float> MipLo = GetMipFromMipSize(MipLoSize);

	// Sample from mips with interpolation:
	const float MipHiSample = GetInterpolatedSoundLevel(MipHi, NormalizedDataPos);
	const float MipLoSample = GetInterpolatedSoundLevel(MipLo, NormalizedDataPos);

	// Interpolate between mips:
	const float MipLerpParam = FMath::Clamp(float(DesiredMipSize - MipLoSize) / float(MipHiSize - MipLoSize), 0.0f, 1.0f);
	return FMath::Lerp(MipLoSample, MipHiSample, MipLerpParam);
}

float FAudioSpectrogramViewport::FHistoryFrameData::GetInterpolatedSoundLevel(const TConstArrayView<float> MipData, const float NormalizedDataPos)
{
	check(NormalizedDataPos >= 0.0f);

	const int MipSize = MipData.Num();
	const float IndexFloat = NormalizedDataPos * (MipSize - 1);
	const float IndexFloor = FMath::Floor(IndexFloat);
	const float LerpParam = IndexFloat - IndexFloor;
	const int IndexLo = FMath::Min(static_cast<int>(IndexFloor) + 0, MipSize - 1);
	const int IndexHi = FMath::Min(static_cast<int>(IndexFloor) + 1, MipSize - 1);
	return FMath::Lerp(MipData[IndexLo], MipData[IndexHi], LerpParam);
}

void FAudioSpectrogramViewport::FHistoryFrameData::ArrayMagnitudeToDecibel(TConstArrayView<float> InValues, TArrayView<float> OutValues, float InMinimumDb)
{
	const float ClampMinMagnitude = Audio::ConvertToLinear(InMinimumDb);
	const float Scale = 20.0f / FMath::Loge(10.0f);
	for (int32 Index = 0; Index < InValues.Num(); Index++)
	{
		const float Magnitude = FMath::Max(InValues[Index], ClampMinMagnitude);
		OutValues[Index] = Scale * FMath::Loge(Magnitude);
	}
}

void FAudioSpectrogramViewport::FHistoryFrameData::ArrayPowerToDecibel(TConstArrayView<float> InValues, TArrayView<float> OutValues, float InMinimumDb)
{
	const float ClampMinMagnitudeSquared = FMath::Pow(10.0f, InMinimumDb / 10.0f);
	const float Scale = 10.0f / FMath::Loge(10.0f);
	for (int32 Index = 0; Index < InValues.Num(); Index++)
	{
		const float MagnitudeSquared = FMath::Max(InValues[Index], ClampMinMagnitudeSquared);
		OutValues[Index] = Scale * FMath::Loge(MagnitudeSquared);
	}
}
