// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSpectrumAnalyzer.h"

#include "AudioDevice.h"
#include "AudioDeviceManager.h"
#include "ConstantQFactory.h"
#include "DSP/EnvelopeFollower.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SynesthesiaSpectrumAnalysisFactory.h"
#include "Widgets/Docking/SDockTab.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioSpectrumAnalyzer)

#define LOCTEXT_NAMESPACE "FAudioSpectrumAnalyzer"

namespace AudioWidgets
{
	namespace AudioSpectrumAnalyzerPrivate
	{
		inline float GetWindowCompensationPowerGain(Audio::EWindowType WindowType, int32 FFTSize)
		{
			ensure(FFTSize <= 16384);
			ensure(FMath::IsPowerOfTwo(FFTSize));

			using namespace Audio;

			// Create a temporary buffer and initialize it to +1 DC:
			FAlignedFloatBuffer Samples;
			Samples.SetNumUninitialized(FFTSize);
			Audio::ArraySetToConstantInplace(Samples, 1.0f);

			// Initialize the window in the same manner as FSynesthesiaSpectrumAnalyzer and FConstantQAnalyzer:
			FWindow Window(WindowType, FFTSize, /*NumChannels=*/1, /*bIsPeriodic=*/false);

			// Apply window to DC signal:
			Window.ApplyToBuffer(Samples.GetData());

			// Calculate the mean square of the windowed signal:
			float WindowedDCMeanSquare = 1.0f;
			Audio::ArrayMeanSquared(Samples, WindowedDCMeanSquare);

			// Return the power gain required to reverse the effect of the windowing process on the RMS of DC:
			constexpr float DCMeanSquare = 1.0f;
			return DCMeanSquare / WindowedDCMeanSquare;
		}

		/**
		 * Light wrapper for accessing settings for the analyzer rack unit. Can be passed by value.
		 */
		class FRackUnitSettingsHelper
		{
		public:
			FRackUnitSettingsHelper(const FProperty& InSettingsProperty)
				: SettingsProperty(InSettingsProperty)
			{
				//
			}

			FSpectrumAnalyzerRackUnitSettings* GetRackUnitSettings() const
			{
				UObject* EditorSettingsObject = GetEditorSettingsObject();
				return SettingsProperty.ContainerPtrToValuePtr<FSpectrumAnalyzerRackUnitSettings>(EditorSettingsObject);
			}

			void SaveConfig() const
			{
				GetEditorSettingsObject()->SaveConfig();
			}

		private:
			UObject* GetEditorSettingsObject() const
			{
				return SettingsProperty.GetOwnerClass()->GetDefaultObject();
			}

			const FProperty& SettingsProperty;
		};
	}

	const FAudioAnalyzerRackUnitTypeInfo FAudioSpectrumAnalyzer::RackUnitTypeInfo
	{
		.TypeName = TEXT("FAudioSpectrumAnalyzer"),
		.DisplayName = LOCTEXT("AudioSpectrumAnalyzerDisplayName", "Spectrum Analyzer"),
		.OnMakeAudioAnalyzerRackUnit = FOnMakeAudioAnalyzerRackUnit::CreateStatic(&MakeRackUnit),
		.VerticalSizeCoefficient = 0.25f,
	};

	FAudioSpectrumAnalyzer::FAudioSpectrumAnalyzer(const FAudioSpectrumAnalyzerParams& Params)
		: SpectrumAnalysisSettings(NewObject<USynesthesiaSpectrumAnalysisSettings>())
		, ConstantQSettings(NewObject<UConstantQSettings>())
		, Widget(SNew(SAudioSpectrumPlot)
			.Style(Params.PlotStyle ? Params.PlotStyle : &FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioSpectrumPlotStyle>("AudioSpectrumPlot.Style"))
			.Clipping(EWidgetClipping::ClipToBounds)
			.TiltExponent(Params.TiltExponent)
			.DisplayCrosshair(true)
			.DisplayFrequencyAxisLabels(Params.bDisplayFrequencyAxisLabels)
			.DisplaySoundLevelAxisLabels(Params.bDisplaySoundLevelAxisLabels)
			.FrequencyAxisScale(Params.FrequencyAxisScale)
			.FrequencyAxisPixelBucketMode(Params.FrequencyAxisPixelBucketMode)
			.OnTiltSpectrumMenuEntryClicked(Params.OnTiltSpectrumMenuEntryClicked)
			.OnFrequencyAxisPixelBucketModeMenuEntryClicked(Params.OnFrequencyAxisPixelBucketModeMenuEntryClicked)
			.OnFrequencyAxisScaleMenuEntryClicked(Params.OnFrequencyAxisScaleMenuEntryClicked)
			.OnDisplayFrequencyAxisLabelsButtonToggled(Params.OnDisplayFrequencyAxisLabelsButtonToggled)
			.OnDisplaySoundLevelAxisLabelsButtonToggled(Params.OnDisplaySoundLevelAxisLabelsButtonToggled)
			.OnGetAudioSpectrumData_Raw(this, &FAudioSpectrumAnalyzer::GetAudioSpectrumData))
		, Ballistics(Params.Ballistics)
		, AnalyzerType(Params.AnalyzerType)
		, FFTAnalyzerFFTSize(Params.FFTAnalyzerFFTSize)
		, CQTAnalyzerFFTSize(Params.CQTAnalyzerFFTSize)
		, OnBallisticsMenuEntryClicked(Params.OnBallisticsMenuEntryClicked)
		, OnAnalyzerTypeMenuEntryClicked(Params.OnAnalyzerTypeMenuEntryClicked)
		, OnFFTAnalyzerFFTSizeMenuEntryClicked(Params.OnFFTAnalyzerFFTSizeMenuEntryClicked)
		, OnCQTAnalyzerFFTSizeMenuEntryClicked(Params.OnCQTAnalyzerFFTSizeMenuEntryClicked)
	{
		SpectrumAnalysisSettings->SpectrumType = EAudioSpectrumType::PowerSpectrum;
		SpectrumAnalysisSettings->FFTSize = FFTAnalyzerFFTSize.Get();
		SpectrumAnalysisSettings->WindowType = EFFTWindowType::Blackman;
		SpectrumAnalysisSettings->bDownmixToMono = true;

		ConstantQSettings->SpectrumType = EAudioSpectrumType::PowerSpectrum;
		ConstantQSettings->NumBandsPerOctave = 6.0f;
		ConstantQSettings->NumBands = 61;
		ConstantQSettings->StartingFrequencyHz = 20000.0f * FMath::Pow(0.5f, (ConstantQSettings->NumBands - 1) / ConstantQSettings->NumBandsPerOctave);
		ConstantQSettings->FFTSize = CQTAnalyzerFFTSize.Get();
		ConstantQSettings->WindowType = EFFTWindowType::Blackman;
		ConstantQSettings->bDownmixToMono = true;
		ConstantQSettings->BandWidthStretch = 2.0f;

		ContextMenuExtension = Widget->AddContextMenuExtension(EExtensionHook::Before, nullptr, FMenuExtensionDelegate::CreateRaw(this, &FAudioSpectrumAnalyzer::ExtendSpectrumPlotContextMenu));

		Init(Params.NumChannels, Params.AudioDeviceId, Params.ExternalAudioBus);
	}

	FAudioSpectrumAnalyzer::FAudioSpectrumAnalyzer(int32 InNumChannels, Audio::FDeviceId InAudioDeviceId, TObjectPtr<UAudioBus> InExternalAudioBus)
		: FAudioSpectrumAnalyzer({ .NumChannels = InNumChannels, .AudioDeviceId = InAudioDeviceId, .ExternalAudioBus = InExternalAudioBus })
	{
		// Delegating constructor
	}

	FAudioSpectrumAnalyzer::~FAudioSpectrumAnalyzer()
	{
		Teardown();

		Widget->UnbindOnGetAudioSpectrumData();

		if (ContextMenuExtension.IsValid())
		{
			Widget->RemoveContextMenuExtension(ContextMenuExtension.ToSharedRef());
		}
	}

	UAudioBus* FAudioSpectrumAnalyzer::GetAudioBus() const
	{
		return AudioBus.Get();
	}

	TSharedRef<SWidget> FAudioSpectrumAnalyzer::GetWidget() const
	{
		return Widget->AsShared();
	}

	void FAudioSpectrumAnalyzer::Init(int32 InNumChannels, Audio::FDeviceId InAudioDeviceId, TObjectPtr<UAudioBus> InExternalAudioBus)
	{
		Teardown();

		AudioDeviceId = InAudioDeviceId;

		// Only create analyzers etc if we have an audio device:
		if (AudioDeviceId != FAudioBusInfo::InvalidAudioDeviceId)
		{
			check(InNumChannels > 0);

			bUseExternalAudioBus = InExternalAudioBus != nullptr;
			AudioBus = bUseExternalAudioBus ? TStrongObjectPtr(InExternalAudioBus.Get()) : TStrongObjectPtr(NewObject<UAudioBus>());
			AudioBus->AudioBusChannels = EAudioBusChannels(InNumChannels - 1);

			CreateSynesthesiaSpectrumAnalyzer();
			CreateConstantQAnalyzer();

			StartAnalyzing(AnalyzerType.Get());
		}
	}

	void FAudioSpectrumAnalyzer::StartAnalyzing(const EAudioSpectrumAnalyzerType InAnalyzerType)
	{
		ensure(!ActiveAnalyzerType.IsSet());

		switch (InAnalyzerType)
		{
		case EAudioSpectrumAnalyzerType::FFT:
			SpectrumAnalyzer->StartAnalyzing(AudioDeviceId, AudioBus.Get());
			break;
		case EAudioSpectrumAnalyzerType::CQT:
			ConstantQAnalyzer->StartAnalyzing(AudioDeviceId, AudioBus.Get());
			break;
		default:
			break;
		}

		ActiveAnalyzerType = InAnalyzerType;
	}

	void FAudioSpectrumAnalyzer::StopAnalyzing()
	{
		ensure(ActiveAnalyzerType.IsSet());

		switch (ActiveAnalyzerType.GetValue())
		{
		case EAudioSpectrumAnalyzerType::FFT:
			SpectrumAnalyzer->StopAnalyzing();
			break;
		case EAudioSpectrumAnalyzerType::CQT:
			ConstantQAnalyzer->StopAnalyzing();
			break;
		default:
			break;
		}

		ActiveAnalyzerType.Reset();
	}

	void FAudioSpectrumAnalyzer::OnSpectrumResults(USynesthesiaSpectrumAnalyzer* InSpectrumAnalyzer, int32 ChannelIndex, const TArray<FSynesthesiaSpectrumResults>& InSpectrumResultsArray)
	{
		if (ActiveAnalyzerType == EAudioSpectrumAnalyzerType::FFT && InSpectrumAnalyzer == SpectrumAnalyzer.Get())
		{
			for (const FSynesthesiaSpectrumResults& SpectrumResults : InSpectrumResultsArray)
			{
				if (PrevTimeStamp.IsSet() && SpectrumResults.TimeSeconds > PrevTimeStamp.GetValue())
				{
					UpdateARSmoothing(SpectrumResults.TimeSeconds, SpectrumResults.SpectrumValues);
				}
				else
				{
					// Find samplerate:
					float SampleRate = 48000.0f;
					if (const FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
					{
						if (const FAudioDevice* AudioDevice = AudioDeviceManager->GetAudioDeviceRaw(AudioDeviceId))
						{
							SampleRate = AudioDevice->GetSampleRate();
						}
					}

					// Init center frequencies:
					CenterFrequencies.SetNumUninitialized(SpectrumAnalyzer->GetNumCenterFrequencies());
					SpectrumAnalyzer->GetCenterFrequencies(SampleRate, CenterFrequencies);

					// Init spectrum data:
					ARSmoothedSquaredMagnitudes = SpectrumResults.SpectrumValues;

					// Update the window compensation power gain:
					TUniquePtr<Audio::IAnalyzerSettings> Settings = SpectrumAnalysisSettings->GetSettings(SampleRate, 1);
					const Audio::FSynesthesiaSpectrumAnalysisSettings* ConcreteSettings = static_cast<Audio::FSynesthesiaSpectrumAnalysisSettings*>(Settings.Get());
					WindowCompensationPowerGain = AudioSpectrumAnalyzerPrivate::GetWindowCompensationPowerGain(ConcreteSettings->WindowType, ConcreteSettings->FFTSize);

					// Apply window compensation power gain:
					Audio::ArrayMultiplyByConstantInPlace(ARSmoothedSquaredMagnitudes, WindowCompensationPowerGain);
				}

				PrevTimeStamp = SpectrumResults.TimeSeconds;
			}
		}
	}

	void FAudioSpectrumAnalyzer::OnConstantQResults(UConstantQAnalyzer* InSpectrumAnalyzer, int32 ChannelIndex, const TArray<FConstantQResults>& InSpectrumResultsArray)
	{
		if (ActiveAnalyzerType == EAudioSpectrumAnalyzerType::CQT && InSpectrumAnalyzer == ConstantQAnalyzer.Get())
		{
			for (const FConstantQResults& SpectrumResults : InSpectrumResultsArray)
			{
				if (PrevTimeStamp.IsSet() && SpectrumResults.TimeSeconds > PrevTimeStamp.GetValue())
				{
					UpdateARSmoothing(SpectrumResults.TimeSeconds, SpectrumResults.SpectrumValues);
				}
				else
				{
					// Init center frequencies:
					CenterFrequencies.SetNumUninitialized(ConstantQAnalyzer->GetNumCenterFrequencies());
					ConstantQAnalyzer->GetCenterFrequencies(CenterFrequencies);

					// Init spectrum data:
					ARSmoothedSquaredMagnitudes = SpectrumResults.SpectrumValues;

					// Update the window compensation power gain:
					TUniquePtr<Audio::IAnalyzerSettings> Settings = ConstantQSettings->GetSettings(0.0f, 1);
					const Audio::FConstantQSettings* ConcreteSettings = static_cast<Audio::FConstantQSettings*>(Settings.Get());
					WindowCompensationPowerGain = AudioSpectrumAnalyzerPrivate::GetWindowCompensationPowerGain(ConcreteSettings->WindowType, ConcreteSettings->FFTSize);

					// Apply window compensation power gain:
					Audio::ArrayMultiplyByConstantInPlace(ARSmoothedSquaredMagnitudes, WindowCompensationPowerGain);
				}

				PrevTimeStamp = SpectrumResults.TimeSeconds;
			}
		}
	}

	void FAudioSpectrumAnalyzer::UpdateARSmoothing(const float TimeStamp, TConstArrayView<float> SquaredMagnitudes)
	{
		// Calculate AR smoother coefficients:
		const float DeltaT = (TimeStamp - PrevTimeStamp.GetValue());
		const bool bIsAnalogAttackRelease = (Ballistics.Get() == EAudioSpectrumAnalyzerBallistics::Analog);
		Audio::FAttackRelease AttackRelease(1.0f / DeltaT, AttackTimeMsec, ReleaseTimeMsec, bIsAnalogAttackRelease);

		// Apply AR smoothing for each frequency:
		check(SquaredMagnitudes.Num() == ARSmoothedSquaredMagnitudes.Num());
		for (int Index = 0; Index < SquaredMagnitudes.Num(); Index++)
		{
			const float OldValue = ARSmoothedSquaredMagnitudes[Index];
			const float NewValue = WindowCompensationPowerGain * SquaredMagnitudes[Index];
			const float ARSmootherCoefficient = (NewValue >= OldValue) ? AttackRelease.GetAttackTimeSamples() : AttackRelease.GetReleaseTimeSamples();
			ARSmoothedSquaredMagnitudes[Index] = FMath::Lerp(NewValue, OldValue, ARSmootherCoefficient);
		}
	}

	void FAudioSpectrumAnalyzer::Teardown()
	{
		if (UObjectInitialized())
		{
			if (SpectrumAnalyzer.IsValid() && SpectrumAnalyzer->IsValidLowLevel())
			{
				if (ActiveAnalyzerType == EAudioSpectrumAnalyzerType::FFT)
				{
					SpectrumAnalyzer->StopAnalyzing();
				}

				ReleaseSynesthesiaSpectrumAnalyzer();
			}

			if (ConstantQAnalyzer.IsValid() && ConstantQAnalyzer->IsValidLowLevel())
			{
				if (ActiveAnalyzerType == EAudioSpectrumAnalyzerType::CQT)
				{
					ConstantQAnalyzer->StopAnalyzing();
				}

				ReleaseConstantQAnalyzer();
			}
		}

		SpectrumResultsDelegateHandle.Reset();
		SpectrumAnalyzer.Reset();

		ConstantQResultsDelegateHandle.Reset();
		ConstantQAnalyzer.Reset();

		ActiveAnalyzerType.Reset();
		PrevTimeStamp.Reset();
		CenterFrequencies.Empty();
		ARSmoothedSquaredMagnitudes.Empty();

		AudioBus.Reset();
		bUseExternalAudioBus = false;
	}

	FAudioPowerSpectrumData FAudioSpectrumAnalyzer::GetAudioSpectrumData()
	{
		// The SAudioSpectrumPlot regularly polls us for audio spectrum data.
		// We can update the analyzer settings here:
		UpdateAnalyzerSettings();

		check(CenterFrequencies.Num() == ARSmoothedSquaredMagnitudes.Num());
		return FAudioPowerSpectrumData{ CenterFrequencies, ARSmoothedSquaredMagnitudes };
	}

	void FAudioSpectrumAnalyzer::ExtendSpectrumPlotContextMenu(FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.BeginSection("AnalyzerSettings", LOCTEXT("AnalyzerSettings", "Analyzer Settings"));
		MenuBuilder.AddSubMenu(
			LOCTEXT("Ballistics", "Ballistics"),
			FText(),
			FNewMenuDelegate::CreateSP(this, &FAudioSpectrumAnalyzer::BuildBallisticsSubMenu));
		MenuBuilder.AddSubMenu(
			LOCTEXT("AnalyzerType", "Analyzer Type"),
			FText(),
			FNewMenuDelegate::CreateSP(this, &FAudioSpectrumAnalyzer::BuildAnalyzerTypeSubMenu));
		MenuBuilder.AddSubMenu(
			LOCTEXT("FFTSize", "FFT Size"),
			FText(),
			FNewMenuDelegate::CreateSP(this, &FAudioSpectrumAnalyzer::BuildFFTSizeSubMenu));
		MenuBuilder.EndSection();
	}

	void FAudioSpectrumAnalyzer::BuildBallisticsSubMenu(FMenuBuilder& SubMenu)
	{
		const UEnum* EnumClass = StaticEnum<EAudioSpectrumAnalyzerBallistics>();
		const int32 NumEnumValues = EnumClass->NumEnums() - 1; // Exclude 'MAX' enum value.
		for (int32 Index = 0; Index < NumEnumValues; Index++)
		{
			const auto EnumValue = static_cast<EAudioSpectrumAnalyzerBallistics>(EnumClass->GetValueByIndex(Index));

			SubMenu.AddMenuEntry(
				EnumClass->GetDisplayNameTextByIndex(Index),
#if WITH_EDITOR
				EnumClass->GetToolTipTextByIndex(Index),
#else
				FText(),
#endif
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSPLambda(this, [this, EnumValue]()
						{
							if (!Ballistics.IsBound())
							{
								Ballistics = EnumValue;
							}

							OnBallisticsMenuEntryClicked.ExecuteIfBound(EnumValue);
						}),
					FCanExecuteAction(),
					FIsActionChecked::CreateSPLambda(this, [this, EnumValue]() { return (Ballistics.Get() == EnumValue); })
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);
		}
	}

	void FAudioSpectrumAnalyzer::BuildAnalyzerTypeSubMenu(FMenuBuilder& SubMenu)
	{
		const UEnum* EnumClass = StaticEnum<EAudioSpectrumAnalyzerType>();
		const int32 NumEnumValues = EnumClass->NumEnums() - 1; // Exclude 'MAX' enum value.
		for (int32 Index = 0; Index < NumEnumValues; Index++)
		{
			const auto EnumValue = static_cast<EAudioSpectrumAnalyzerType>(EnumClass->GetValueByIndex(Index));

			SubMenu.AddMenuEntry(
				EnumClass->GetDisplayNameTextByIndex(Index),
#if WITH_EDITOR
				EnumClass->GetToolTipTextByIndex(Index),
#else
				FText(),
#endif
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSPLambda(this, [this, EnumValue]()
						{
							if (!AnalyzerType.IsBound())
							{
								AnalyzerType = EnumValue;
							}

							OnAnalyzerTypeMenuEntryClicked.ExecuteIfBound(EnumValue);
						}),
					FCanExecuteAction(),
					FIsActionChecked::CreateSPLambda(this, [this, EnumValue]() { return (AnalyzerType.Get() == EnumValue); })
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);
		}
	}

	void FAudioSpectrumAnalyzer::BuildFFTSizeSubMenu(FMenuBuilder& SubMenu)
	{
		// There is a different FFTSize enum depending on the analyzer type.

		if (AnalyzerType.Get() == EAudioSpectrumAnalyzerType::FFT)
		{
			const UEnum* EnumClass = StaticEnum<EFFTSize>();
			const int32 NumEnumValues = EnumClass->NumEnums() - 1; // Exclude 'MAX' enum value.
			for (int32 Index = 0; Index < NumEnumValues; Index++)
			{
				const auto EnumValue = static_cast<EFFTSize>(EnumClass->GetValueByIndex(Index));
				if (EnumValue == EFFTSize::DefaultSize)
				{
					// Skip the duplicate 512 enum value 'DefaultSize'.
					continue;
				}

				SubMenu.AddMenuEntry(
					EnumClass->GetDisplayNameTextByIndex(Index),
#if WITH_EDITOR
					EnumClass->GetToolTipTextByIndex(Index),
#else
					FText(),
#endif
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSPLambda(this, [this, EnumValue]()
							{
								if (!FFTAnalyzerFFTSize.IsBound())
								{
									FFTAnalyzerFFTSize = EnumValue;
								}

								OnFFTAnalyzerFFTSizeMenuEntryClicked.ExecuteIfBound(EnumValue);
							}),
						FCanExecuteAction(),
						FIsActionChecked::CreateSPLambda(this, [this, EnumValue]() { return (FFTAnalyzerFFTSize.Get() == EnumValue); })
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton);
			}
		}
		else if (AnalyzerType.Get() == EAudioSpectrumAnalyzerType::CQT)
		{
			const UEnum* EnumClass = StaticEnum<EConstantQFFTSizeEnum>();
			const int32 NumEnumValues = EnumClass->NumEnums() - 1; // Exclude 'MAX' enum value.
			for (int32 Index = 0; Index < NumEnumValues; Index++)
			{
				const auto EnumValue = static_cast<EConstantQFFTSizeEnum>(EnumClass->GetValueByIndex(Index));

				SubMenu.AddMenuEntry(
					EnumClass->GetDisplayNameTextByIndex(Index),
#if WITH_EDITOR
					EnumClass->GetToolTipTextByIndex(Index),
#else
					FText(),
#endif
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSPLambda(this, [this, EnumValue]()
							{
								if (!CQTAnalyzerFFTSize.IsBound())
								{
									CQTAnalyzerFFTSize = EnumValue;
								}

								OnCQTAnalyzerFFTSizeMenuEntryClicked.ExecuteIfBound(EnumValue);
							}),
						FCanExecuteAction(),
						FIsActionChecked::CreateSPLambda(this, [this, EnumValue]() { return (CQTAnalyzerFFTSize.Get() == EnumValue); })
					),
					NAME_None,
					EUserInterfaceActionType::ToggleButton);
			}
		}
	}

	void FAudioSpectrumAnalyzer::UpdateAnalyzerSettings()
	{
		if (AudioDeviceId == FAudioBusInfo::InvalidAudioDeviceId)
		{
			// No analyzers available if no valid audio device.
			ensure(!ActiveAnalyzerType.IsSet());
			return;
		}

		const EAudioSpectrumAnalyzerType RequiredAnalyzerType = AnalyzerType.Get();
		const EFFTSize FFTAnalyzerRequiredFFTSize = FFTAnalyzerFFTSize.Get();
		const EConstantQFFTSizeEnum CQTAnalyzerRequiredFFTSize = CQTAnalyzerFFTSize.Get();

		const bool bRequiredAnalyzerTypeChanged = (ActiveAnalyzerType != RequiredAnalyzerType);
		const bool bFFTAnalyzerRequiredFFTSizeChanged = (SpectrumAnalysisSettings->FFTSize != FFTAnalyzerRequiredFFTSize);
		const bool bCQTAnalyzerRequiredFFTSizeChanged = (ConstantQSettings->FFTSize != CQTAnalyzerRequiredFFTSize);
		if (bRequiredAnalyzerTypeChanged || bFFTAnalyzerRequiredFFTSizeChanged || bCQTAnalyzerRequiredFFTSizeChanged)
		{
			StopAnalyzing();

			if (bRequiredAnalyzerTypeChanged)
			{
				// There will be different center frequencies when the analyzer type changes:
				PrevTimeStamp.Reset();
				CenterFrequencies.Reset();
				ARSmoothedSquaredMagnitudes.Reset();
			}

			if (bFFTAnalyzerRequiredFFTSizeChanged)
			{
				ReleaseSynesthesiaSpectrumAnalyzer();
				SpectrumAnalysisSettings->FFTSize = FFTAnalyzerRequiredFFTSize;
				CreateSynesthesiaSpectrumAnalyzer();
			}

			if (bCQTAnalyzerRequiredFFTSizeChanged)
			{
				ReleaseConstantQAnalyzer();
				ConstantQSettings->FFTSize = CQTAnalyzerRequiredFFTSize;
				CreateConstantQAnalyzer();
			}

			StartAnalyzing(RequiredAnalyzerType);
		}
	}

	void FAudioSpectrumAnalyzer::CreateSynesthesiaSpectrumAnalyzer()
	{
		ensure(!SpectrumAnalyzer.IsValid());
		ensure(!SpectrumResultsDelegateHandle.IsValid());

		SpectrumAnalyzer = TStrongObjectPtr(NewObject<USynesthesiaSpectrumAnalyzer>());
		SpectrumAnalyzer->Settings = SpectrumAnalysisSettings.Get();
		SpectrumResultsDelegateHandle = SpectrumAnalyzer->OnSpectrumResultsNative.AddRaw(this, &FAudioSpectrumAnalyzer::OnSpectrumResults);
	}

	void FAudioSpectrumAnalyzer::ReleaseSynesthesiaSpectrumAnalyzer()
	{
		if (ensure(SpectrumAnalyzer.IsValid() && SpectrumResultsDelegateHandle.IsValid()))
		{
			SpectrumAnalyzer->OnSpectrumResultsNative.Remove(SpectrumResultsDelegateHandle);
		}

		SpectrumResultsDelegateHandle.Reset();
		SpectrumAnalyzer.Reset();
	}

	void FAudioSpectrumAnalyzer::CreateConstantQAnalyzer()
	{
		ConstantQAnalyzer = TStrongObjectPtr(NewObject<UConstantQAnalyzer>());
		ConstantQAnalyzer->Settings = ConstantQSettings.Get();
		ConstantQResultsDelegateHandle = ConstantQAnalyzer->OnConstantQResultsNative.AddRaw(this, &FAudioSpectrumAnalyzer::OnConstantQResults);
	}

	void FAudioSpectrumAnalyzer::ReleaseConstantQAnalyzer()
	{
		if (ensure(ConstantQAnalyzer.IsValid() && ConstantQResultsDelegateHandle.IsValid()))
		{
			ConstantQAnalyzer->OnConstantQResultsNative.Remove(ConstantQResultsDelegateHandle);
		}

		ConstantQResultsDelegateHandle.Reset();
		ConstantQAnalyzer.Reset();
	}

	TSharedRef<SDockTab> FAudioSpectrumAnalyzer::SpawnTab(const FSpawnTabArgs& Args) const
	{
		return SNew(SDockTab)
			.Clipping(EWidgetClipping::ClipToBounds)
			.Label(RackUnitTypeInfo.DisplayName)
			[
				GetWidget()
			];
	}

	TSharedRef<IAudioAnalyzerRackUnit> FAudioSpectrumAnalyzer::MakeRackUnit(const FAudioAnalyzerRackUnitConstructParams& Params)
	{
		using namespace AudioSpectrumAnalyzerPrivate;

		FAudioSpectrumAnalyzerParams AnalyzerParams;
		AnalyzerParams.NumChannels = Params.AudioBusInfo.GetNumChannels();
		AnalyzerParams.AudioDeviceId = Params.AudioBusInfo.AudioDeviceId;
		AnalyzerParams.ExternalAudioBus = Params.AudioBusInfo.AudioBus;

		if (Params.EditorSettingsClass != nullptr)
		{
			// If we have been given a valid editor settings class, bind analyzer options to the settings:
			const FProperty* SpectrumAnalyzerSettingsProperty = Params.EditorSettingsClass->FindPropertyByName("SpectrumAnalyzerSettings");
			if (SpectrumAnalyzerSettingsProperty != nullptr)
			{
				const FRackUnitSettingsHelper SettingsHelper(*SpectrumAnalyzerSettingsProperty);

				AnalyzerParams.Ballistics.BindLambda([=]()
					{
						return SettingsHelper.GetRackUnitSettings()->Ballistics;
					});
				AnalyzerParams.AnalyzerType.BindLambda([=]()
					{
						return SettingsHelper.GetRackUnitSettings()->AnalyzerType;
					});
				AnalyzerParams.FFTAnalyzerFFTSize.BindLambda([=]()
					{
						return SettingsHelper.GetRackUnitSettings()->FFTAnalyzerFFTSize;
					});
				AnalyzerParams.CQTAnalyzerFFTSize.BindLambda([=]()
					{
						return SettingsHelper.GetRackUnitSettings()->CQTAnalyzerFFTSize;
					});
				AnalyzerParams.TiltExponent.BindLambda([=]()
					{
						const EAudioSpectrumPlotTilt TiltSpectrum = SettingsHelper.GetRackUnitSettings()->TiltSpectrum;
						return SAudioSpectrumPlot::GetTiltExponentValue(TiltSpectrum);
					});
				AnalyzerParams.FrequencyAxisPixelBucketMode.BindLambda([=]()
					{
						return SettingsHelper.GetRackUnitSettings()->PixelPlotMode;
					});
				AnalyzerParams.FrequencyAxisScale.BindLambda([=]()
					{
						return SettingsHelper.GetRackUnitSettings()->FrequencyScale;
					});
				AnalyzerParams.bDisplayFrequencyAxisLabels.BindLambda([=]()
					{
						return SettingsHelper.GetRackUnitSettings()->bDisplayFrequencyAxisLabels;
					});
				AnalyzerParams.bDisplaySoundLevelAxisLabels.BindLambda([=]()
					{
						return SettingsHelper.GetRackUnitSettings()->bDisplaySoundLevelAxisLabels;
					});

				AnalyzerParams.OnBallisticsMenuEntryClicked.BindLambda([=](EAudioSpectrumAnalyzerBallistics SelectedValue)
					{
						SettingsHelper.GetRackUnitSettings()->Ballistics = SelectedValue;
						SettingsHelper.SaveConfig();
					});
				AnalyzerParams.OnAnalyzerTypeMenuEntryClicked.BindLambda([=](EAudioSpectrumAnalyzerType SelectedValue)
					{
						SettingsHelper.GetRackUnitSettings()->AnalyzerType = SelectedValue;
						SettingsHelper.SaveConfig();
					});
				AnalyzerParams.OnFFTAnalyzerFFTSizeMenuEntryClicked.BindLambda([=](EFFTSize SelectedValue)
					{
						SettingsHelper.GetRackUnitSettings()->FFTAnalyzerFFTSize = SelectedValue;
						SettingsHelper.SaveConfig();
					});
				AnalyzerParams.OnCQTAnalyzerFFTSizeMenuEntryClicked.BindLambda([=](EConstantQFFTSizeEnum SelectedValue)
					{
						SettingsHelper.GetRackUnitSettings()->CQTAnalyzerFFTSize = SelectedValue;
						SettingsHelper.SaveConfig();
					});
				AnalyzerParams.OnTiltSpectrumMenuEntryClicked.BindLambda([=](EAudioSpectrumPlotTilt SelectedValue)
					{
						SettingsHelper.GetRackUnitSettings()->TiltSpectrum = SelectedValue;
						SettingsHelper.SaveConfig();
					});
				AnalyzerParams.OnFrequencyAxisPixelBucketModeMenuEntryClicked.BindLambda([=](EAudioSpectrumPlotFrequencyAxisPixelBucketMode SelectedValue)
					{
						SettingsHelper.GetRackUnitSettings()->PixelPlotMode = SelectedValue;
						SettingsHelper.SaveConfig();
					});
				AnalyzerParams.OnFrequencyAxisScaleMenuEntryClicked.BindLambda([=](EAudioSpectrumPlotFrequencyAxisScale SelectedValue)
					{
						SettingsHelper.GetRackUnitSettings()->FrequencyScale = SelectedValue;
						SettingsHelper.SaveConfig();
					});
				AnalyzerParams.OnDisplayFrequencyAxisLabelsButtonToggled.BindLambda([=]()
					{
						FSpectrumAnalyzerRackUnitSettings* SpectrumAnalyzerSettings = SettingsHelper.GetRackUnitSettings();
						SpectrumAnalyzerSettings->bDisplayFrequencyAxisLabels = !SpectrumAnalyzerSettings->bDisplayFrequencyAxisLabels;
						SettingsHelper.SaveConfig();
					});
				AnalyzerParams.OnDisplaySoundLevelAxisLabelsButtonToggled.BindLambda([=]()
					{
						FSpectrumAnalyzerRackUnitSettings* SpectrumAnalyzerSettings = SettingsHelper.GetRackUnitSettings();
						SpectrumAnalyzerSettings->bDisplaySoundLevelAxisLabels = !SpectrumAnalyzerSettings->bDisplaySoundLevelAxisLabels;
						SettingsHelper.SaveConfig();
					});
			}
		}

		AnalyzerParams.PlotStyle = &Params.StyleSet->GetWidgetStyle<FAudioSpectrumPlotStyle>("AudioSpectrumPlot.Style");

		return MakeShared<FAudioSpectrumAnalyzer>(AnalyzerParams);
	}
} // namespace AudioWidgets

#undef LOCTEXT_NAMESPACE
