// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioSpectrumPlotStyle.h"
#include "Styling/StyleDefaults.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioSpectrumPlotStyle)

FAudioSpectrumPlotStyle::FAudioSpectrumPlotStyle()
{
}


const FName FAudioSpectrumPlotStyle::TypeName("FAudioSpectrumPlotStyle");

const FAudioSpectrumPlotStyle& FAudioSpectrumPlotStyle::GetDefault()
{
	static FAudioSpectrumPlotStyle Default;
	return Default;
}

