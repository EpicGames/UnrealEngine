// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioVectorscope.h"

#include "SAudioVectorscopePanelWidget.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "FAudioVectorscope"

namespace AudioWidgets
{
	const FAudioAnalyzerRackUnitTypeInfo FAudioVectorscope::RackUnitTypeInfo
	{
		.TypeName = TEXT("FAudioVectorscope"),
		.DisplayName = LOCTEXT("AudioVectorscopeDisplayName", "Vectorscope"),
		.OnMakeAudioAnalyzerRackUnit = FOnMakeAudioAnalyzerRackUnit::CreateStatic(&MakeRackUnit),
		.VerticalSizeCoefficient = 0.25f,
	};

	FAudioVectorscope::FAudioVectorscope(Audio::FDeviceId InAudioDeviceId,
		const uint32 InNumChannels,
		const float InTimeWindowMs,
		const float InMaxTimeWindowMs,
		const float InAnalysisPeriodMs,
		const EAudioPanelLayoutType InPanelLayoutType,
		const FAudioVectorscopePanelStyle* InPanelStyle,
		TObjectPtr<UAudioBus> InExternalAudioBus)
	{
		VectorscopePanelStyle = InPanelStyle ? *InPanelStyle : FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioVectorscopePanelStyle>("AudioVectorscope.PanelStyle");

		if (InExternalAudioBus != nullptr)
		{
			ensure(InExternalAudioBus->GetNumChannels() == InNumChannels);
			AudioBus = TStrongObjectPtr(InExternalAudioBus.Get());
		}
		else if (InNumChannels > 0)
		{
			CreateAudioBus(InNumChannels);
		}
		
		CreateDataProvider(InAudioDeviceId, InTimeWindowMs, InMaxTimeWindowMs, InAnalysisPeriodMs);
		CreateVectorscopeWidget(InPanelLayoutType);
	}

	void FAudioVectorscope::CreateAudioBus(const uint32 InNumChannels)
	{
		AudioBus = TStrongObjectPtr(NewObject<UAudioBus>());
		AudioBus->AudioBusChannels = AudioBusUtils::ConvertIntToEAudioBusChannels(InNumChannels);
	}

	void FAudioVectorscope::CreateDataProvider(Audio::FDeviceId InAudioDeviceId, const float InTimeWindowMs,	const float InMaxTimeWindowMs, const float InAnalysisPeriodMs)
	{
		if (InAudioDeviceId == FAudioBusInfo::InvalidAudioDeviceId)
		{
			// Can't have a data provider without a valid audio device.
			AudioSamplesDataProvider.Reset();
			return;
		}

		check(AudioBus);

		AudioSamplesDataProvider = MakeShared<FWaveformAudioSamplesDataProvider>(InAudioDeviceId, AudioBus.Get(), AudioBus->GetNumChannels(), InTimeWindowMs, InMaxTimeWindowMs, InAnalysisPeriodMs);
	}

	void FAudioVectorscope::CreateVectorscopeWidget(const EAudioPanelLayoutType InPanelLayoutType, const FAudioVectorscopePanelStyle* PanelStyle)
	{
		FFixedSampledSequenceView SequenceView{ .NumDimensions = 2, .SampleRate = 48000 }; // Initialize with usable fallback values (but no sample data in the view).

		if (AudioSamplesDataProvider.IsValid())
		{
			// Get the actual sequence view from the data provider:
			SequenceView = AudioSamplesDataProvider->GetDataView();
		}

		if (PanelStyle)
		{
			VectorscopePanelStyle = *PanelStyle;
		}

		if (!VectorscopePanelWidget.IsValid())
		{
			VectorscopePanelWidget = SNew(SAudioVectorscopePanelWidget, SequenceView)
				.PanelLayoutType(InPanelLayoutType)
				.PanelStyle(&VectorscopePanelStyle);
		}
		else
		{
			VectorscopePanelWidget->BuildWidget(SequenceView, InPanelLayoutType);
		}

		if (AudioSamplesDataProvider.IsValid())
		{
			// Interconnect data provider and widget
			AudioSamplesDataProvider->OnDataViewGenerated.AddSP(VectorscopePanelWidget.Get(), &SAudioVectorscopePanelWidget::ReceiveSequenceView);

			if (InPanelLayoutType == EAudioPanelLayoutType::Advanced)
			{
				VectorscopePanelWidget->OnDisplayPersistenceValueChanged.AddSP(AudioSamplesDataProvider.Get(), &FWaveformAudioSamplesDataProvider::SetTimeWindow);
			}
		}
	}

	void FAudioVectorscope::StartProcessing()
	{
		if (AudioSamplesDataProvider.IsValid())
		{
			AudioSamplesDataProvider->StartProcessing();
		}
	}

	void FAudioVectorscope::StopProcessing()
	{
		if (AudioSamplesDataProvider.IsValid())
		{
			AudioSamplesDataProvider->StopProcessing();
		}
	}

	UAudioBus* FAudioVectorscope::GetAudioBus() const
	{
		return AudioBus.Get();
	}

	TSharedRef<SWidget> FAudioVectorscope::GetPanelWidget() const
	{
		return VectorscopePanelWidget.ToSharedRef();
	}

	void FAudioVectorscope::SetAudioBusInfo(const FAudioBusInfo& AudioBusInfo)
	{
		AudioBus = TStrongObjectPtr(AudioBusInfo.AudioBus.Get());
		CreateDataProvider(AudioBusInfo.AudioDeviceId, RackUnitTimeWindowMs, RackUnitMaxTimeWindowMs, RackUnitAnalysisPeriodMs);
		CreateVectorscopeWidget(RackUnitPanelLayoutType);
	}

	TSharedRef<SDockTab> FAudioVectorscope::SpawnTab(const FSpawnTabArgs& Args) const
	{
		return SNew(SDockTab)
			.Clipping(EWidgetClipping::ClipToBounds)
			.Label(RackUnitTypeInfo.DisplayName)
			[
				GetPanelWidget()
			];
	}

	TSharedRef<IAudioAnalyzerRackUnit> FAudioVectorscope::MakeRackUnit(const FAudioAnalyzerRackUnitConstructParams& Params)
	{
		return MakeShared<FAudioVectorscope>(
			Params.AudioBusInfo.AudioDeviceId,
			Params.AudioBusInfo.GetNumChannels(),
			RackUnitTimeWindowMs,
			RackUnitMaxTimeWindowMs,
			RackUnitAnalysisPeriodMs,
			RackUnitPanelLayoutType,
			&Params.StyleSet->GetWidgetStyle<FAudioVectorscopePanelStyle>("AudioVectorscope.PanelStyle"),
			Params.AudioBusInfo.AudioBus);
	}
} // namespace AudioWidgets

#undef LOCTEXT_NAMESPACE
