// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioVectorscopeUMG.h"

#include "AudioMixerDevice.h"
#include "Engine/World.h"
#include "SAudioVectorscopePanelWidget.h"
#include "WaveformAudioSamplesDataProvider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioVectorscopeUMG)

#define LOCTEXT_NAMESPACE "AudioVectorscopeUMG"

UAudioVectorscope::UAudioVectorscope(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, VectorscopeStyle(FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioVectorscopePanelStyle>("AudioVectorscope.PanelStyle"))
{
#if WITH_EDITORONLY_DATA
	AccessibleBehavior       = ESlateAccessibleBehavior::NotAccessible;
	bCanChildrenBeAccessible = false;
#endif

	DummyAudioSamples.Init(0.0f, 100);
	DummyDataView = FFixedSampledSequenceView{ MakeArrayView(DummyAudioSamples.GetData(), DummyAudioSamples.Num()), 2, 48000 };
}

void UAudioVectorscope::CreateDummyVectorscopeWidget()
{
	VectorscopePanelWidget = SNew(SAudioVectorscopePanelWidget, DummyDataView)
	.PanelLayoutType(PanelLayoutType)
	.PanelStyle(&VectorscopeStyle);
}

void UAudioVectorscope::CreateDataProvider()
{
	UWorld* World = GetWorld();
	if (!World)
	{
		return;
	}

	const FAudioDeviceHandle AudioDevice = World->GetAudioDevice();
	if (!AudioDevice.IsValid())
	{
		return;
	}

	AudioSamplesDataProvider = MakeShared<AudioWidgets::FWaveformAudioSamplesDataProvider>(AudioDevice.GetDeviceID(), AudioBus, AudioBus->GetNumChannels(), DisplayPersistenceMs, MaxDisplayPersistenceMs, AnalysisPeriodMs);
}

void UAudioVectorscope::CreateVectorscopeWidget()
{
	check(AudioSamplesDataProvider);

	using namespace AudioWidgets;

	const FFixedSampledSequenceView SequenceView = AudioSamplesDataProvider->GetDataView();

	VectorscopePanelWidget = SNew(SAudioVectorscopePanelWidget, SequenceView)
	.PanelLayoutType(PanelLayoutType)
	.PanelStyle(&VectorscopeStyle);

	// Interconnect data provider and widget
	AudioSamplesDataProvider->OnDataViewGenerated.AddSP(VectorscopePanelWidget.Get(), &SAudioVectorscopePanelWidget::ReceiveSequenceView);

	if (PanelLayoutType == EAudioPanelLayoutType::Advanced)
	{
		VectorscopePanelWidget->OnDisplayPersistenceValueChanged.AddSP(AudioSamplesDataProvider.Get(), &FWaveformAudioSamplesDataProvider::SetTimeWindow);
	}
}

TSharedRef<SWidget> UAudioVectorscope::RebuildWidget()
{
	DisplayPersistenceMs = FMath::Clamp(DisplayPersistenceMs, 10.0f, MaxDisplayPersistenceMs);

	if (!AudioBus)
	{
		CreateDummyVectorscopeWidget();
	}
	else
	{
		CreateDataProvider();
		CreateVectorscopeWidget();
	}

	return VectorscopePanelWidget.ToSharedRef();
}

void UAudioVectorscope::SynchronizeProperties()
{
	Super::SynchronizeProperties();

	DisplayPersistenceMs = FMath::Clamp(DisplayPersistenceMs, 10.0f, MaxDisplayPersistenceMs);

	if (!AudioBus)
	{
		if (AudioSamplesDataProvider.IsValid())
		{
			AudioSamplesDataProvider.Reset();
			CreateDummyVectorscopeWidget();
		}
	}
	else
	{
		if (!AudioSamplesDataProvider.IsValid() || AudioBus != AudioSamplesDataProvider->GetAudioBus())
		{
			CreateDataProvider();
			CreateVectorscopeWidget();
		}
	}

	if (AudioSamplesDataProvider.IsValid())
	{
		if (MaxDisplayPersistenceMs != AudioSamplesDataProvider->GetMaxTimeWindowMs())
		{
			AudioSamplesDataProvider->SetMaxTimeWindowMs(MaxDisplayPersistenceMs);
		}

		AudioSamplesDataProvider->SetTimeWindow(DisplayPersistenceMs);

		AudioSamplesDataProvider->RequestSequenceView(TRange<double>::Inclusive(0, 1));
	}

	if (VectorscopePanelWidget.IsValid())
	{
		if (PanelLayoutType != VectorscopePanelWidget->GetPanelLayoutType() && AudioSamplesDataProvider.IsValid())
		{
			CreateVectorscopeWidget();
		}

		VectorscopePanelWidget->UpdateValueGridOverlayStyle(VectorscopeStyle.ValueGridStyle);
		VectorscopePanelWidget->SetValueGridOverlayMaxNumDivisions(GridDivisions);

		VectorscopePanelWidget->UpdateSequenceVectorViewerStyle(VectorscopeStyle.VectorViewerStyle);

		VectorscopePanelWidget->SetMaxDisplayPersistence(MaxDisplayPersistenceMs);

		VectorscopePanelWidget->SetDisplayPersistence(DisplayPersistenceMs);
		VectorscopePanelWidget->SetVectorViewerScaleFactor(Scale);

		VectorscopePanelWidget->SetGridVisibility(bShowGrid);
	}
}

void UAudioVectorscope::ReleaseSlateResources(bool bReleaseChildren)
{
	Super::ReleaseSlateResources(bReleaseChildren);
	
	VectorscopePanelWidget.Reset();
}

#if WITH_EDITOR
const FText UAudioVectorscope::GetPaletteCategory()
{
	return LOCTEXT("Audio", "Audio");
}
#endif

void UAudioVectorscope::StartProcessing()
{
	if (AudioSamplesDataProvider.IsValid())
	{
		AudioSamplesDataProvider->StartProcessing();
	}
}

void UAudioVectorscope::StopProcessing()
{
	if (AudioSamplesDataProvider.IsValid())
	{
		AudioSamplesDataProvider->StopProcessing();
	}
}

#undef LOCTEXT_NAMESPACE
