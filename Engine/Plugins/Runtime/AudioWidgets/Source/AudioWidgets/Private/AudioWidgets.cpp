// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioWidgets.h"
#include "AdvancedWidgetsModule.h"
#include "AudioAnalyzerRackUnitRegistry.h"
#include "AudioLoudnessMeter.h"
#include "AudioMeter.h"
#include "AudioOscilloscope.h"
#include "AudioSpectrogram.h"
#include "AudioSpectrumAnalyzer.h"
#include "AudioVectorscope.h"
#include "AudioWidgetsStyle.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FAudioWidgetsModule"

void FAudioWidgetsModule::StartupModule()
{
	using namespace AudioWidgets;

	// Required to load so the AudioWidget plugin content can reference widgets
	// defined in AdvancedWidgets (ex. RadialSlider for UMG-defined knobs)
	FModuleManager::Get().LoadModuleChecked<FAdvancedWidgetsModule>("AdvancedWidgets");
	// Initialize static style instance 
	FAudioWidgetsStyle::Get();

	// Register standard analyzer rack units:
	RegisterAudioAnalyzerRackUnitType(&FAudioLoudnessMeter::RackUnitTypeInfo);
	RegisterAudioAnalyzerRackUnitType(&FAudioMeter::RackUnitTypeInfo);
	RegisterAudioAnalyzerRackUnitType(&FAudioOscilloscope::RackUnitTypeInfo);
	RegisterAudioAnalyzerRackUnitType(&FAudioVectorscope::RackUnitTypeInfo);
	RegisterAudioAnalyzerRackUnitType(&FAudioSpectrogram::RackUnitTypeInfo);
	RegisterAudioAnalyzerRackUnitType(&FAudioSpectrumAnalyzer::RackUnitTypeInfo);
}

void FAudioWidgetsModule::ShutdownModule()
{
	AudioWidgets::FAudioAnalyzerRackUnitRegistry::TearDown();
}

void FAudioWidgetsModule::RegisterAudioAnalyzerRackUnitType(const AudioWidgets::FAudioAnalyzerRackUnitTypeInfo* RackUnitTypeInfo)
{
	AudioWidgets::FAudioAnalyzerRackUnitRegistry::Get().RegisterRackUnitType(RackUnitTypeInfo);
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FAudioWidgetsModule, AudioWidgets)
