// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioWidgetsLog.h"

DEFINE_LOG_CATEGORY(LogAudioWidgets);
