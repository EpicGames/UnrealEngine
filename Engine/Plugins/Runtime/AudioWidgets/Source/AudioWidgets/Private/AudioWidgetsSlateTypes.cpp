// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioWidgetsSlateTypes.h"

#include "Brushes/SlateNoResource.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioWidgetsSlateTypes)

namespace AudioWidgetStylesSharedParams
{
	const FLazyName BackgroundBrushName = "WhiteBrush";
	const FLinearColor PlayheadColor = FLinearColor(255.f, 0.1f, 0.2f, 1.f);
	const FLinearColor RulerTicksColor = FLinearColor(1.f, 1.f, 1.f, 0.9f);
	const float DefaultHeight = 720.f;
	const float DefaultWidth = 1280.f;
}

// Audio Text Box Style 
FAudioTextBoxStyle::FAudioTextBoxStyle()
	: BackgroundImage(FSlateRoundedBoxBrush(FStyleColors::White, 4.0f, FVector2D(56.0f, 28.0f)))
	, BackgroundColor(FStyleColors::Recessed)
{
}

void FAudioTextBoxStyle::GetResources(TArray< const FSlateBrush* >& OutBrushes) const
{
	OutBrushes.Add(&BackgroundImage);
}

const FName FAudioTextBoxStyle::TypeName("FAudioTextBoxStyle");

const FAudioTextBoxStyle& FAudioTextBoxStyle::GetDefault()
{
	static FAudioTextBoxStyle Default;
	return Default;
}

// Audio Slider Style 
FAudioSliderStyle::FAudioSliderStyle()
	: SliderStyle(FSliderStyle::GetDefault())
	, TextBoxStyle(FAudioTextBoxStyle::GetDefault())
	, WidgetBackgroundImage(FSlateOptionalBrush())
	, SliderBackgroundColor(FStyleColors::AccentGray)
	, SliderBackgroundSize(FVector2D(28.0f, 450.0f))
	, LabelPadding(0.0f)
	, SliderBarColor(FStyleColors::Black)
	, SliderThumbColor(FStyleColors::White)
{
}

void FAudioSliderStyle::GetResources(TArray< const FSlateBrush* >& OutBrushes) const
{
	OutBrushes.Add(&WidgetBackgroundImage);
}

const FName FAudioSliderStyle::TypeName("FAudioSliderStyle");

const FAudioSliderStyle& FAudioSliderStyle::GetDefault()
{
	static FAudioSliderStyle Default;
	return Default;
}

// Audio Radial Slider Style 
FAudioRadialSliderStyle::FAudioRadialSliderStyle()
	: TextBoxStyle(FAudioTextBoxStyle::GetDefault())
	, CenterBackgroundColor(FStyleColors::Recessed)
	, SliderBarColor(FStyleColors::AccentGray)
	, SliderProgressColor(FStyleColors::White)
	, LabelPadding(0.0f)
	, DefaultSliderRadius(50.0f)
{
}

const FName FAudioRadialSliderStyle::TypeName("FAudioRadialSliderStyle");

const FAudioRadialSliderStyle& FAudioRadialSliderStyle::GetDefault()
{
	static FAudioRadialSliderStyle Default;
	return Default;
}

const FName FSampledSequenceViewerStyle::TypeName("FSampledSequenceViewerStyle");

FSampledSequenceViewerStyle::FSampledSequenceViewerStyle()
	: SequenceColor(FLinearColor::White)
	, SequenceLineThickness(1.f)
	, MajorGridLineColor(FLinearColor::Black)
	, MinorGridLineColor(FLinearColor(0.f, 0.f, 0.f, 0.5f))
	, ZeroCrossingLineColor(FLinearColor::Black)
	, ZeroCrossingLineThickness(1.f)
	, SampleMarkersSize(2.5f)
	, SequenceBackgroundColor(FLinearColor(0.02f, 0.02f, 0.02f, 1.f))
	, BackgroundBrush(*FAppStyle::GetBrush(AudioWidgetStylesSharedParams::BackgroundBrushName))
	, DesiredWidth(AudioWidgetStylesSharedParams::DefaultWidth)
	, DesiredHeight(AudioWidgetStylesSharedParams::DefaultHeight)
{
}

const FSampledSequenceViewerStyle& FSampledSequenceViewerStyle::GetDefault()
{
	static FSampledSequenceViewerStyle Default;
	return Default;
}

void FSampledSequenceViewerStyle::GetResources(TArray< const FSlateBrush* >& OutBrushes) const
{
	OutBrushes.Add(&BackgroundBrush);
}

const FName FPlayheadOverlayStyle::TypeName("FPlayheadOverlayStyle");

FPlayheadOverlayStyle::FPlayheadOverlayStyle()
	: PlayheadColor(AudioWidgetStylesSharedParams::PlayheadColor)
	, PlayheadWidth(1.0f)
	, DesiredWidth(AudioWidgetStylesSharedParams::DefaultWidth)
	, DesiredHeight(AudioWidgetStylesSharedParams::DefaultHeight)
{
}

const FPlayheadOverlayStyle& FPlayheadOverlayStyle::GetDefault()
{
	static FPlayheadOverlayStyle Default;
	return Default;
}

const FName FFixedSampleSequenceRulerStyle::TypeName("FFixedSampleSequenceRulerStyle");

FFixedSampleSequenceRulerStyle::FFixedSampleSequenceRulerStyle()
	: HandleWidth(15.f)
	, HandleColor(AudioWidgetStylesSharedParams::PlayheadColor)
	, HandleBrush()
	, TicksColor(AudioWidgetStylesSharedParams::RulerTicksColor)
	, TicksTextColor(AudioWidgetStylesSharedParams::RulerTicksColor)
	, TicksTextFont(FAppStyle::GetFontStyle("Regular"))
	, TicksTextOffset(5.f)
	, BackgroundColor(FLinearColor::Black)
	, BackgroundBrush(*FAppStyle::GetBrush(AudioWidgetStylesSharedParams::BackgroundBrushName))
	, DesiredWidth(AudioWidgetStylesSharedParams::DefaultWidth)
	, DesiredHeight(30.f)
{
}

const FFixedSampleSequenceRulerStyle& FFixedSampleSequenceRulerStyle::GetDefault()
{
	static FFixedSampleSequenceRulerStyle Default;
	return Default;
}

void FFixedSampleSequenceRulerStyle::GetResources(TArray< const FSlateBrush* >& OutBrushes) const
{
	OutBrushes.Add(&HandleBrush);
	OutBrushes.Add(&BackgroundBrush);
}

FSampledSequenceValueGridOverlayStyle::FSampledSequenceValueGridOverlayStyle()
	: GridColor(FLinearColor::Black)
	, GridThickness(1.f)
	, LabelTextColor(FLinearColor::White)
	, LabelTextFont(FAppStyle::GetFontStyle("Regular"))
	, DesiredWidth(AudioWidgetStylesSharedParams::DefaultWidth)
	, DesiredHeight(AudioWidgetStylesSharedParams::DefaultHeight)
{
}

const FSampledSequenceValueGridOverlayStyle& FSampledSequenceValueGridOverlayStyle::GetDefault()
{
	static FSampledSequenceValueGridOverlayStyle Default;
	return Default;
}

const FName FSampledSequenceValueGridOverlayStyle::TypeName("FSampledSequenceValueGridOverlayStyle");
