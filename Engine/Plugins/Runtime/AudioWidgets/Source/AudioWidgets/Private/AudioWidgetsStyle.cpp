// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioWidgetsStyle.h"

#include "AudioMeter.h"
#include "AudioOscilloscopePanelStyle.h"
#include "AudioSpectrumPlotStyle.h"
#include "AudioVectorscopePanelStyle.h"
#include "AudioWidgetsSlateTypes.h"
#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateNoResource.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Engine/Texture2D.h"
#include "Misc/Paths.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

FOnNewTimeRulerStyle		FAudioOscilloscopePanelStyle::OnNewTimeRulerStyle;
FOnNewValueGridOverlayStyle	FAudioOscilloscopePanelStyle::OnNewValueGridOverlayStyle;
FOnNewWaveformViewerStyle	FAudioOscilloscopePanelStyle::OnNewWaveformViewerStyle;

FOnNewValueGridOverlayStyle	FAudioVectorscopePanelStyle::OnNewValueGridOverlayStyle;
FOnNewVectorViewerStyle		FAudioVectorscopePanelStyle::OnNewVectorViewerStyle;

FName FAudioWidgetsStyle::StyleName("AudioWidgetsStyle");

namespace AudioWidgetsStylePrivate
{
	static FLazyName ScrubHandleBrushName = "SampledSequenceRuler.VanillaScrubHandleDown";
}

FAudioWidgetsStyle::FAudioWidgetsStyle()
	: FSlateStyleSet(StyleName)
{
	SetParentStyleName(FAppStyle::GetAppStyleSetName());
	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Runtime/AudioWidgets/Content"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	SetResources();
	SetupStyles();

	/** 
	* AudioTextBox Style
	*/
	const float LabelWidth = 64.0f;
	const float LabelHeight = 28.0f;
	const FVector2D LabelBackgroundSize = FVector2D(LabelWidth, LabelHeight);
	const FLinearColor LabelBackgroundColor = FStyleColors::Recessed.GetSpecifiedColor();
	const float LabelCornerRadius = 4.0f;
	
	Set("AudioTextBox.Style", FAudioTextBoxStyle()
		.SetBackgroundColor(FStyleColors::Recessed)
		.SetBackgroundImage(FSlateRoundedBoxBrush(FStyleColors::White, LabelCornerRadius, LabelBackgroundSize)));

	/**
	* AudioSlider Style
	*/
	// Default orientation is vertical, so width/height are relative to that
	const FVector2D ThumbImageSize = FVector2D(22.0f, 22.0f);

	const float SliderBarWidth = 10.0f;
	const float SliderBarHeight = 432.0f;
	const float SliderBackgroundOutlineWidth = 9.0f;
	const FVector2D SliderBarCapSize = FVector2D(SliderBarWidth, SliderBarWidth / 2.0f);
	const FVector2D SliderBarRectangleSize = FVector2D(SliderBarWidth, SliderBarHeight - SliderBarWidth);

	const float SliderBackgroundWidth = 28.0f;
	const float SliderBackgroundHeight = 450.0f;
	const FVector2D SliderBackgroundCapSize = FVector2D(SliderBackgroundWidth, SliderBackgroundWidth / 2.0f);
	const FVector2D SliderBackgroundRectangleSize = FVector2D(SliderBackgroundWidth, SliderBackgroundHeight - SliderBackgroundWidth);
	const FVector2D SliderBackgroundSize = FVector2D(SliderBackgroundWidth, SliderBackgroundHeight);
	
	const float LabelPadding = 3.0f;
	const FVector2D SliderDesiredWidgetSizeVertical = FVector2D(LabelBackgroundSize.X, LabelBackgroundSize.Y + LabelPadding + SliderBackgroundHeight);
	const FVector2D SliderDesiredWidgetSizeHorizontal = FVector2D(SliderBackgroundHeight + LabelBackgroundSize.X + LabelPadding, SliderBackgroundWidth);

	const FSlateColor ThumbColor = FStyleColors::White;
	FSlateRoundedBoxBrush ThumbImage = FSlateRoundedBoxBrush(ThumbColor, ThumbImageSize.X / 2.0f, ThumbImageSize);
	FSlateRoundedBoxBrush WidgetBackgroundImage = FSlateRoundedBoxBrush(FStyleColors::White, LabelCornerRadius, SliderDesiredWidgetSizeVertical);

	Set("AudioSlider.Slider", FSliderStyle()
		.SetNormalBarImage(FSlateNoResource())
		.SetHoveredBarImage(FSlateNoResource())
		.SetDisabledBarImage(FSlateNoResource())
		.SetNormalThumbImage(ThumbImage)
		.SetHoveredThumbImage(ThumbImage)
		.SetDisabledThumbImage(FSlateNoResource())
	);

	Set("AudioSlider.Style", FAudioSliderStyle()
		.SetSliderStyle(FSliderStyle()
			.SetNormalBarImage(FSlateNoResource())
			.SetHoveredBarImage(FSlateNoResource())
			.SetDisabledBarImage(FSlateNoResource())
			.SetNormalThumbImage(ThumbImage)
			.SetHoveredThumbImage(ThumbImage)
			.SetDisabledThumbImage(FSlateNoResource()))
		.SetTextBoxStyle(FAudioTextBoxStyle::GetDefault())
		.SetWidgetBackgroundImage(WidgetBackgroundImage)
		.SetSliderBackgroundColor(FStyleColors::Recessed)
		.SetSliderBarColor(FStyleColors::Black)
		.SetSliderThumbColor(ThumbColor)
		.SetSliderBackgroundSize(SliderBackgroundSize)
		.SetWidgetBackgroundColor(FStyleColors::Transparent)
		.SetLabelPadding(LabelPadding)
	);

	/**
	* AudioRadialSlider Style
	*/
	const float RadialSliderDefaultRadius = 50.0f;
	const FVector2D RadialSliderDesiredSize = FVector2D(RadialSliderDefaultRadius, RadialSliderDefaultRadius + LabelPadding + LabelHeight);
	Set("AudioRadialSlider.DesiredWidgetSize", RadialSliderDesiredSize);

	Set("AudioRadialSlider.Style", FAudioRadialSliderStyle()
		.SetTextBoxStyle(FAudioTextBoxStyle::GetDefault())
		.SetCenterBackgroundColor(FStyleColors::Recessed)
		.SetSliderProgressColor(FStyleColors::White)
		.SetSliderBarColor(FStyleColors::AccentGray)
		.SetLabelPadding(3.0f)
		.SetDefaultSliderRadius(50.0f));

	/**
	* SampledSequenceViewer Style
	*/
	Set("SampledSequenceViewer.Style", FSampledSequenceViewerStyle());

	/**
	* FixedSampledSequenceRuler Style
	*/
	Set("FixedSampledSequenceRuler.Style", FFixedSampleSequenceRulerStyle()
		.SetHandleBrush(*GetBrush(AudioWidgetsStylePrivate::ScrubHandleBrushName)));

	/**
	* Playhead Overlay Style
	*/
	Set("PlayheadOverlay.Style", FPlayheadOverlayStyle());


	/**
	* ValueGrid Style
	*/
	Set("ValueGridOverlay.Style", FSampledSequenceValueGridOverlayStyle());

	/**
	**AudioMaterialKnob Style
	*/
	Set("AudioMaterialKnob.Style", FAudioMaterialKnobStyle());

	/**
	**AudioMaterialMeter Style
	*/
	Set("AudioMaterialMeter.Style", FAudioMaterialMeterStyle());

	/**
	**AudioMaterialEnvelope Style
	*/
	Set("AudioMaterialEnvelope.Style", FAudioMaterialEnvelopeStyle());

	/**
	**AudioMaterialButton Style
	*/
	Set("AudioMaterialButton.Style", FAudioMaterialButtonStyle());

	/**
	**AudioMaterialSlider Style
	*/
	Set("AudioMaterialSlider.Style", FAudioMaterialSliderStyle()
		.SetTextBoxStyle(FAudioTextBoxStyle::GetDefault()));

	/**
	**Analyzer widget styles
	*/
	Set("AudioMeter.DefaultColorStyle", FAudioMeterDefaultColorStyle());
	Set("AudioMeter.Style", FAudioMeterStyle());
	Set("AudioOscilloscope.PanelStyle", FAudioOscilloscopePanelStyle());
	Set("AudioSpectrumPlot.Style", FAudioSpectrumPlotStyle());
	Set("AudioVectorscope.PanelStyle", FAudioVectorscopePanelStyle());
	Set("SampledSequenceVectorViewer.Style", FSampledSequenceVectorViewerStyle());
	Set("TriggerThresholdLine.Style", FTriggerThresholdLineStyle());

	Set("AudioAnalyzerRack.BackgroundColor", FLinearColor::Transparent);

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}
void FAudioWidgetsStyle::SetResources()
{
	Set(AudioWidgetsStylePrivate::ScrubHandleBrushName, new FSlateBoxBrush(LoadObject<UTexture2D>(nullptr, TEXT("/AudioWidgets/Resources/ScrubHandleDown_Clamped.ScrubHandleDown_Clamped")), FMargin(6.f / 13.f, 3.f / 12.f, 6.f / 13.f, 7.f / 12.f)));
}

void FAudioWidgetsStyle::SetupStyles()
{
	const FVector2f Icon16x16 = FVector2f(16.0f, 16.0f);
	const FVector2f Icon20x20 = FVector2f(20.0f, 20.0f);
	const FVector2f Icon30x30 = FVector2f(30.0f, 30.0f);

	Set("AudioWidgetsStyle.FadeIn", new IMAGE_BRUSH_SVG(TEXT("Slate/Icons/FadeIn_20"), Icon20x20));
	Set("AudioWidgetsStyle.FadeInLinear", new IMAGE_BRUSH_SVG(TEXT("Slate/Icons/FadeInLinear_16"), Icon16x16));
	Set("AudioWidgetsStyle.FadeInLogarithmic", new IMAGE_BRUSH_SVG(TEXT("Slate/Icons/FadeInLogarithmic_16"), Icon16x16));
	Set("AudioWidgetsStyle.FadeInExponential", new IMAGE_BRUSH_SVG(TEXT("Slate/Icons/FadeInExponential_16"), Icon16x16));
	Set("AudioWidgetsStyle.FadeInSigmoid", new IMAGE_BRUSH_SVG(TEXT("Slate/Icons/FadeInSigmoid_16"), Icon16x16));
	
	Set("AudioWidgetsStyle.FadeOut", new IMAGE_BRUSH_SVG(TEXT("Slate/Icons/FadeOut_20"), Icon20x20));
	Set("AudioWidgetsStyle.FadeOutLinear", new IMAGE_BRUSH_SVG(TEXT("Slate/Icons/FadeOutLinear_16"), Icon16x16));
	Set("AudioWidgetsStyle.FadeOutLogarithmic", new IMAGE_BRUSH_SVG(TEXT("Slate/Icons/FadeOutLogarithmic_16"), Icon16x16));
	Set("AudioWidgetsStyle.FadeOutExponential", new IMAGE_BRUSH_SVG(TEXT("Slate/Icons/FadeOutExponential_16"), Icon16x16));
	Set("AudioWidgetsStyle.FadeOutSigmoid", new IMAGE_BRUSH_SVG(TEXT("Slate/Icons/FadeOutSigmoid_16"), Icon16x16));

	Set("AudioWidgetsStyle.FadeStart", new IMAGE_BRUSH_SVG(TEXT("Slate/Icons/FadeStart_30"), Icon30x30));
	Set("AudioWidgetsStyle.FadeEnd", new IMAGE_BRUSH_SVG(TEXT("Slate/Icons/FadeEnd_30"), Icon30x30));

	Set("AudioWidgetsStyle.MarkerAdd", new IMAGE_BRUSH_SVG(TEXT("Slate/Icons/Marker_20"), Icon20x20));
	Set("AudioWidgetsStyle.MarkerDelete", new IMAGE_BRUSH_SVG(TEXT("Slate/Icons/MarkerDelete_20"), Icon20x20));

	Set("AudioWidgetsStyle.LoopStart", new IMAGE_BRUSH_SVG(TEXT("Slate/Icons/LoopStart_30"), Icon30x30));
	Set("AudioWidgetsStyle.LoopEnd", new IMAGE_BRUSH_SVG(TEXT("Slate/Icons/LoopEnd_30"), Icon30x30));

	Set("AudioWidgetsStyle.Reset", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Reset", Icon16x16));
}

FAudioWidgetsStyle::~FAudioWidgetsStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FAudioWidgetsStyle& FAudioWidgetsStyle::Get()
{
	static FAudioWidgetsStyle Inst;
	return Inst;
}
