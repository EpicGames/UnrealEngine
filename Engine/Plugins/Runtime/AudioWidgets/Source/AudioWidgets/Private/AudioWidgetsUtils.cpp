// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioWidgetsUtils.h"

FReply AudioWidgetsUtils::RouteMouseInput(WidgetMouseInputFunction InputFunction, const FPointerEvent& MouseEvent, TArrayView<TSharedPtr<SWidget>> WidgetsStack, const bool bReverse)
{
	FReply TransformationInteraction = FReply::Unhandled();

	const int32 StartIndex = bReverse ? WidgetsStack.Num() - 1 : 0;
	
	auto ExitCondition = [&bReverse, &WidgetsStack](const int32 i)
	{
		return bReverse ? i >= 0 : i < WidgetsStack.Num();
	};

	auto Increment = [&bReverse](int32& i)
	{
		i = bReverse ? i - 1 : i + 1;
	};

	for (int32 i = StartIndex; ExitCondition(i); Increment(i))
	{
		const TSharedPtr<SWidget>& LayerWidget = WidgetsStack[i];

		if (LayerWidget)
		{
			TransformationInteraction = (LayerWidget.Get()->*InputFunction)(LayerWidget->GetTickSpaceGeometry(), MouseEvent);

			if (TransformationInteraction.IsEventHandled())
			{
				return TransformationInteraction;
			}
		}
	}

	return TransformationInteraction;
}

FCursorReply AudioWidgetsUtils::RouteCursorQuery(const FPointerEvent& CursorEvent, TArrayView<const TSharedPtr<SWidget>> WidgetsStack, const bool bReverse)
{
	FCursorReply CursorReply = FCursorReply::Unhandled();

	const int32 StartIndex = bReverse ? WidgetsStack.Num() - 1 : 0;

	auto ExitCondition = [&bReverse, &WidgetsStack](const int32 i)
	{
		return bReverse ? i >= 0 : i < WidgetsStack.Num();
	};

	auto Increment = [&bReverse](int32& i)
	{
		i = bReverse ? i - 1 : i + 1;
	};

	for (int32 i = StartIndex; ExitCondition(i); i = bReverse ? i - 1 : i + 1)
	{
		const TSharedPtr<SWidget>& LayerWidget = WidgetsStack[i];

		if (LayerWidget)
		{
			CursorReply = LayerWidget->OnCursorQuery(LayerWidget->GetTickSpaceGeometry(), CursorEvent);

			if (CursorReply.IsEventHandled())
			{
				return CursorReply;
			}
		}
	}

	return CursorReply;
}