// Copyright Epic Games, Inc. All Rights Reserved.

#include "FixedSampledSequenceGridData.h"

#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericPlatformMath.h"

FFixedSampledSequenceGridData::FFixedSampledSequenceGridData(const uint32 InTotalFrames, const uint32 InSampleRateHz, const FSlateFontInfo& InTicksTimeFont, const float InGridSizePixels)
	: DisplayRange(TRange<uint32>::Inclusive(0, InTotalFrames))
	, GridSizePixels(InGridSizePixels)
	, TicksTimeFont(InTicksTimeFont)
	, GridFrameRate(InSampleRateHz, 1)
{
	UpdateGridMetrics(GridSizePixels);
}

FFixedSampledSequenceGridData::FFixedSampledSequenceGridData(const uint32 InTotalFrames, const uint32 InSampleRateHz, const float InGridSizePixels /*= 1.f*/, const FSlateFontInfo* InTicksTimeFont /*= nullptr*/)
	: TotalFrames(InTotalFrames)
	, DisplayRange(TRange<uint32>::Inclusive(0, InTotalFrames))
	, GridSizePixels(InGridSizePixels)
	, GridFrameRate(InSampleRateHz, 1)
{
	if (InTicksTimeFont)
	{
		TicksTimeFont = *InTicksTimeFont;
	}

	UpdateGridMetrics(GridSizePixels);
}

void FFixedSampledSequenceGridData::UpdateDisplayRange(const TRange<uint32> InDisplayRange)
{
	check(DisplayRange.Size<uint32>() >= 2)

	if (InDisplayRange == DisplayRange)
	{
		return;
	}

	DisplayRange = InDisplayRange;
	UpdateGridMetrics(GridSizePixels);
}

bool FFixedSampledSequenceGridData::UpdateGridMetrics(const float InGridSizePixels)
{
	if (GridSizePixels != InGridSizePixels)
	{
		GridSizePixels = InGridSizePixels;
	}

	const float WaveformDurationSeconds = TotalFrames / (double)GridFrameRate.Numerator;
	const double StartTimeSeconds = DisplayRange.GetLowerBoundValue() / (double)GridFrameRate.Numerator;
	const double DisplayedDuration = (DisplayRange.Size<double>() - 1) / GridFrameRate.Numerator; //we account for one less frame so we can distribute them evenly on the screen
	const double PixelsPerSecond = GridSizePixels / DisplayedDuration;
	double MajorGridStepSeconds = 0.0;
	float MinTicksPixelSpacing = 30.0f;


	TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	FString LongestDisplayedTimeString = FString::Printf(TEXT("%.3f"), WaveformDurationSeconds * DisplayRange.GetUpperBoundValue());
	FVector2D MaxTextSize = FontMeasureService->Measure(LongestDisplayedTimeString, TicksTimeFont);
	MinTicksPixelSpacing = MaxTextSize.X;



	if (!GridFrameRate.ComputeGridSpacing(PixelsPerSecond, MajorGridStepSeconds, GridMetrics.NumMinorGridDivisions, MinTicksPixelSpacing + 5.f))
	{
		return false;
	}

	const bool bZeroGridOffset = DisplayRange.Size<uint32>() < GridSizePixels;
	const double GridOffset = bZeroGridOffset ? 0 : FGenericPlatformMath::Fmod(StartTimeSeconds, MajorGridStepSeconds);
	GridMetrics.FirstMajorTickX = (0.f - GridOffset) * PixelsPerSecond;
	GridMetrics.MajorGridXStep = MajorGridStepSeconds * PixelsPerSecond;
	GridMetrics.StartFrame = DisplayRange.GetLowerBoundValue();
	GridMetrics.PixelsPerFrame = PixelsPerSecond / GridFrameRate.Numerator;
	GridMetrics.SampleRate = GridFrameRate.Numerator;

	if (OnGridMetricsUpdated.IsBound())
	{
		OnGridMetricsUpdated.Broadcast();
	}

	return true;
}

const FFixedSampledSequenceGridMetrics FFixedSampledSequenceGridData::GetGridMetrics() const
{
	return GridMetrics;
}

void FFixedSampledSequenceGridData::SetTicksTimeFont(const FSlateFontInfo& InNewFont)
{
	TicksTimeFont = InNewFont;
	UpdateGridMetrics(GridSizePixels);	
}

void FFixedSampledSequenceGridData::SetTicksTimeFont(const FSlateFontInfo * InNewFont)
{
	if (InNewFont)
	{
		TicksTimeFont = *InNewFont;
	}
}

const float FFixedSampledSequenceGridData::SnapPositionToClosestFrame(const float InPixelPosition) const
{
	const float DistanceFromPreviousSample = FGenericPlatformMath::Fmod(InPixelPosition, GridMetrics.PixelsPerFrame);
	const bool bSnapToNext = DistanceFromPreviousSample > GridMetrics.PixelsPerFrame / 2.f;
	float SnappedPosition = 0.f;

	if (bSnapToNext)
	{
		SnappedPosition = InPixelPosition + (GridMetrics.PixelsPerFrame - DistanceFromPreviousSample);
	}
	else
	{
		SnappedPosition = InPixelPosition - DistanceFromPreviousSample;
	}


	return SnappedPosition;
}