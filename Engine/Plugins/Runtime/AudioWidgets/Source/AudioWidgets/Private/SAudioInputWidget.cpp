// Copyright Epic Games, Inc. All Rights Reserved.


#include "SAudioInputWidget.h"
#include "AudioDefines.h"
#include "DSP/Dsp.h"

const FVector2D FAudioUnitProcessor::NormalizedLinearSliderRange = FVector2D(0.0f, 1.0f);

const float FAudioUnitProcessor::GetOutputValue(const FVector2D OutputRange, const float InSliderValue)
{
	return FMath::GetMappedRangeValueClamped(FVector2D(0.0f, 1.0f), OutputRange, InSliderValue);
};

const float FAudioUnitProcessor::GetOutputValueForText(const FVector2D OutputRange, const float InSliderValue)
{
	return GetOutputValue(OutputRange, InSliderValue);
};

const float FAudioUnitProcessor::GetSliderValue(const FVector2D OutputRange, const float OutputValue)
{
	return FMath::GetMappedRangeValueClamped(OutputRange, FVector2D(0.0f, 1.0f), OutputValue);
};

const float FAudioUnitProcessor::GetSliderValueForText(const FVector2D OutputRange, const float OutputValue)
{
	return GetSliderValue(OutputRange, OutputValue);
};

const FText FVolumeProcessor::GetUnitsText()
{
	return FText::FromString("dB");
}

const FVector2D FVolumeProcessor::GetDefaultOutputRange()
{
	if (bUseLinearOutput)
	{
		return NormalizedLinearSliderRange;
	}
	
	return FVector2D(-100.0f, 0.0f);
}

const float FVolumeProcessor::GetOutputValue(const FVector2D OutputRange, const float InSliderValue)
{
	if (bUseLinearOutput)
	{
		// Return linear given normalized linear 
		const FVector2D LinearSliderRange = FVector2D(Audio::ConvertToLinear(OutputRange.X), Audio::ConvertToLinear(OutputRange.Y));
		return FMath::GetMappedRangeValueClamped(NormalizedLinearSliderRange, LinearSliderRange, InSliderValue);
	}
	else
	{
		return GetDbValueFromSliderValue(OutputRange, InSliderValue);
	}
}

const float FVolumeProcessor::GetOutputValueForText(const FVector2D OutputRange, const float InSliderValue)
{
	return GetDbValueFromSliderValue(OutputRange, InSliderValue);
}

const float FVolumeProcessor::GetSliderValue(const FVector2D OutputRange, const float OutputValue)
{
	if (bUseLinearOutput)
	{
		// Convert from linear to normalized linear 
		const FVector2D LinearSliderRange = FVector2D(Audio::ConvertToLinear(OutputRange.X), Audio::ConvertToLinear(OutputRange.Y));
		return FMath::GetMappedRangeValueClamped(LinearSliderRange, NormalizedLinearSliderRange, OutputValue);
	}
	else
	{
		return GetSliderValueFromDb(OutputRange, OutputValue);
	}
}

const float FVolumeProcessor::GetSliderValueForText(const FVector2D OutputRange, const float OutputValue)
{
	return GetSliderValueFromDb(OutputRange, OutputValue);;
}

const float FVolumeProcessor::MinDbValue = MIN_VOLUME_DECIBELS;
const float FVolumeProcessor::MaxDbValue = 770.0f;
const FVector2D FVolumeProcessor::GetOutputRange(const FVector2D InRange)
{
	// For volume slider, OutputRange is always in dB
	if (bUseLinearOutput)
	{
		// If using linear output, assume given range is linear (not normalized though) 
		FVector2D RangeInDecibels = FVector2D(Audio::ConvertToDecibels(InRange.X), Audio::ConvertToDecibels(InRange.Y));
		return FVector2D(FMath::Max(MinDbValue, RangeInDecibels.X), FMath::Min(MaxDbValue, RangeInDecibels.Y));
	}

	return FVector2D(FMath::Max(MinDbValue, InRange.X), FMath::Min(MaxDbValue, InRange.Y));			
}

const float FVolumeProcessor::GetDbValueFromSliderValue(const FVector2D OutputRange, const float InSliderValue)
{
	// convert from linear 0-1 space to decibel OutputRange that has been converted to linear 
	const FVector2D LinearSliderRange = FVector2D(Audio::ConvertToLinear(OutputRange.X), Audio::ConvertToLinear(OutputRange.Y));
	const float LinearSliderValue = FMath::GetMappedRangeValueClamped(NormalizedLinearSliderRange, LinearSliderRange, InSliderValue);
	// convert from linear to decibels 
	float OutputValue = Audio::ConvertToDecibels(LinearSliderValue);

	return FMath::Clamp(OutputValue, OutputRange.X, OutputRange.Y);
}

const float FVolumeProcessor::GetSliderValueFromDb(const FVector2D OutputRange, const float DbValue)
{
	float ClampedValue = FMath::Clamp(DbValue, OutputRange.X, OutputRange.Y);
	// convert from decibels to linear
	float LinearSliderValue = Audio::ConvertToLinear(ClampedValue);
	// convert from decibel OutputRange that has been converted to linear to linear 0-1 space 
	const FVector2D LinearSliderRange = FVector2D(Audio::ConvertToLinear(OutputRange.X), Audio::ConvertToLinear(OutputRange.Y));
	return FMath::GetMappedRangeValueClamped(LinearSliderRange, NormalizedLinearSliderRange, LinearSliderValue);
}

const FText FFrequencyProcessor::GetUnitsText()
{
	return FText::FromString("Hz");;
}

const FVector2D FFrequencyProcessor::GetDefaultOutputRange()
{
	return FVector2D(MIN_FILTER_FREQUENCY, MAX_FILTER_FREQUENCY);
}

const float FFrequencyProcessor::GetOutputValue(const FVector2D OutputRange, const float InSliderValue)
{
	return	Audio::GetLogFrequencyClamped(InSliderValue, NormalizedLinearSliderRange, OutputRange);
}

const float FFrequencyProcessor::GetSliderValue(const FVector2D OutputRange, const float OutputValue)
{
	// edge case to avoid audio function returning negative value
	if (FMath::IsNearlyEqual(OutputValue, OutputRange.X))
	{
		return NormalizedLinearSliderRange.X;
	}
	if (FMath::IsNearlyEqual(OutputValue, OutputRange.Y))
	{
		return NormalizedLinearSliderRange.Y;
	}

	return Audio::GetLinearFrequencyClamped(OutputValue, NormalizedLinearSliderRange, OutputRange);	
}
