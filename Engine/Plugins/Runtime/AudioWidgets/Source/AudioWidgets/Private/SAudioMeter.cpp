// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAudioMeter.h"

#include "SAudioMeterWidget.h"

namespace SAudioMeterPrivate
{
	const TAttribute<TArray<FAudioMeterChannelInfo>> ConvertMeterChannelInfoAttribute(const TAttribute<TArray<FMeterChannelInfo>>& InMeterChannelInfoAttribute)
	{
		return TAttribute<TArray<FAudioMeterChannelInfo>>::Create([InMeterChannelInfoAttribute]()
			{
				const TArray<FMeterChannelInfo>& MeterChannelInfoArray = InMeterChannelInfoAttribute.Get();

				TArray<FAudioMeterChannelInfo> AudioMeterChannelInfoArray;
				AudioMeterChannelInfoArray.Reserve(MeterChannelInfoArray.Num());

				for (const FMeterChannelInfo& MeterChannelInfo : MeterChannelInfoArray)
				{
					AudioMeterChannelInfoArray.Emplace(static_cast<const FAudioMeterChannelInfo&>(MeterChannelInfo));
				}

				return AudioMeterChannelInfoArray;
			});
	}
}

void SAudioMeter::Construct(const SAudioMeter::FArguments& InArgs)
{
	SAssignNew(AudioMeterWidget, SAudioMeterWidget)
	.Orientation(InArgs._Orientation)
	.BackgroundColor(InArgs._BackgroundColor)
	.MeterBackgroundColor(InArgs._MeterBackgroundColor)
	.MeterValueColor(InArgs._MeterValueColor)
	.MeterPeakColor(InArgs._MeterPeakColor)
	.MeterScaleColor(InArgs._MeterScaleColor)
	.MeterScaleLabelColor(InArgs._MeterScaleLabelColor)
	.MeterClippingColor(InArgs._MeterClippingColor)
	.Style(InArgs._Style)
	.MeterChannelInfo(SAudioMeterPrivate::ConvertMeterChannelInfoAttribute(InArgs._MeterChannelInfo));
}

int32 SAudioMeter::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	return AudioMeterWidget->OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

float SAudioMeter::GetScaleHeight() const
{
	return AudioMeterWidget->GetScaleHeight();
}

FVector2D SAudioMeter::ComputeDesiredSize(float LayoutScaleMultiplier) const
{
	return AudioMeterWidget->ComputeDesiredSize(LayoutScaleMultiplier);
}

bool SAudioMeter::ComputeVolatility() const
{
	return AudioMeterWidget->ComputeVolatility();
}

void SAudioMeter::SetMeterChannelInfo(const TAttribute<TArray<FMeterChannelInfo>>& InMeterChannelInfo)
{
    AudioMeterWidget->SetMeterChannelInfo(SAudioMeterPrivate::ConvertMeterChannelInfoAttribute(InMeterChannelInfo));
}

TArray<FMeterChannelInfo> SAudioMeter::GetMeterChannelInfo() const
{
	return static_cast<TArray<FMeterChannelInfo>>(AudioMeterWidget->GetMeterChannelInfo());
}

void SAudioMeter::SetOrientation(EOrientation InOrientation)
{
	AudioMeterWidget->SetOrientation(InOrientation);
}

void SAudioMeter::SetBackgroundColor(FSlateColor InBackgroundColor)
{
	AudioMeterWidget->SetBackgroundColor(InBackgroundColor);
}

void SAudioMeter::SetMeterBackgroundColor(FSlateColor InMeterBackgroundColor)
{
	AudioMeterWidget->SetMeterBackgroundColor(InMeterBackgroundColor);
}

void SAudioMeter::SetMeterValueColor(FSlateColor InMeterValueColor)
{
	AudioMeterWidget->SetMeterValueColor(InMeterValueColor);
}

void SAudioMeter::SetMeterPeakColor(FSlateColor InMeterPeakColor)
{
	AudioMeterWidget->SetMeterPeakColor(InMeterPeakColor);
}

void SAudioMeter::SetMeterClippingColor(FSlateColor InMeterClippingColor)
{
	AudioMeterWidget->SetMeterClippingColor(InMeterClippingColor);
}

void SAudioMeter::SetMeterScaleColor(FSlateColor InMeterScaleColor)
{
	AudioMeterWidget->SetMeterScaleColor(InMeterScaleColor);
}

void SAudioMeter::SetMeterScaleLabelColor(FSlateColor InMeterScaleLabelColor)
{
	AudioMeterWidget->SetMeterScaleLabelColor(InMeterScaleLabelColor);
}
