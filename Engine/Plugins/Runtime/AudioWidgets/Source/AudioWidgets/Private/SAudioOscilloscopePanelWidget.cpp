// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAudioOscilloscopePanelWidget.h"

#include "AudioDefines.h"
#include "AudioWidgetsLog.h"
#include "DSP/Dsp.h"
#include "FixedSampledSequenceGridData.h"
#include "SAudioRadialSlider.h"
#include "SFixedSampledSequenceRuler.h"
#include "SFixedSampledSequenceViewer.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SOverlay.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(SAudioOscilloscopePanelWidget)

#define LOCTEXT_NAMESPACE "SOscilloscopePanelWidget"

namespace AudioOscilloscopePanelWidgetPrivate
{
	TFunction<FText(const double)> YAxisValueGeneratorLinear = [](const double InLabelValue)
	{
		const float LinearAmplitude = 1.0f - 2.0f * InLabelValue;

		const FNumberFormattingOptions Formatting = FNumberFormattingOptions().SetMaximumFractionalDigits(2);

		return FText::AsNumber(LinearAmplitude, &Formatting);
	};

	TFunction<FText(const double)> YAxisValueGeneratorDb = [](const double InLabelValue)
	{
		const float AbsAmplitude  = FMath::Abs(InLabelValue * 2.0f - 1.0f);
		const float GridLineValue = Audio::ConvertToDecibels(AbsAmplitude);

		const FNumberFormattingOptions Formatting = FNumberFormattingOptions().SetMaximumFractionalDigits(1);

		const FText GridLineValueText = (GridLineValue <= MIN_VOLUME_DECIBELS) ? FText::FromString("-inf") : FText::AsNumber(GridLineValue, &Formatting);

		return GridLineValueText;
	};

	const TFunction<FText(const double)>& GetValueGridOverlayLabelGenerator(const EYAxisLabelsUnit InValueGridOverlayDisplayUnit)
	{
		switch (InValueGridOverlayDisplayUnit)
		{
		default:
		case EYAxisLabelsUnit::Linear:
			return AudioOscilloscopePanelWidgetPrivate::YAxisValueGeneratorLinear;
		case EYAxisLabelsUnit::Db:
			return AudioOscilloscopePanelWidgetPrivate::YAxisValueGeneratorDb;
		}
	};

	ESampledSequenceDisplayUnit ConvertEXAxisLabelsUnitToESampledSequenceDisplayUnit(EXAxisLabelsUnit InXAxisLabelsUnit)
	{
		switch (InXAxisLabelsUnit)
		{
			default:
			case EXAxisLabelsUnit::Samples:
				return ESampledSequenceDisplayUnit::Samples;
			case EXAxisLabelsUnit::Seconds:
				return ESampledSequenceDisplayUnit::Seconds;
		}
	}
}

void SAudioOscilloscopePanelWidget::Construct(const FArguments& InArgs, const FFixedSampledSequenceView& InData, const int32 InNumChannels)
{
	SequenceRulerDisplayUnit      = InArgs._SequenceRulerDisplayUnit;
	ValueGridOverlayDisplayUnit   = InArgs._YAxisLabelsUnit.Get();
	ValueGridMaxDivisionParameter = InArgs._ValueGridMaxDivisionParameter;
	bHideSequenceRuler            = InArgs._HideSequenceRuler.Get();
	bHideSequenceGrid             = InArgs._HideSequenceGrid.Get();
	bHideValueGrid                = InArgs._HideValueGrid.Get();
	bHideTriggerThresholdLine     = InArgs._HideTriggerThresholdLine.Get();

	check(InArgs._PanelStyle);
	PanelStyle = InArgs._PanelStyle;

	BuildWidget(InData, InNumChannels, InArgs._PanelLayoutType);
}

void SAudioOscilloscopePanelWidget::BuildWidget(const FFixedSampledSequenceView& InData, const int32 InNumChannels, const EAudioPanelLayoutType InPanelLayoutType)
{
	NumChannels     = InNumChannels;
	PanelLayoutType = InPanelLayoutType;

	DataView = InData;

	CreateGridData(PanelStyle->TimeRulerStyle);

	CreateSequenceRuler(SequenceGridData.ToSharedRef(), PanelStyle->TimeRulerStyle);
	CreateBackground(PanelStyle->WaveViewerStyle);
	CreateValueGridOverlay(ValueGridMaxDivisionParameter, SampledSequenceValueGridOverlay::EGridDivideMode::MidSplit, ValueGridOverlayDisplayUnit, PanelStyle->ValueGridStyle);
	CreateSequenceViewer(SequenceGridData.ToSharedRef(), DataView, PanelStyle->WaveViewerStyle);
	CreateTriggerThresholdLine(PanelStyle->TriggerThresholdLineStyle);

	if (PanelLayoutType == EAudioPanelLayoutType::Advanced)
	{
		CreateOscilloscopeControls();
	}

	CreateLayout();
}

void SAudioOscilloscopePanelWidget::CreateLayout()
{
	// Oscilloscope view
	auto CreateOscilloscopeViewContainer = [this]()
	{
		TSharedRef<SOverlay> OscilloscopeViewOverlays = SNew(SOverlay);

		OscilloscopeViewOverlays->AddSlot()
		[
			BackgroundBorder.ToSharedRef()
		];

		if (!bHideValueGrid)
		{
			OscilloscopeViewOverlays->AddSlot()
			[
				ValueGridOverlay.ToSharedRef()
			];
		}

		OscilloscopeViewOverlays->AddSlot()
		[
			SequenceViewer.ToSharedRef()
		];

		OscilloscopeViewOverlays->AddSlot()
		[
			TriggerThresholdLineWidget.ToSharedRef()
		];

		TSharedPtr<SVerticalBox> OscilloscopeViewContainer = SNew(SVerticalBox);

		if (!bHideSequenceRuler)
		{
			OscilloscopeViewContainer->AddSlot().AutoHeight()
			[
				SequenceRuler.ToSharedRef()
			];
		}

		OscilloscopeViewContainer->AddSlot()
		[
			OscilloscopeViewOverlays
		];

		OscilloscopeViewContainer->SetClipping(EWidgetClipping::ClipToBounds);

		return OscilloscopeViewContainer;
	};

	if (PanelLayoutType == EAudioPanelLayoutType::Advanced)
	{
		// Oscilloscope controls
		auto CreateOscilloscopeControlsContainer = [this]()
		{
			TSharedPtr<SVerticalBox> OscilloscopeControlsContainer = SNew(SVerticalBox);

			// Channel Combobox
			OscilloscopeControlsContainer->AddSlot()
			.FillHeight(0.2f)
			.Padding(0.0f, 5.0f, 0.0f, 0.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(0.3f)
				.Padding(0.0f, 0.0f, 0.0f, 4.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Oscilloscope_Channel_Display_Label", "Channel"))
					.Justification(ETextJustify::Center)
				]
				+ SVerticalBox::Slot()
				.FillHeight(0.4f)
				.Padding(5.0f, 0.0f, 5.0f, 0.0f)
				[
					ChannelCombobox.ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.FillHeight(0.3f)
				[
					SNew(SVerticalBox)
				]
			];

			// Trigger Mode Combobox
			OscilloscopeControlsContainer->AddSlot()
			.FillHeight(0.2f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(0.3f)
				.Padding(0.0f, 0.0f, 0.0f, 4.0f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Oscilloscope_TriggerMode_Display_Label", "Trigger Mode"))
					.Justification(ETextJustify::Center)
				]
				+ SVerticalBox::Slot()
				.FillHeight(0.4f)
				.Padding(5.0f, 0.0f, 5.0f, 0.0f)
				[
					TriggerModeCombobox.ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.FillHeight(0.3f)
				[
					SNew(SVerticalBox)
				]
			];

			// Trigger Threshold knob
			OscilloscopeControlsContainer->AddSlot()
			.FillHeight(0.2f)
			.Padding(0.0f, 0.0f, 0.0f, 8.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(0.2f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Oscilloscope_TriggerThreshold_Display_Label", "Trigger Threshold"))
					.Justification(ETextJustify::Center)
				]
				+ SVerticalBox::Slot()
				.FillHeight(0.8f)
				[
					TriggerThresholdKnob.ToSharedRef()
				]
			];

			// Time Window Knob
			OscilloscopeControlsContainer->AddSlot()
			.FillHeight(0.2f)
			.Padding(0.0f, 0.0f, 0.0f, 5.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(0.2f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Oscilloscope_TimeWindow_Display_Label", "Time Window"))
					.Justification(ETextJustify::Center)
				]
				+ SVerticalBox::Slot()
				.FillHeight(0.8f)
				[
					TimeWindowKnob.ToSharedRef()
				]
			];

			// Analysis Period Knob
			OscilloscopeControlsContainer->AddSlot()
			.FillHeight(0.2f)
			.Padding(0.0f, 0.0f, 0.0f, 5.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(0.2f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Oscilloscope_AnalysisPeriod_Display_Label", "Analysis Period"))
					.Justification(ETextJustify::Center)
				]
				+ SVerticalBox::Slot()
				.FillHeight(0.8f)
				[
					AnalysisPeriodKnob.ToSharedRef()
				]
			];

			return OscilloscopeControlsContainer;
		};

		OscilloscopeViewProportion = 0.9f;

		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(OscilloscopeViewProportion)
			[
				CreateOscilloscopeViewContainer().ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f - OscilloscopeViewProportion)
			[
				CreateOscilloscopeControlsContainer().ToSharedRef()
			]
		];
	}
	else
	{
		ChildSlot
		[
			CreateOscilloscopeViewContainer().ToSharedRef()
		];
	}
}

void SAudioOscilloscopePanelWidget::CreateGridData(const FFixedSampleSequenceRulerStyle& RulerStyle)
{
	// Calculate number of frames, ensuring to span at least two frames so that FFixedSampledSequenceGridData can successfully calculate its grid metrics:
	const uint32 TotalFrames = FMath::Max(DataView.SampleData.Num() / DataView.NumDimensions, 2U);
	SequenceGridData = MakeShared<FFixedSampledSequenceGridData>(TotalFrames, DataView.SampleRate, RulerStyle.TicksTextFont, RulerStyle.DesiredWidth);
}

void SAudioOscilloscopePanelWidget::CreateSequenceRuler(TSharedRef<FFixedSampledSequenceGridData> InGridData, const FFixedSampleSequenceRulerStyle& RulerStyle)
{
	SequenceRuler = SNew(SFixedSampledSequenceRuler, InGridData)
	.DisplayUnit(AudioOscilloscopePanelWidgetPrivate::ConvertEXAxisLabelsUnitToESampledSequenceDisplayUnit(SequenceRulerDisplayUnit))
	.Style(&RulerStyle)
	.DisplayPlayhead(false)
	.Clipping(EWidgetClipping::ClipToBounds);

	SequenceGridData->OnGridMetricsUpdated.AddSP(SequenceRuler.ToSharedRef(), &SFixedSampledSequenceRuler::UpdateGridMetrics);
}

void SAudioOscilloscopePanelWidget::CreateBackground(const FSampledSequenceViewerStyle& ViewerStyle)
{
	BackgroundBorder = SNew(SBorder)
	.BorderImage(&ViewerStyle.BackgroundBrush)
	.BorderBackgroundColor(ViewerStyle.SequenceBackgroundColor);
}

void SAudioOscilloscopePanelWidget::CreateValueGridOverlay(const uint32 MaxDivisionParameter,
	const SampledSequenceValueGridOverlay::EGridDivideMode DivideMode,
	const EYAxisLabelsUnit InValueGridOverlayDisplayUnit,
	const FSampledSequenceValueGridOverlayStyle& ValueGridStyle)
{
	ValueGridOverlay = SNew(SSampledSequenceValueGridOverlay)
	.MaxDivisionParameter(MaxDivisionParameter)
	.DivideMode(DivideMode)
	.ValueGridLabelGenerator(AudioOscilloscopePanelWidgetPrivate::GetValueGridOverlayLabelGenerator(InValueGridOverlayDisplayUnit))
	.HideLabels(false)
	.Style(&ValueGridStyle)
	.SequenceDrawingParams(DrawingParams)
	.NumDimensions(DataView.NumDimensions);
}

void SAudioOscilloscopePanelWidget::CreateSequenceViewer(TSharedRef<FFixedSampledSequenceGridData> InGridData, const FFixedSampledSequenceView& InData, const FSampledSequenceViewerStyle& ViewerStyle)
{
	SequenceViewer = SNew(SFixedSampledSequenceViewer, InData.SampleData, InData.NumDimensions, InGridData)
	.Style(&ViewerStyle)
	.SequenceDrawingParams(DrawingParams)
	.HideBackground(true)
	.HideGrid(bHideSequenceGrid);

	SequenceGridData->OnGridMetricsUpdated.AddSP(SequenceViewer.ToSharedRef(), &SFixedSampledSequenceViewer::UpdateGridMetrics);
}

void SAudioOscilloscopePanelWidget::CreateTriggerThresholdLine(const FTriggerThresholdLineStyle& TriggerThresholdLineStyle)
{
	TriggerThresholdLineWidget = SNew(STriggerThresholdLineWidget)
	.Style(&TriggerThresholdLineStyle)
	.Visibility(bHideTriggerThresholdLine ? EVisibility::Hidden : EVisibility::Visible)
	.SequenceDrawingParams(DrawingParams);
}

void SAudioOscilloscopePanelWidget::CreateChannelCombobox()
{
	auto OnGenerateChannelCombobox = [this](TSharedPtr<FString> InChannel)
	{
		return SNew(STextBlock).Text(FText::FromString(*InChannel));
	};

	auto OnChannelSelectionChanged = [this](TSharedPtr<FString> InChannel, ESelectInfo::Type InSelectInfo)
	{
		if (!InChannel.IsValid())
		{
			UE_LOGF(LogAudioWidgets, Verbose, "Oscilloscope channel selection changed but the TSharedPtr that carries the channel number FString is null.");
			return;
		}

		SelectedChannelPtr = InChannel;
		ChannelCombobox->SetSelectedItem(SelectedChannelPtr);

		OnSelectedChannelChanged.Broadcast(FCString::Atoi(**InChannel.Get()));
	};

	auto GetSelectedChannel = [this]()
	{
		return SelectedChannelPtr.IsValid() ? FText::FromString(*SelectedChannelPtr) : FText::GetEmpty();
	};

	ChannelComboboxOptionsSource.Empty();
	for (uint32 Index = 0; Index < NumChannels; ++Index)
	{
		ChannelComboboxOptionsSource.Add(MakeShareable(new FString(FString::FromInt(Index + 1))));
	}

	SelectedChannelPtr = ChannelComboboxOptionsSource[0];

	ChannelCombobox = SNew(SComboBox<TSharedPtr<FString>>)
	.OptionsSource(&ChannelComboboxOptionsSource)
	.OnGenerateWidget_Lambda(OnGenerateChannelCombobox)
	.OnSelectionChanged_Lambda(OnChannelSelectionChanged)
	.InitiallySelectedItem(SelectedChannelPtr)
	[
		SNew(STextBlock).Text_Lambda(GetSelectedChannel)
	];
}

void SAudioOscilloscopePanelWidget::CreateTriggerModeCombobox()
{
	auto OnGenerateTriggerModeCombobox = [this](TSharedPtr<EAudioOscilloscopeTriggerMode> InTriggerMode)
	{
		const FString InTriggerModeEnumStr = StaticEnum<EAudioOscilloscopeTriggerMode>()->GetNameStringByValue(static_cast<uint8>(*InTriggerMode));

		return SNew(STextBlock).Text(FText::FromString(*InTriggerModeEnumStr));
	};

	auto OnTriggerModeSelectionChanged = [this](TSharedPtr<EAudioOscilloscopeTriggerMode> InTriggerMode, ESelectInfo::Type InSelectInfo)
	{
		if (!InTriggerMode.IsValid())
		{
			UE_LOGF(LogAudioWidgets, Verbose, "Oscilloscope trigger mode selection changed but the TSharedPtr that carries the trigger mode is null.");
			return;
		}

		SelectedTriggerModePtr = InTriggerMode;
		TriggerModeCombobox->SetSelectedItem(SelectedTriggerModePtr);

		const EVisibility VisibilityMode = (*SelectedTriggerModePtr.Get() != EAudioOscilloscopeTriggerMode::None) ? EVisibility::Visible : EVisibility::Hidden;
		TriggerThresholdLineWidget->SetVisibility(VisibilityMode);

		OnTriggerModeChanged.Broadcast(*InTriggerMode.Get());
	};

	auto GetSelectedTriggerMode = [this]()
	{
		const FString SelectedTriggerModeEnumStr = StaticEnum<EAudioOscilloscopeTriggerMode>()->GetNameStringByValue(static_cast<uint8>(*SelectedTriggerModePtr));

		return SelectedTriggerModePtr.IsValid() ? FText::FromString(*SelectedTriggerModeEnumStr) : FText::GetEmpty();
	};

	TriggerModeComboboxOptionsSource.Empty();
	for (int64 Index = 0; Index < StaticEnum<EAudioOscilloscopeTriggerMode>()->GetMaxEnumValue(); ++Index)
	{
		TriggerModeComboboxOptionsSource.Add(MakeShareable(new EAudioOscilloscopeTriggerMode(static_cast<EAudioOscilloscopeTriggerMode>(Index))));
	}

	SelectedTriggerModePtr = TriggerModeComboboxOptionsSource[0];

	TriggerModeCombobox = SNew(SComboBox<TSharedPtr<EAudioOscilloscopeTriggerMode>>)
	.OptionsSource(&TriggerModeComboboxOptionsSource)
	.OnGenerateWidget_Lambda(OnGenerateTriggerModeCombobox)
	.OnSelectionChanged_Lambda(OnTriggerModeSelectionChanged)
	.InitiallySelectedItem(SelectedTriggerModePtr)
	[
		SNew(STextBlock).Text_Lambda(GetSelectedTriggerMode)
	];
}

void SAudioOscilloscopePanelWidget::CreateTriggerThresholdKnob()
{
	auto OnValueChangedLambda = [this](float Value)
	{
		if (!bIsInputWidgetTransacting)
		{
		#if WITH_EDITOR
			if (GEditor)
			{
				GEditor->BeginTransaction(LOCTEXT("Oscilloscope_TriggerThreshold_Knob_Changed_Msg", "Set oscilloscope Trigger Threshold value."));
			}
		#endif
			bIsInputWidgetTransacting = true;
		}

		if (const float TriggerThresholdKnobValue = TriggerThresholdKnob->GetOutputValue(Value); 
			TriggerThresholdKnobValue != TriggerThresholdValue)
		{
			TriggerThresholdValue = TriggerThresholdKnobValue;

			TriggerThresholdLineWidget->SetTriggerThreshold(TriggerThresholdValue);
		}
	};

	auto OnRadialSliderMouseCaptureBeginLambda = [this]()
	{
		if (!bIsInputWidgetTransacting)
		{
		#if WITH_EDITOR
			if (GEditor)
			{
				GEditor->BeginTransaction(LOCTEXT("Oscilloscope_TriggerThreshold_Knob_CaptureBegin_Msg", "Set oscilloscope Trigger Threshold value."));
			}
		#endif
			bIsInputWidgetTransacting = true;
		}
	};

	auto OnRadialSliderMouseCaptureEndLambda = [this]()
	{
		if (bIsInputWidgetTransacting)
		{
		#if WITH_EDITOR
			if (GEditor)
			{
				GEditor->EndTransaction();
			}
		#endif
			bIsInputWidgetTransacting = false;

			OnTriggerThresholdChanged.Broadcast(TriggerThresholdValue);
		}
		else
		{
			UE_LOGF(LogAudioWidgets, Warning, "Unmatched oscilloscope widget transaction.");
		}
	};

	auto OnRadialSliderEnabledLambda = [this]()
	{
		return SelectedTriggerModePtr.IsValid() ? *SelectedTriggerModePtr.Get() != EAudioOscilloscopeTriggerMode::None : false;
	};

	TriggerThresholdKnob = SNew(SAudioRadialSlider)
	.OnMouseCaptureBegin_Lambda(OnRadialSliderMouseCaptureBeginLambda)
	.OnMouseCaptureEnd_Lambda(OnRadialSliderMouseCaptureEndLambda)
	.OnValueChanged_Lambda(OnValueChangedLambda)
	.SliderValue(0.5f)
	.IsEnabled_Lambda(OnRadialSliderEnabledLambda); // In normalized range, range will change below

	TriggerThresholdKnob->SetShowUnitsText(false);
	TriggerThresholdKnob->SetOutputRange(TriggerThresholdKnobOutputRange);
}

void SAudioOscilloscopePanelWidget::CreateTimeWindowKnob()
{
	auto OnValueChangedLambda = [this](float Value)
	{
		if (!bIsInputWidgetTransacting)
		{
		#if WITH_EDITOR
			if (GEditor)
			{
				GEditor->BeginTransaction(LOCTEXT("Oscilloscope_TimeWindow_Knob_Changed_Msg", "Set oscilloscope Time Window value."));
			}
		#endif
			bIsInputWidgetTransacting = true;
		}

		if (const float TimeWindowKnobValue = TimeWindowKnob->GetOutputValue(Value);
			TimeWindowKnobValue != TimeWindowValue)
		{
			TimeWindowValue = TimeWindowKnobValue;

			OnTimeWindowValueChanged.Broadcast(TimeWindowValue);
		}
	};

	auto OnRadialSliderMouseCaptureBeginLambda = [this]()
	{
		if (!bIsInputWidgetTransacting)
		{
		#if WITH_EDITOR
			if (GEditor)
			{
				GEditor->BeginTransaction(LOCTEXT("Oscilloscope_TimeWindow_Knob_CaptureBegin_Msg", "Set oscilloscope Time Window value."));
			}
		#endif
			bIsInputWidgetTransacting = true;
		}
	};

	auto OnRadialSliderMouseCaptureEndLambda = [this]()
	{
		if (bIsInputWidgetTransacting)
		{
		#if WITH_EDITOR
			if (GEditor)
			{
				GEditor->EndTransaction();
			}
		#endif
			bIsInputWidgetTransacting = false;
		}
		else
		{
			UE_LOGF(LogAudioWidgets, Warning, "Unmatched oscilloscope widget transaction.");
		}
	};

	TimeWindowKnob = SNew(SAudioRadialSlider)
	.OnMouseCaptureBegin_Lambda(OnRadialSliderMouseCaptureBeginLambda)
	.OnMouseCaptureEnd_Lambda(OnRadialSliderMouseCaptureEndLambda);

	TimeWindowKnob->SetOutputRange(TimeWindowKnobOutputRange);
	TimeWindowKnob->SetUnitsText(FText::FromString("ms"));

	TimeWindowKnob->OnValueChanged.BindLambda(OnValueChangedLambda);
}

void SAudioOscilloscopePanelWidget::CreateAnalysisPeriodKnob()
{
	auto OnValueChangedLambda = [this](float Value)
	{
		if (!bIsInputWidgetTransacting)
		{
		#if WITH_EDITOR
			if (GEditor)
			{
				GEditor->BeginTransaction(LOCTEXT("Oscilloscope_AnalysisPeriod_Knob_Changed_Msg", "Set oscilloscope Analysis Period value."));
			}
		#endif
			bIsInputWidgetTransacting = true;
		}

		if (const float AnalysisPeriodKnobValue = AnalysisPeriodKnob->GetOutputValue(Value);
			AnalysisPeriodKnobValue != AnalysisPeriodValue)
		{
			AnalysisPeriodValue = AnalysisPeriodKnobValue;

			OnAnalysisPeriodChanged.Broadcast(AnalysisPeriodValue);
		}
	};

	auto OnRadialSliderMouseCaptureBeginLambda = [this]()
	{
		if (!bIsInputWidgetTransacting)
		{
		#if WITH_EDITOR
			if (GEditor)
			{
				GEditor->BeginTransaction(LOCTEXT("Oscilloscope_AnalysisPeriod_Knob_CaptureBegin_Msg", "Set oscilloscope Analysis Period value."));
			}
		#endif
			bIsInputWidgetTransacting = true;
		}
	};

	auto OnRadialSliderMouseCaptureEndLambda = [this]()
	{
		if (bIsInputWidgetTransacting)
		{
		#if WITH_EDITOR
			if (GEditor)
			{
				GEditor->EndTransaction();
			}
		#endif
			bIsInputWidgetTransacting = false;
		}
		else
		{
			UE_LOGF(LogAudioWidgets, Warning, "Unmatched oscilloscope widget transaction.");
		}
	};

	AnalysisPeriodKnob = SNew(SAudioRadialSlider)
	.OnMouseCaptureBegin_Lambda(OnRadialSliderMouseCaptureBeginLambda)
	.OnMouseCaptureEnd_Lambda(OnRadialSliderMouseCaptureEndLambda);

	AnalysisPeriodKnob->SetOutputRange(AnalysisPeriodKnobOutputRange);
	AnalysisPeriodKnob->SetUnitsText(FText::FromString("ms"));

	AnalysisPeriodKnob->OnValueChanged.BindLambda(OnValueChangedLambda);
}

void SAudioOscilloscopePanelWidget::CreateOscilloscopeControls()
{
	CreateChannelCombobox();
	CreateTriggerModeCombobox();
	CreateTriggerThresholdKnob();
	CreateTimeWindowKnob();
	CreateAnalysisPeriodKnob();
}

void SAudioOscilloscopePanelWidget::SetSequenceRulerDisplayUnit(const EXAxisLabelsUnit InDisplayUnit)
{
	SequenceRulerDisplayUnit = InDisplayUnit;
	SequenceRuler->UpdateDisplayUnit(AudioOscilloscopePanelWidgetPrivate::ConvertEXAxisLabelsUnitToESampledSequenceDisplayUnit(SequenceRulerDisplayUnit));
}

void SAudioOscilloscopePanelWidget::SetXAxisGridVisibility(const bool InbIsVisible)
{
	SequenceViewer->SetHideGrid(!InbIsVisible);
}

void SAudioOscilloscopePanelWidget::SetValueGridOverlayDisplayUnit(const EYAxisLabelsUnit InDisplayUnit)
{
	ValueGridOverlayDisplayUnit = InDisplayUnit;
	ValueGridOverlay->SetLabelGenerator(AudioOscilloscopePanelWidgetPrivate::GetValueGridOverlayLabelGenerator(InDisplayUnit));
}

void SAudioOscilloscopePanelWidget::SetYAxisGridVisibility(const bool InbIsVisible)
{
	ValueGridOverlay->SetHideGrid(!InbIsVisible);
}

void SAudioOscilloscopePanelWidget::SetYAxisLabelsVisibility(const bool InbIsVisible)
{
	ValueGridOverlay->SetHideLabels(!InbIsVisible);
}

void SAudioOscilloscopePanelWidget::SetMaxTimeWindowMs(const float InMaxTimeWindowMs)
{
	TimeWindowKnobOutputRange.Y = InMaxTimeWindowMs;

	if (TimeWindowKnob.IsValid())
	{
		TimeWindowKnob->SetOutputRange(TimeWindowKnobOutputRange);
	}
}

void SAudioOscilloscopePanelWidget::SetChannelToAnalyze(const int32 InChannelToAnalyze)
{
	if (ChannelCombobox.IsValid())
	{
		ChannelCombobox->SetSelectedItem(MakeShareable(new FString(FString::FromInt(InChannelToAnalyze))));
	}
}

void SAudioOscilloscopePanelWidget::SetTriggerMode(const EAudioOscilloscopeTriggerMode InTriggerMode)
{
	TriggerThresholdLineWidget->SetVisibility(InTriggerMode != EAudioOscilloscopeTriggerMode::None ? EVisibility::Visible : EVisibility::Hidden);

	if (TriggerModeCombobox.IsValid())
	{
		TriggerModeCombobox->SetSelectedItem(MakeShareable(new EAudioOscilloscopeTriggerMode(InTriggerMode)));
	}
}

void SAudioOscilloscopePanelWidget::SetTriggerThreshold(const float InTriggerThreshold)
{
	TriggerThresholdLineWidget->SetTriggerThreshold(InTriggerThreshold);

	if (TriggerThresholdKnob.IsValid())
	{
		TriggerThresholdKnob->SetSliderValue(FMath::GetMappedRangeValueUnclamped(TriggerThresholdKnobOutputRange, { 0.0, 1.0 }, InTriggerThreshold));
	}
}

void SAudioOscilloscopePanelWidget::SetTimeWindow(const float InTimeWindow)
{
	if (TimeWindowKnob.IsValid())
	{
		TimeWindowKnob->SetSliderValue(FMath::GetMappedRangeValueUnclamped(TimeWindowKnobOutputRange, { 0.0, 1.0 }, InTimeWindow));
	}
}

void SAudioOscilloscopePanelWidget::SetAnalysisPeriod(const float InAnalysisPeriod)
{
	if (AnalysisPeriodKnob.IsValid())
	{
		AnalysisPeriodKnob->SetSliderValue(FMath::GetMappedRangeValueUnclamped(AnalysisPeriodKnobOutputRange, { 0.0, 1.0 }, InAnalysisPeriod));
	}
}

void SAudioOscilloscopePanelWidget::UpdateSequenceRulerStyle(const FFixedSampleSequenceRulerStyle UpdatedStyle)
{
	SequenceRuler->OnStyleUpdated(UpdatedStyle);
}

void SAudioOscilloscopePanelWidget::UpdateValueGridOverlayStyle(const FSampledSequenceValueGridOverlayStyle UpdatedStyle)
{
	ValueGridOverlay->OnStyleUpdated(UpdatedStyle);
}

void SAudioOscilloscopePanelWidget::UpdateSequenceViewerStyle(const FSampledSequenceViewerStyle UpdatedStyle)
{
	SequenceViewer->OnStyleUpdated(UpdatedStyle);
	BackgroundBorder->SetBorderBackgroundColor(UpdatedStyle.SequenceBackgroundColor);
}

void SAudioOscilloscopePanelWidget::UpdateTriggerThresholdStyle(const FTriggerThresholdLineStyle UpdatedStyle)
{
	TriggerThresholdLineWidget->OnStyleUpdated(UpdatedStyle);
}

void SAudioOscilloscopePanelWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	const float PaintedWidth = AllottedGeometry.GetLocalSize().X;

	if (PaintedWidth != CachedPixelWidth)
	{
		CachedPixelWidth = PaintedWidth;
		SequenceGridData->UpdateGridMetrics(PaintedWidth * OscilloscopeViewProportion);
	}
}

void SAudioOscilloscopePanelWidget::ReceiveSequenceView(const FFixedSampledSequenceView InData, const uint32 FirstSampleIndex)
{
	const uint32 FirstRenderedFrame = FirstSampleIndex / InData.NumDimensions;
	const uint32 NumFrames          = InData.SampleData.Num() / InData.NumDimensions;

	SequenceGridData->UpdateDisplayRange(TRange<uint32>(FirstRenderedFrame, FirstRenderedFrame + NumFrames));

	SequenceViewer->UpdateView(InData.SampleData, InData.NumDimensions);
}

#undef LOCTEXT_NAMESPACE
