// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAudioRadialSlider.h"

#include "AudioDefines.h"
#include "DSP/Dsp.h"
#include "SAudioTextBox.h"
#include "Widgets/SRadialSlider.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SOverlay.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SAudioRadialSlider)

const FVector2D SAudioRadialSlider::NormalizedLinearSliderRange = FVector2D(0.0f, 1.0f);

SAudioRadialSlider::SAudioRadialSlider()
{
}

void SAudioRadialSlider::Construct(const SAudioRadialSlider::FArguments& InArgs)
{
	Style = InArgs._Style;
	OnValueChanged = InArgs._OnValueChanged;
	OnMouseCaptureBegin = InArgs._OnMouseCaptureBegin;
	OnMouseCaptureEnd = InArgs._OnMouseCaptureEnd;
	SliderValue = InArgs._SliderValue;
	CenterBackgroundColor = InArgs._CenterBackgroundColor;
	SliderProgressColor = InArgs._SliderProgressColor;
	SliderBarColor = InArgs._SliderBarColor;
	HandStartEndRatio = InArgs._HandStartEndRatio;
	WidgetLayout = InArgs._WidgetLayout;
	DesiredSizeOverride = InArgs._DesiredSizeOverride;
	SliderCurve = InArgs._SliderCurve;
	// default linear curve from 0.0 to 1.0
	SliderCurve.GetRichCurve()->AddKey(0.0f, 0.0f);
	SliderCurve.GetRichCurve()->AddKey(1.0f, 1.0f);
	
	// Get style 
	const ISlateStyle* AudioWidgetsStyle = FSlateStyleRegistry::FindSlateStyle("AudioWidgetsStyle");
	if (AudioWidgetsStyle)
	{
		Style = &AudioWidgetsStyle->GetWidgetStyle<FAudioRadialSliderStyle>("AudioRadialSlider.Style");
	}

	// Create components
	SAssignNew(Label, SAudioTextBox)
		.Style(&Style->TextBoxStyle)
		.OnValueTextCommitted_Lambda([this](const FText& Text, ETextCommit::Type CommitType)
		{
			const float OutputValue = FCString::Atof(*Text.ToString());
			const float NewSliderValue = GetSliderValueForText(OutputValue);
			if (!FMath::IsNearlyEqual(NewSliderValue, SliderValue.Get()))
			{
				SliderValue.Set(NewSliderValue);
				RadialSlider->SetValue(NewSliderValue);
				OnValueChanged.ExecuteIfBound(NewSliderValue);
				OnMouseCaptureEnd.ExecuteIfBound();
			}
		});

	SAssignNew(RadialSlider, SRadialSlider)
		.OnValueChanged_Lambda([this](float InSliderValue)
		{
			SliderValue.Set(InSliderValue);
			OnValueChanged.ExecuteIfBound(InSliderValue);
			const float OutputValue = GetOutputValueForText(InSliderValue);
			Label->SetValueText(OutputValue);
		})
		.OnMouseCaptureBegin(OnMouseCaptureBegin)
		.OnMouseCaptureEnd(OnMouseCaptureEnd)
		.UseVerticalDrag(true)
		.ShowSliderHand(true)
		.ShowSliderHandle(false);
	RadialSlider->SetCenterBackgroundColor(CenterBackgroundColor.Get());
	RadialSlider->SetSliderProgressColor(SliderProgressColor.Get());
	RadialSlider->SetSliderBarColor(SliderBarColor.Get());
	RadialSlider->SetSliderRange(SliderCurve);

	ChildSlot
	[
		CreateLayoutWidgetSwitcher()
	];

	SetOutputRange(OutputRange);
}

void SAudioRadialSlider::SetCenterBackgroundColor(FSlateColor InColor)
{
	SetAttribute(CenterBackgroundColor, TAttribute<FSlateColor>(InColor), EInvalidateWidgetReason::Paint);
	RadialSlider->SetCenterBackgroundColor(InColor);
}

void SAudioRadialSlider::SetSliderProgressColor(FSlateColor InColor)
{
	SetAttribute(SliderProgressColor, TAttribute<FSlateColor>(InColor), EInvalidateWidgetReason::Paint);
	RadialSlider->SetSliderProgressColor(InColor);
}

void SAudioRadialSlider::SetSliderBarColor(FSlateColor InColor)
{
	SetAttribute(SliderBarColor, TAttribute<FSlateColor>(InColor), EInvalidateWidgetReason::Paint);
	RadialSlider->SetSliderBarColor(InColor);
}

void SAudioRadialSlider::SetHandStartEndRatio(const FVector2D InHandStartEndRatio)
{
	SetAttribute(HandStartEndRatio, TAttribute<FVector2D>(InHandStartEndRatio), EInvalidateWidgetReason::Paint);
	RadialSlider->SetHandStartEndRatio(InHandStartEndRatio);
}

void SAudioRadialSlider::SetWidgetLayout(EAudioRadialSliderLayout InLayout)
{
	SetAttribute(WidgetLayout, TAttribute<EAudioRadialSliderLayout>(InLayout), EInvalidateWidgetReason::Layout);
	LayoutWidgetSwitcher->SetActiveWidgetIndex(InLayout);
}

FVector2D SAudioRadialSlider::ComputeDesiredSize(float) const
{
	static const FVector2D DefaultDesiredSize = FVector2D(50.0f, 81.0f);

	if (DesiredSizeOverride.Get().IsSet())
	{
		return DesiredSizeOverride.Get().GetValue();
	}
	const float SliderRadius = Style->DefaultSliderRadius;
	const FVector2D TextBoxImageSize = Style->TextBoxStyle.BackgroundImage.ImageSize;
	return FVector2D(FMath::Max(SliderRadius, TextBoxImageSize.X), SliderRadius + TextBoxImageSize.Y + Style->LabelPadding);
}

void SAudioRadialSlider::SetDesiredSizeOverride(const FVector2D Size)
{
	SetAttribute(DesiredSizeOverride, TAttribute<TOptional<FVector2D>>(Size), EInvalidateWidgetReason::Layout);
}

TSharedRef<SWidgetSwitcher> SAudioRadialSlider::CreateLayoutWidgetSwitcher()
{
	SAssignNew(LayoutWidgetSwitcher, SWidgetSwitcher);

	float LabelVerticalPadding = Style->LabelPadding;

	LayoutWidgetSwitcher->AddSlot(EAudioRadialSliderLayout::Layout_LabelTop)
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Bottom)
		.AutoHeight()
		.Padding(0.0f, 0.0f, 0.0f, LabelVerticalPadding)
		[
			Label.ToSharedRef()
		]
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			RadialSlider.ToSharedRef()
		]
	];

	LayoutWidgetSwitcher->AddSlot(EAudioRadialSliderLayout::Layout_LabelCenter)
	[
		SNew(SOverlay)
		+SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			RadialSlider.ToSharedRef()
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			Label.ToSharedRef()
		]
	];

	LayoutWidgetSwitcher->AddSlot(EAudioRadialSliderLayout::Layout_LabelBottom)
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			RadialSlider.ToSharedRef()
		]
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Top)
		.AutoHeight()
		.Padding(0.0f, LabelVerticalPadding, 0.0f, 0.0f)
		[
			Label.ToSharedRef()
		]
	];

	LayoutWidgetSwitcher->SetActiveWidgetIndex(WidgetLayout.Get());
	return LayoutWidgetSwitcher.ToSharedRef();
}

void SAudioRadialSlider::SetSliderValue(float InSliderValue)
{
	SliderValue.Set(InSliderValue);
	const float OutputValueText = GetOutputValueForText(InSliderValue);
	Label->SetValueText(OutputValueText);
	RadialSlider->SetValue(InSliderValue);
}

const float SAudioRadialSlider::GetSliderValue(const float OutputValue)
{
	return FMath::GetMappedRangeValueClamped(OutputRange, NormalizedLinearSliderRange, OutputValue);
}

const float SAudioRadialSlider::GetOutputValue(const float InSliderValue)
{
	return FMath::GetMappedRangeValueClamped(NormalizedLinearSliderRange, OutputRange, InSliderValue);
}

void SAudioRadialSlider::SetOutputRange(const FVector2D Range)
{
	OutputRange = Range;
	// if Range.Y < Range.X, set Range.X to Range.Y
	OutputRange.X = FMath::Min(Range.X, Range.Y);

	const float OutputValue = GetOutputValue(SliderValue.Get());
	const float ClampedOutputValue = FMath::Clamp(OutputValue, OutputRange.X, OutputRange.Y);
	const float ClampedSliderValue = GetSliderValue(ClampedOutputValue);
	SetSliderValue(ClampedSliderValue);

	Label->UpdateValueTextWidth(OutputRange);
}

const float SAudioRadialSlider::GetOutputValueForText(const float InSliderValue)
{
	return GetOutputValue(InSliderValue);
}

const float SAudioRadialSlider::GetSliderValueForText(const float OutputValue)
{
	return GetSliderValue(OutputValue);
}

void SAudioRadialSlider::SetLabelBackgroundColor(FSlateColor InColor)
{
	SetAttribute(LabelBackgroundColor, TAttribute<FSlateColor>(InColor), EInvalidateWidgetReason::Paint);
	Label->SetLabelBackgroundColor(InColor);
}

void SAudioRadialSlider::SetUnitsText(const FText Units)
{
	Label->SetUnitsText(Units);
}

void SAudioRadialSlider::SetUnitsTextReadOnly(const bool bIsReadOnly)
{
	Label->SetUnitsTextReadOnly(bIsReadOnly);
}

void SAudioRadialSlider::SetValueTextReadOnly(const bool bIsReadOnly)
{
	Label->SetValueTextReadOnly(bIsReadOnly);
}

void SAudioRadialSlider::SetShowLabelOnlyOnHover(const bool bShowLabelOnlyOnHover)
{
	Label->SetShowLabelOnlyOnHover(bShowLabelOnlyOnHover);
}

void SAudioRadialSlider::SetShowUnitsText(const bool bShowUnitsText)
{
	Label->SetShowUnitsText(bShowUnitsText);
}

void SAudioRadialSlider::SetSliderThickness(const float Thickness)
{
	RadialSlider->SetThickness(FMath::Max(0.0f, Thickness));
}

// SAudioVolumeRadialSlider
const float SAudioVolumeRadialSlider::MinDbValue = MIN_VOLUME_DECIBELS;
const float SAudioVolumeRadialSlider::MaxDbValue = 770.0f;

SAudioVolumeRadialSlider::SAudioVolumeRadialSlider()
{
}

void SAudioVolumeRadialSlider::Construct(const SAudioRadialSlider::FArguments& InArgs)
{
	SAudioRadialSlider::Construct(InArgs);

	SAudioRadialSlider::SetOutputRange(FVector2D(SAudioVolumeRadialSlider::MinDbValue, 0.0f));
	Label->SetUnitsText(FText::FromString("dB"));
}

const float SAudioVolumeRadialSlider::GetDbValueFromSliderValue(const float InSliderValue)  
{
	// convert from linear 0-1 space to decibel OutputRange that has been converted to linear 
	const FVector2D LinearSliderRange = FVector2D(Audio::ConvertToLinear(OutputRange.X), Audio::ConvertToLinear(OutputRange.Y));
	const float LinearSliderValue = FMath::GetMappedRangeValueClamped(NormalizedLinearSliderRange, LinearSliderRange, InSliderValue);
	// convert from linear to decibels 
	float OutputValue = Audio::ConvertToDecibels(LinearSliderValue);
	return FMath::Clamp(OutputValue, OutputRange.X, OutputRange.Y);
}

const float SAudioVolumeRadialSlider::GetSliderValueFromDb(const float OutputValue) 
{
	float ClampedValue = FMath::Clamp(OutputValue, OutputRange.X, OutputRange.Y);
	// convert from decibels to linear
	float LinearSliderValue = Audio::ConvertToLinear(ClampedValue);
	// convert from decibel OutputRange that has been converted to linear to linear 0-1 space 
	const FVector2D LinearSliderRange = FVector2D(Audio::ConvertToLinear(OutputRange.X), Audio::ConvertToLinear(OutputRange.Y));
	return FMath::GetMappedRangeValueClamped(LinearSliderRange, NormalizedLinearSliderRange, LinearSliderValue);
}

const float SAudioVolumeRadialSlider::GetOutputValue(const float InSliderValue) 
{
	if (bUseLinearOutput)
	{
		// Return linear given normalized linear 
		const FVector2D LinearSliderRange = FVector2D(Audio::ConvertToLinear(OutputRange.X), Audio::ConvertToLinear(OutputRange.Y));
		return FMath::GetMappedRangeValueClamped(NormalizedLinearSliderRange, LinearSliderRange, InSliderValue);
	}
	else
	{
		return GetDbValueFromSliderValue(InSliderValue);
	}
}

const float SAudioVolumeRadialSlider::GetSliderValue(const float OutputValue)
{
	if (bUseLinearOutput)
	{
		// Convert from linear to normalized linear 
		const FVector2D LinearSliderRange = FVector2D(Audio::ConvertToLinear(OutputRange.X), Audio::ConvertToLinear(OutputRange.Y));
		return FMath::GetMappedRangeValueClamped(LinearSliderRange, NormalizedLinearSliderRange, OutputValue);
	}
	else
	{
		return GetSliderValueFromDb(OutputValue);
	}
}

const float SAudioVolumeRadialSlider::GetOutputValueForText(const float InSliderValue)
{
	return GetDbValueFromSliderValue(InSliderValue);
}

const float SAudioVolumeRadialSlider::GetSliderValueForText(const float OutputValue)
{
	return GetSliderValueFromDb(OutputValue);
}

void SAudioVolumeRadialSlider::SetUseLinearOutput(bool InUseLinearOutput)
{
	bUseLinearOutput = InUseLinearOutput;
}

void SAudioVolumeRadialSlider::SetOutputRange(const FVector2D Range) 
{
	// For volume slider, OutputRange is always in dB
	if (!(Range - OutputRange).IsNearlyZero())
	{
		if (bUseLinearOutput)
		{
			// If using linear output, assume given range is linear (not normalized though) 
			FVector2D RangeInDecibels = FVector2D(Audio::ConvertToDecibels(Range.X), Audio::ConvertToDecibels(Range.Y));
			SAudioRadialSlider::SetOutputRange(FVector2D(FMath::Max(MinDbValue, RangeInDecibels.X), FMath::Min(MaxDbValue, RangeInDecibels.Y)));
		}
		else
		{
			SAudioRadialSlider::SetOutputRange(FVector2D(FMath::Max(MinDbValue, Range.X), FMath::Min(MaxDbValue, Range.Y)));
		}
	}
}

// SAudioFrequencyRadialSlider
SAudioFrequencyRadialSlider::SAudioFrequencyRadialSlider()
{
}

void SAudioFrequencyRadialSlider::Construct(const SAudioRadialSlider::FArguments& InArgs)
{
	SAudioRadialSlider::Construct(InArgs);

	SAudioRadialSlider::SetOutputRange(FVector2D(MIN_FILTER_FREQUENCY, MAX_FILTER_FREQUENCY));
	Label->SetUnitsText(FText::FromString("Hz"));
}

const float SAudioFrequencyRadialSlider::GetOutputValue(const float InSliderValue)
{
	return Audio::GetLogFrequencyClamped(InSliderValue, NormalizedLinearSliderRange, OutputRange);
}

const float SAudioFrequencyRadialSlider::GetSliderValue(const float OutputValue)
{
	// edge case to avoid audio function returning negative value
	if (FMath::IsNearlyEqual(OutputValue, OutputRange.X))
	{
		return NormalizedLinearSliderRange.X;
	}
	if (FMath::IsNearlyEqual(OutputValue, OutputRange.Y))
	{
		return NormalizedLinearSliderRange.Y;
	}
	return Audio::GetLinearFrequencyClamped(OutputValue, NormalizedLinearSliderRange, OutputRange);
}
