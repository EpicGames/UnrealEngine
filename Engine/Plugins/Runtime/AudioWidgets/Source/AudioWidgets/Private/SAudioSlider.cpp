// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAudioSlider.h"
#include "AudioDefines.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "DSP/Dsp.h"
#include "Curves/CurveFloat.h"
#include "SAudioTextBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SOverlay.h"

// SAudioSliderBase
const FVector2D SAudioSliderBase::NormalizedLinearSliderRange = FVector2D(0.0f, 1.0f);

SAudioSliderBase::SAudioSliderBase()
{
}

void SAudioSliderBase::Construct(const SAudioSliderBase::FArguments& InArgs)
{
	Style = InArgs._Style;
	OnValueChanged = InArgs._OnValueChanged;
	OnValueCommitted = InArgs._OnValueCommitted;
	OnMouseCaptureBegin = InArgs._OnMouseCaptureBegin;
	OnMouseCaptureEnd = InArgs._OnMouseCaptureEnd;
	SliderValueAttribute = InArgs._SliderValue;
	SliderBackgroundColor = InArgs._SliderBackgroundColor;
	SliderBarColor = InArgs._SliderBarColor;
	SliderThumbColor = InArgs._SliderThumbColor;
	WidgetBackgroundColor = InArgs._WidgetBackgroundColor;
	Orientation = InArgs._Orientation;
	DesiredSizeOverride = InArgs._DesiredSizeOverride;

	// Get style
	const ISlateStyle* AudioWidgetsStyle = FSlateStyleRegistry::FindSlateStyle("AudioWidgetsStyle");
	if (AudioWidgetsStyle)
	{
		Style = &AudioWidgetsStyle->GetWidgetStyle<FAudioSliderStyle>("AudioSlider.Style");
	}

	// Create components
	SliderBackgroundSize = Style->SliderBackgroundSize;
	SliderBackgroundBrush = FSlateRoundedBoxBrush(SliderBarColor.Get().GetSpecifiedColor(), SliderBackgroundSize.X / 2.0f, SliderBackgroundColor.Get().GetSpecifiedColor(), SliderBackgroundSize.X / 3.0f, SliderBackgroundSize);

	SAssignNew(SliderBackgroundImage, SImage)
		.Image(&SliderBackgroundBrush);
	SAssignNew(WidgetBackgroundImage, SImage)
		.Image(&Style->WidgetBackgroundImage)
		.ColorAndOpacity(WidgetBackgroundColor);

	// Underlying slider widget
	SAssignNew(Slider, SSlider)
		.Value(SliderValueAttribute)
		.Style(&Style->SliderStyle)
		.Orientation(Orientation.Get())
		.IndentHandle(false)
		.OnValueChanged_Lambda([this](float Value)
		{
			SetSliderValue(Value);
			OnValueChanged.ExecuteIfBound(Value);
			const float OutputValue = GetOutputValueForText(Value);
			Label->SetValueText(OutputValue);
		})
		.OnMouseCaptureBegin_Lambda([this]()
		{
			OnMouseCaptureBegin.ExecuteIfBound();
		})
		.OnMouseCaptureEnd_Lambda([this]()
		{
			OnValueCommitted.ExecuteIfBound(SliderValueAttribute.Get());
			OnMouseCaptureEnd.ExecuteIfBound();
		});

	// Text label
	SAssignNew(Label, SAudioTextBox)
		.Style(&Style->TextBoxStyle)
		.OnValueTextCommitted_Lambda([this](const FText& Text, ETextCommit::Type CommitType)
		{
			const float OutputValue = FCString::Atof(*Text.ToString());
			const float NewSliderValue = GetSliderValueForText(OutputValue);
			if (!FMath::IsNearlyEqual(NewSliderValue, SliderValueAttribute.Get()))
			{
				SetSliderValue(NewSliderValue);
				Slider->SetValue(NewSliderValue);
				OnValueChanged.ExecuteIfBound(NewSliderValue);
				OnValueCommitted.ExecuteIfBound(NewSliderValue);
			}
		});

	ChildSlot
	[
		CreateWidgetLayout()
	];
}

void SAudioSliderBase::SetSliderValue(float InSliderValue)
{
	SliderValueAttribute.Set(InSliderValue);
	const float OutputValueForText = GetOutputValueForText(InSliderValue);
	Label->SetValueText(OutputValueForText);
	Slider->SetValue(InSliderValue);
}

void SAudioSliderBase::SetSliderValue(TAttribute<float> InSliderValue)
{
	SliderValueAttribute = MoveTemp(InSliderValue);
	if (Slider.IsValid())
	{
		Slider->SetValue(SliderValueAttribute);
	}

	const float Current = SliderValueAttribute.Get();
	Label->SetValueText(GetOutputValueForText(Current));
}

void SAudioSliderBase::SetOrientation(EOrientation InOrientation)
{
	SetAttribute(Orientation, TAttribute<EOrientation>(InOrientation), EInvalidateWidgetReason::Layout);
	Slider->SetOrientation(InOrientation);
	LayoutWidgetSwitcher->SetActiveWidgetIndex(Orientation.Get());

	// Set widget component orientations
	const FVector2D TextBoxImageSize = Style->TextBoxStyle.BackgroundImage.ImageSize;
	if (Orientation.Get() == Orient_Horizontal)
	{
		const FVector2D DesiredWidgetSizeHorizontal = FVector2D(SliderBackgroundSize.Y + TextBoxImageSize.X + Style->LabelPadding, SliderBackgroundSize.X);

		SliderBackgroundImage->SetDesiredSizeOverride(FVector2D(SliderBackgroundSize.Y, SliderBackgroundSize.X));
		WidgetBackgroundImage->SetDesiredSizeOverride(DesiredWidgetSizeHorizontal);
	}
	else if (Orientation.Get() == Orient_Vertical)
	{
		SliderBackgroundImage->SetDesiredSizeOverride(TOptional<FVector2D>());
		WidgetBackgroundImage->SetDesiredSizeOverride(TOptional<FVector2D>());
	}
}

FVector2D SAudioSliderBase::ComputeDesiredSize(float) const
{
	if (DesiredSizeOverride.Get().IsSet())
	{
		return DesiredSizeOverride.Get().GetValue();
	}
	const FVector2D TextBoxImageSize = Style->TextBoxStyle.BackgroundImage.ImageSize;

	const FVector2D DesiredWidgetSizeVertical = FVector2D(TextBoxImageSize.X, TextBoxImageSize.Y + Style->LabelPadding + SliderBackgroundSize.Y);
	const FVector2D DesiredWidgetSizeHorizontal = FVector2D(SliderBackgroundSize.Y + TextBoxImageSize.X + Style->LabelPadding, SliderBackgroundSize.X);

	return Orientation.Get() == Orient_Vertical ? 
		DesiredWidgetSizeVertical : DesiredWidgetSizeHorizontal;
}

void SAudioSliderBase::SetDesiredSizeOverride(const FVector2D Size)
{
	SetAttribute(DesiredSizeOverride, TAttribute<TOptional<FVector2D>>(Size), EInvalidateWidgetReason::Layout);
}

const float SAudioSliderBase::GetOutputValue(const float InSliderValue)
{
	return FMath::Clamp(InSliderValue, OutputRange.X, OutputRange.Y);
}

const float SAudioSliderBase::GetOutputValueForText(const float InSliderValue)
{
	return GetOutputValue(InSliderValue);
}

const float SAudioSliderBase::GetSliderValueForText(const float OutputValue)
{
	return GetSliderValue(OutputValue);
}

const float SAudioSliderBase::GetSliderValue(const float OutputValue)
{
	return OutputValue;
}

void SAudioSliderBase::SetSliderBackgroundColor(FSlateColor InSliderBackgroundColor)
{
	SetAttribute(SliderBackgroundColor, TAttribute<FSlateColor>(InSliderBackgroundColor), EInvalidateWidgetReason::Paint);
	SliderBackgroundBrush = FSlateRoundedBoxBrush(SliderBarColor.Get().GetSpecifiedColor(), SliderBackgroundSize.X / 2.0f, SliderBackgroundColor.Get().GetSpecifiedColor(), SliderBackgroundSize.X / 3.0f, SliderBackgroundSize);
	SliderBackgroundImage->SetImage(&SliderBackgroundBrush);
}

void SAudioSliderBase::SetSliderBarColor(FSlateColor InSliderBarColor)
{
	SetAttribute(SliderBarColor, TAttribute<FSlateColor>(InSliderBarColor), EInvalidateWidgetReason::Paint);
	SliderBackgroundBrush = FSlateRoundedBoxBrush(SliderBarColor.Get().GetSpecifiedColor(), SliderBackgroundSize.X / 2.0f, SliderBackgroundColor.Get().GetSpecifiedColor(), SliderBackgroundSize.X / 3.0f, SliderBackgroundSize);
	SliderBackgroundImage->SetImage(&SliderBackgroundBrush);
}

void SAudioSliderBase::SetSliderThumbColor(FSlateColor InSliderThumbColor)
{
	SetAttribute(SliderThumbColor, TAttribute<FSlateColor>(InSliderThumbColor), EInvalidateWidgetReason::Paint);
	Slider->SetSliderHandleColor(SliderThumbColor.Get());
}

void SAudioSliderBase::SetWidgetBackgroundColor(FSlateColor InWidgetBackgroundColor)
{
	SetAttribute(WidgetBackgroundColor, TAttribute<FSlateColor>(InWidgetBackgroundColor), EInvalidateWidgetReason::Paint);
	WidgetBackgroundImage->SetColorAndOpacity(WidgetBackgroundColor);
}

void SAudioSliderBase::SetOutputRange(const FVector2D Range)
{
	OutputRange = Range;
	// if Range.Y < Range.X, set Range.X to Range.Y
	OutputRange.X = FMath::Min(Range.X, Range.Y);

	// Cache these values to handle case where we have a bound slider value attribute
	const float OldOutputValueNormalized = SliderValueAttribute.Get();
	const float OldOutputValueMapped = GetOutputValue(OldOutputValueNormalized);

	const float NewClampedOutputValue = FMath::Clamp(OldOutputValueMapped, OutputRange.X, OutputRange.Y);
	const float NewNormalizedValue = GetSliderValue(NewClampedOutputValue);

	Label->SetValueText(GetOutputValueForText(NewNormalizedValue));
	Label->UpdateValueTextWidth(OutputRange);

	if (!FMath::IsNearlyEqual(OldOutputValueNormalized, NewNormalizedValue))
	{
		// Avoid mutating the SliderValueAttribute if we're bound to avoid severing the binding
		if (SliderValueAttribute.IsBound())
		{
			OnValueChanged.ExecuteIfBound(NewNormalizedValue);
			OnValueCommitted.ExecuteIfBound(NewNormalizedValue);
		}
		else
		{
			SliderValueAttribute.Set(NewNormalizedValue);
			if (Slider.IsValid())
			{
				Slider->SetValue(SliderValueAttribute);
			}
			OnValueChanged.ExecuteIfBound(NewNormalizedValue);
			OnValueCommitted.ExecuteIfBound(NewNormalizedValue);
		}
	}
}

void SAudioSliderBase::SetLabelBackgroundColor(FSlateColor InColor)
{
	Label->SetLabelBackgroundColor(InColor.GetSpecifiedColor());
}

void SAudioSliderBase::SetUnitsText(const FText Units)
{
	Label->SetUnitsText(Units);
}

void SAudioSliderBase::SetUnitsTextReadOnly(const bool bIsReadOnly)
{
	Label->SetUnitsTextReadOnly(bIsReadOnly);
}

void SAudioSliderBase::SetValueTextReadOnly(const bool bIsReadOnly)
{
	Label->SetValueTextReadOnly(bIsReadOnly);
}

void SAudioSliderBase::SetShowLabelOnlyOnHover(const bool bShowLabelOnlyOnHover)
{
	Label->SetShowLabelOnlyOnHover(bShowLabelOnlyOnHover);
}

void SAudioSliderBase::SetShowUnitsText(const bool bShowUnitsText)
{
	Label->SetShowUnitsText(bShowUnitsText);
}

TSharedRef<SWidgetSwitcher> SAudioSliderBase::CreateWidgetLayout()
{
	SAssignNew(LayoutWidgetSwitcher, SWidgetSwitcher);
	// Create overall layout
	// Horizontal orientation
	LayoutWidgetSwitcher->AddSlot(EOrientation::Orient_Horizontal)
	[
		SNew(SOverlay)
		// Widget background image
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			WidgetBackgroundImage.ToSharedRef()
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
			// Slider
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			[
				SNew(SOverlay)
				// Slider background image
				+ SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					SliderBackgroundImage.ToSharedRef()
				]
				+ SOverlay::Slot()
				// Actual SSlider
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.Padding(Style->LabelPadding, 0.0f)
				[
					Slider.ToSharedRef()
				]
			]
			// Text Label
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(Style->LabelPadding, 0.0f, 0.0f, 0.0f)
			[
				Label.ToSharedRef()
			]
		]	
	];
	// Vertical orientation
	LayoutWidgetSwitcher->AddSlot(EOrientation::Orient_Vertical)
	[
		SNew(SOverlay)
		// Widget background image
		+ SOverlay::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Fill)
		[
			WidgetBackgroundImage.ToSharedRef()
		]
		+ SOverlay::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Fill)
		[
			SNew(SVerticalBox)
			// Text Label
			+ SVerticalBox::Slot()
			.Padding(0.0f, 0.0f, 0.0f, Style->LabelPadding)
			.AutoHeight()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				Label.ToSharedRef()
			]
			// Slider
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Fill)
			[
				SNew(SOverlay)
				// Slider background image
				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Fill)
				[
					SliderBackgroundImage.ToSharedRef()
				]
				+ SOverlay::Slot()
				// Actual SSlider
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Fill)
				.Padding(0.0f, Style->LabelPadding)
				[
					Slider.ToSharedRef()
				]
			]
		]		
	];
	LayoutWidgetSwitcher->SetActiveWidgetIndex(Orientation.Get());
	SetOrientation(Orientation.Get());

	return LayoutWidgetSwitcher.ToSharedRef();
}

// SAudioSlider
SAudioSlider::SAudioSlider()
{
}

void SAudioSlider::Construct(const SAudioSliderBase::FArguments& InArgs)
{
	SAudioSliderBase::Construct(InArgs);
}

void SAudioSlider::SetLinToOutputCurve(const TWeakObjectPtr<const UCurveFloat> InLinToOutputCurve)
{
	LinToOutputCurve = InLinToOutputCurve;
	Label->SetValueText(GetOutputValueForText(SliderValueAttribute.Get()));
}

void SAudioSlider::SetOutputToLinCurve(const TWeakObjectPtr<const UCurveFloat> InOutputToLinCurve)
{
	OutputToLinCurve = InOutputToLinCurve;
}

const TWeakObjectPtr<const UCurveFloat> SAudioSlider::GetOutputToLinCurve()
{
	return OutputToLinCurve;
}

const TWeakObjectPtr<const UCurveFloat> SAudioSlider::GetLinToOutputCurve()
{
	return LinToOutputCurve;
}

const float SAudioSlider::GetOutputValue(const float InSliderValue)
{
	if (LinToOutputCurve.IsValid())
	{
		const float CurveOutputValue = LinToOutputCurve->GetFloatValue(InSliderValue);
		return FMath::Clamp(CurveOutputValue, OutputRange.X, OutputRange.Y);
	}
	return FMath::GetMappedRangeValueClamped(NormalizedLinearSliderRange, OutputRange, InSliderValue);
}

const float SAudioSlider::GetSliderValue(const float OutputValue)
{
	if (OutputToLinCurve.IsValid())
	{
		const float CurveSliderValue = OutputToLinCurve->GetFloatValue(OutputValue);
		return FMath::Clamp(CurveSliderValue, OutputRange.X, OutputRange.Y);
	}
	return FMath::GetMappedRangeValueClamped(OutputRange, NormalizedLinearSliderRange, OutputValue);
}

// SAudioVolumeSlider
const float SAudioVolumeSlider::MinDbValue = MIN_VOLUME_DECIBELS;
const float SAudioVolumeSlider::MaxDbValue = 770.0f;
SAudioVolumeSlider::SAudioVolumeSlider()
{
}

void SAudioVolumeSlider::Construct(const SAudioSlider::FArguments& InArgs)
{
	SAudioSliderBase::Construct(InArgs);
	SAudioSliderBase::SetOutputRange(FVector2D(-100.0f, 0.0f));
	Label->SetUnitsText(FText::FromString("dB"));
}

const float SAudioVolumeSlider::GetDbValueFromSliderValue(const float InSliderValue)
{
	// convert from linear 0-1 space to decibel OutputRange that has been converted to linear 
	const FVector2D LinearSliderRange = FVector2D(Audio::ConvertToLinear(OutputRange.X), Audio::ConvertToLinear(OutputRange.Y));
	const float LinearSliderValue = FMath::GetMappedRangeValueClamped(NormalizedLinearSliderRange, LinearSliderRange, InSliderValue);
	// convert from linear to decibels 
	float OutputValue = Audio::ConvertToDecibels(LinearSliderValue);
	return FMath::Clamp(OutputValue, OutputRange.X, OutputRange.Y);
}

const float SAudioVolumeSlider::GetSliderValueFromDb(const float DbValue)
{
	float ClampedValue = FMath::Clamp(DbValue, OutputRange.X, OutputRange.Y);
	// convert from decibels to linear
	float LinearSliderValue = Audio::ConvertToLinear(ClampedValue);
	// convert from decibel OutputRange that has been converted to linear to linear 0-1 space 
	const FVector2D LinearSliderRange = FVector2D(Audio::ConvertToLinear(OutputRange.X), Audio::ConvertToLinear(OutputRange.Y));
	return FMath::GetMappedRangeValueClamped(LinearSliderRange, NormalizedLinearSliderRange, LinearSliderValue);
}

const float SAudioVolumeSlider::GetOutputValue(const float InSliderValue)
{
	if (bUseLinearOutput)
	{
		// Return linear given normalized linear 
		const FVector2D LinearSliderRange = FVector2D(Audio::ConvertToLinear(OutputRange.X), Audio::ConvertToLinear(OutputRange.Y));
		return FMath::GetMappedRangeValueClamped(NormalizedLinearSliderRange, LinearSliderRange, InSliderValue);
	}
	else
	{
		return GetDbValueFromSliderValue(InSliderValue);
	}
}

const float SAudioVolumeSlider::GetSliderValue(const float OutputValue)
{
	if (bUseLinearOutput)
	{
		// Convert from linear to normalized linear 
		const FVector2D LinearSliderRange = FVector2D(Audio::ConvertToLinear(OutputRange.X), Audio::ConvertToLinear(OutputRange.Y));
		return FMath::GetMappedRangeValueClamped(LinearSliderRange, NormalizedLinearSliderRange, OutputValue);
	}
	else
	{
		return GetSliderValueFromDb(OutputValue);
	}
}

const float SAudioVolumeSlider::GetOutputValueForText(const float InSliderValue)
{
	return GetDbValueFromSliderValue(InSliderValue);
}

const float SAudioVolumeSlider::GetSliderValueForText(const float OutputValue)
{
	return GetSliderValueFromDb(OutputValue);
}

void SAudioVolumeSlider::SetUseLinearOutput(bool InUseLinearOutput)
{
	bUseLinearOutput = InUseLinearOutput;
}

void SAudioVolumeSlider::SetOutputRange(const FVector2D Range)
{
	// For volume slider, OutputRange is always in dB
	if (!(Range - OutputRange).IsNearlyZero())
	{
		if (bUseLinearOutput)
		{
			// If using linear output, assume given range is linear (not normalized though) 
			FVector2D RangeInDecibels = FVector2D(Audio::ConvertToDecibels(Range.X), Audio::ConvertToDecibels(Range.Y));
			SAudioSliderBase::SetOutputRange(FVector2D(FMath::Max(MinDbValue, RangeInDecibels.X), FMath::Min(MaxDbValue, RangeInDecibels.Y)));
		}
		else
		{
			SAudioSliderBase::SetOutputRange(FVector2D(FMath::Max(MinDbValue, Range.X), FMath::Min(MaxDbValue, Range.Y)));
		}
	}
}

// SAudioFrequencySlider
SAudioFrequencySlider::SAudioFrequencySlider()
{
}

void SAudioFrequencySlider::Construct(const SAudioSlider::FArguments& InArgs)
{
	SAudioSliderBase::Construct(InArgs);
	SetOutputRange(FVector2D(MIN_FILTER_FREQUENCY, MAX_FILTER_FREQUENCY));
	Label->SetUnitsText(FText::FromString("Hz"));
}

const float SAudioFrequencySlider::GetOutputValue(const float InSliderValue)
{
	return Audio::GetLogFrequencyClamped(InSliderValue, NormalizedLinearSliderRange, OutputRange);
}

const float SAudioFrequencySlider::GetSliderValue(const float OutputValue)
{
	// edge case to avoid audio function returning negative value
	if (FMath::IsNearlyEqual(OutputValue, OutputRange.X))
	{
		return NormalizedLinearSliderRange.X;
	}
	if (FMath::IsNearlyEqual(OutputValue, OutputRange.Y))
	{
		return NormalizedLinearSliderRange.Y;
	}
	return Audio::GetLinearFrequencyClamped(OutputValue, NormalizedLinearSliderRange, OutputRange);
}
