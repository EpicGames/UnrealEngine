// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAudioSpectrogram.h"

#include "AudioColorMapper.h"
#include "ConstantQ.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Layout/WidgetPath.h"
#include "SynesthesiaSpectrumAnalysis.h"

#define LOCTEXT_NAMESPACE "SAudioSpectrogram"

FName SAudioSpectrogram::ContextMenuExtensionHook("SpectrogramDisplayOptions");

void SAudioSpectrogram::Construct(const FArguments& InArgs)
{
	ViewMinFrequency = InArgs._ViewMinFrequency;
	ViewMaxFrequency = InArgs._ViewMaxFrequency;
	ColorMapMinSoundLevel = InArgs._ColorMapMinSoundLevel;
	ColorMapMaxSoundLevel = InArgs._ColorMapMaxSoundLevel;
	ColorMap = InArgs._ColorMap;
	FrequencyAxisScale = InArgs._FrequencyAxisScale;
	FrequencyAxisPixelBucketMode = InArgs._FrequencyAxisPixelBucketMode;
	Orientation = InArgs._Orientation;
	bAllowContextMenu = InArgs._AllowContextMenu;
	bFillBackground = InArgs._FillBackground;
	OnContextMenuOpening = InArgs._OnContextMenuOpening;
	OnFrequencyAxisPixelBucketModeMenuEntryClicked = InArgs._OnFrequencyAxisPixelBucketModeMenuEntryClicked;
	OnFrequencyAxisScaleMenuEntryClicked = InArgs._OnFrequencyAxisScaleMenuEntryClicked;
	OnColorMapMenuEntryClicked = InArgs._OnColorMapMenuEntryClicked;
	OnOrientationMenuEntryClicked = InArgs._OnOrientationMenuEntryClicked;

	SpectrogramViewport = MakeShareable(new FAudioSpectrogramViewport());
}

void SAudioSpectrogram::AddFrame(const FAudioSpectrogramFrameData& SpectrogramFrameData)
{
	SpectrogramViewport->AddFrame(SpectrogramFrameData);
}

void SAudioSpectrogram::AddFrame(const FSynesthesiaSpectrumResults& SpectrumResults, const EAudioSpectrumType SpectrumType, const float SampleRate)
{
	const TConstArrayView<float> SpectrumValues(SpectrumResults.SpectrumValues);
	const int32 FFTSize = 2 * (SpectrumValues.Num() - 1);
	const float BinSize = SampleRate / FFTSize;

	// Just ignoring the Nyquist sample here to get us a power of two length:
	ensure(FMath::IsPowerOfTwo(SpectrumValues.Num() - 1));
	const TConstArrayView<float> SpectrumValuesNoNyquist = SpectrumValues.LeftChop(1);

	const FAudioSpectrogramFrameData SpectrogramFrameData
	{
		.SpectrumValues = SpectrumValuesNoNyquist,
		.SpectrumType = SpectrumType,
		.MinFrequency = 0.0f,
		.MaxFrequency = (SpectrumValuesNoNyquist.Num() - 1) * BinSize,
		.bLogSpacedFreqencies = false,
	};

	AddFrame(SpectrogramFrameData);
}

void SAudioSpectrogram::AddFrame(const FConstantQResults& ConstantQResults, const float StartingFrequencyHz, const float NumBandsPerOctave, const EAudioSpectrumType SpectrumType)
{
	const int32 NumBands = ConstantQResults.SpectrumValues.Num();

	const FAudioSpectrogramFrameData SpectrogramFrameData
	{
		.SpectrumValues = ConstantQResults.SpectrumValues,
		.SpectrumType = SpectrumType,
		.MinFrequency = StartingFrequencyHz,
		.MaxFrequency = StartingFrequencyHz * FMath::Pow(2.0f, (NumBands - 1) / NumBandsPerOctave),
		.bLogSpacedFreqencies = true,
	};

	AddFrame(SpectrogramFrameData);
}

TSharedRef<const FExtensionBase> SAudioSpectrogram::AddContextMenuExtension(EExtensionHook::Position HookPosition, const TSharedPtr<FUICommandList>& CommandList, const FMenuExtensionDelegate& MenuExtensionDelegate)
{
	if (!ContextMenuExtender.IsValid())
	{
		ContextMenuExtender = MakeShared<FExtender>();
	}

	return ContextMenuExtender->AddMenuExtension(ContextMenuExtensionHook, HookPosition, CommandList, MenuExtensionDelegate);
}

void SAudioSpectrogram::RemoveContextMenuExtension(const TSharedRef<const FExtensionBase>& Extension)
{
	if (ensure(ContextMenuExtender.IsValid()))
	{
		ContextMenuExtender->RemoveExtension(Extension);
	}
}

FReply SAudioSpectrogram::OnMouseButtonDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (!HasMouseCapture())
	{
		if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			// Right clicking to summon context menu, but we'll do that on mouse-up.
			return FReply::Handled().CaptureMouse(AsShared()).SetUserFocus(AsShared(), EFocusCause::Mouse);
		}
	}

	return SCompoundWidget::OnMouseButtonDown(InMyGeometry, InMouseEvent);
}

FReply SAudioSpectrogram::OnMouseButtonUp(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	// The mouse must have been captured by mouse down before we'll process mouse ups
	if (HasMouseCapture())
	{
		if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			if (InMyGeometry.IsUnderLocation(InMouseEvent.GetScreenSpacePosition()) && bAllowContextMenu.Get())
			{
				TSharedPtr<SWidget> ContextMenu = OnContextMenuOpening.IsBound() ? OnContextMenuOpening.Execute() : BuildDefaultContextMenu();

				if (ContextMenu.IsValid())
				{
					const FWidgetPath WidgetPath = (InMouseEvent.GetEventPath() != nullptr) ? *InMouseEvent.GetEventPath() : FWidgetPath();

					FSlateApplication::Get().PushMenu(
						AsShared(),
						WidgetPath,
						ContextMenu.ToSharedRef(),
						InMouseEvent.GetScreenSpacePosition(),
						FPopupTransitionEffect::ESlideDirection::ContextMenu);
				}
			}

			return FReply::Handled().ReleaseMouseCapture();
		}
	}

	return SCompoundWidget::OnMouseButtonUp(InMyGeometry, InMouseEvent);
}

int32 SAudioSpectrogram::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// Get post DPI-scaled sizing:
	const FVector2f AbsoluteSize = AllottedGeometry.GetAbsoluteSize();

	// Round down size to whole pixels:
	const int RenderWidth = FMath::FloorToInt(AbsoluteSize.X);
	const int RenderHeight = FMath::FloorToInt(AbsoluteSize.Y);
	const FVector2f RenderSize(RenderWidth, RenderHeight);
	const FVector2f AdjustedLocalSize = TransformVector(Inverse(AllottedGeometry.GetAccumulatedRenderTransform()), RenderSize);

	// Apply rotation to viewport as required by Orientation attribute:
	const bool bRotateViewport = (Orientation.Get() == EOrientation::Orient_Horizontal);
	const FMatrix2x2f ScaleAndRotate(0.0f, -RenderSize.Y / RenderSize.X, RenderSize.X / RenderSize.Y, 0.0f); // Scale and rotate to swap X and Y.
	const FSlateRenderTransform ViewportRenderTransform = (bRotateViewport) ? FSlateRenderTransform(ScaleAndRotate) : FSlateRenderTransform();
	const FVector2f ViewportRenderTransformPivot(0.5f);

	// Create a child geometry using adjusted size and possible render rotation:
	const FGeometry ChildGeometry = AllottedGeometry.MakeChild(AdjustedLocalSize, FSlateLayoutTransform(), ViewportRenderTransform, ViewportRenderTransformPivot);
	
	// Create spectrogram render params, setting viewport history size to exact pixel size:
	const FAudioSpectrogramViewportRenderParams RenderParams
	{
		.NumRows = FMath::Max((bRotateViewport) ? RenderWidth : RenderHeight, 1),
		.NumPixelsPerRow = FMath::Max((bRotateViewport) ? RenderHeight : RenderWidth, 2),
		.ViewMinFrequency = ViewMinFrequency.Get(),
		.ViewMaxFrequency = ViewMaxFrequency.Get(),
		.ColorMapMinSoundLevel = ColorMapMinSoundLevel.Get(),
		.ColorMapMaxSoundLevel = ColorMapMaxSoundLevel.Get(),
		.ColorMap = ColorMap.Get(),
		.FrequencyAxisScale = FrequencyAxisScale.Get(),
		.FrequencyAxisPixelBucketMode = FrequencyAxisPixelBucketMode.Get(),
	};
	SpectrogramViewport->SetRenderParams(RenderParams);

	if (bFillBackground.Get())
	{
		// Fill allotted geometry with colormap minimum value:
		const FSlateBrush BackgroundBrush;
		const FLinearColor BoxColor = FAudioColorMapper(0.0f, 1.0f, RenderParams.ColorMap).GetColorFromValue(0.0f);
		FSlateDrawElement::MakeBox(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), &BackgroundBrush, ESlateDrawEffect::None, BoxColor);
		LayerId++;
	}

	// Create the viewport using child geometry:
	FSlateDrawElement::MakeViewport(OutDrawElements, LayerId, ChildGeometry.ToPaintGeometry(), SpectrogramViewport, ESlateDrawEffect::NoGamma);

	return LayerId + 1;
}

TSharedRef<SWidget> SAudioSpectrogram::BuildDefaultContextMenu()
{
	constexpr bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr, ContextMenuExtender);

	MenuBuilder.BeginSection(ContextMenuExtensionHook, LOCTEXT("DisplayOptions", "Display Options"));

	if (OnFrequencyAxisPixelBucketModeMenuEntryClicked.IsBound() || !FrequencyAxisPixelBucketMode.IsBound())
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("FrequencyAxisPixelBucketMode", "Pixel Plot Mode"),
			FText(),
			FNewMenuDelegate::CreateSP(this, &SAudioSpectrogram::BuildFrequencyAxisPixelBucketModeSubMenu));
	}

	if (OnFrequencyAxisScaleMenuEntryClicked.IsBound() || !FrequencyAxisScale.IsBound())
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("FrequencyAxisScale", "Frequency Scale"),
			FText(),
			FNewMenuDelegate::CreateSP(this, &SAudioSpectrogram::BuildFrequencyAxisScaleSubMenu));
	}

	if (OnColorMapMenuEntryClicked.IsBound() || !ColorMap.IsBound())
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("ColorMap", "Color Map"),
			FText(),
			FNewMenuDelegate::CreateSP(this, &SAudioSpectrogram::BuildColorMapSubMenu));
	}

	if (OnOrientationMenuEntryClicked.IsBound() || !Orientation.IsBound())
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("Orientation", "Orientation"),
			FText(),
			FNewMenuDelegate::CreateSP(this, &SAudioSpectrogram::BuildOrientationSubMenu));
	}

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SAudioSpectrogram::BuildColorMapSubMenu(FMenuBuilder& SubMenu)
{
	const UEnum* EnumClass = StaticEnum<EAudioColorGradient>();
	const int32 NumEnumValues = EnumClass->NumEnums() - 1; // Exclude 'MAX' enum value.
	for (int32 Index = 0; Index < NumEnumValues; Index++)
	{
		const auto EnumValue = static_cast<EAudioColorGradient>(EnumClass->GetValueByIndex(Index));

		SubMenu.AddMenuEntry(
			EnumClass->GetDisplayNameTextByIndex(Index),
#if WITH_EDITOR
			EnumClass->GetToolTipTextByIndex(Index),
#else
			FText(),
#endif
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSPLambda(this, [this, EnumValue]()
					{
						if (!ColorMap.IsBound())
						{
							ColorMap = EnumValue;
						}

						OnColorMapMenuEntryClicked.ExecuteIfBound(EnumValue);
					}),
				FCanExecuteAction(),
				FIsActionChecked::CreateSPLambda(this, [this, EnumValue]() { return (ColorMap.Get() == EnumValue); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}
}

void SAudioSpectrogram::BuildFrequencyAxisScaleSubMenu(FMenuBuilder& SubMenu)
{
	const UEnum* EnumClass = StaticEnum<EAudioSpectrogramFrequencyAxisScale>();
	const int32 NumEnumValues = EnumClass->NumEnums() - 1; // Exclude 'MAX' enum value.
	for (int32 Index = 0; Index < NumEnumValues; Index++)
	{
		const auto EnumValue = static_cast<EAudioSpectrogramFrequencyAxisScale>(EnumClass->GetValueByIndex(Index));

		SubMenu.AddMenuEntry(
			EnumClass->GetDisplayNameTextByIndex(Index),
#if WITH_EDITOR
			EnumClass->GetToolTipTextByIndex(Index),
#else
			FText(),
#endif
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSPLambda(this, [this, EnumValue]()
					{
						if (!FrequencyAxisScale.IsBound())
						{
							FrequencyAxisScale = EnumValue;
						}

						OnFrequencyAxisScaleMenuEntryClicked.ExecuteIfBound(EnumValue);
					}),
				FCanExecuteAction(),
				FIsActionChecked::CreateSPLambda(this, [this, EnumValue]() { return (FrequencyAxisScale.Get() == EnumValue); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}
}

void SAudioSpectrogram::BuildFrequencyAxisPixelBucketModeSubMenu(FMenuBuilder& SubMenu)
{
	const UEnum* EnumClass = StaticEnum<EAudioSpectrogramFrequencyAxisPixelBucketMode>();
	const int32 NumEnumValues = EnumClass->NumEnums() - 1; // Exclude 'MAX' enum value.
	for (int32 Index = 0; Index < NumEnumValues; Index++)
	{
		const auto EnumValue = static_cast<EAudioSpectrogramFrequencyAxisPixelBucketMode>(EnumClass->GetValueByIndex(Index));

		SubMenu.AddMenuEntry(
			EnumClass->GetDisplayNameTextByIndex(Index),
#if WITH_EDITOR
			EnumClass->GetToolTipTextByIndex(Index),
#else
			FText(),
#endif
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSPLambda(this, [this, EnumValue]()
					{
						if (!FrequencyAxisPixelBucketMode.IsBound())
						{
							FrequencyAxisPixelBucketMode = EnumValue;
						}

						OnFrequencyAxisPixelBucketModeMenuEntryClicked.ExecuteIfBound(EnumValue);
					}),
				FCanExecuteAction(),
				FIsActionChecked::CreateSPLambda(this, [this, EnumValue]() { return (FrequencyAxisPixelBucketMode.Get() == EnumValue); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}
}

void SAudioSpectrogram::BuildOrientationSubMenu(FMenuBuilder& SubMenu)
{
	const UEnum* EnumClass = StaticEnum<EOrientation>();
	const int32 NumEnumValues = EnumClass->NumEnums() - 1; // Exclude 'MAX' enum value.
	for (int32 Index = 0; Index < NumEnumValues; Index++)
	{
		const auto EnumValue = static_cast<EOrientation>(EnumClass->GetValueByIndex(Index));

		SubMenu.AddMenuEntry(
			EnumClass->GetDisplayNameTextByIndex(Index),
#if WITH_EDITOR
			EnumClass->GetToolTipTextByIndex(Index),
#else
			FText(),
#endif
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSPLambda(this, [this, EnumValue]()
					{
						if (!Orientation.IsBound())
						{
							Orientation = EnumValue;
						}

						OnOrientationMenuEntryClicked.ExecuteIfBound(EnumValue);
					}),
				FCanExecuteAction(),
				FIsActionChecked::CreateSPLambda(this, [this, EnumValue]() { return (Orientation.Get() == EnumValue); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}
}

#undef LOCTEXT_NAMESPACE
