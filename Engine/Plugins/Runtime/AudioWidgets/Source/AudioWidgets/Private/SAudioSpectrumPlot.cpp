// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAudioSpectrumPlot.h"

#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SlateOptMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SAudioSpectrumPlot)

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

#define LOCTEXT_NAMESPACE "SAudioSpectrumPlot"

/**
 * Helper class for drawing grid lines with text labels. Includes logic to avoid drawing overlapping labels if the grid lines are close together.
 */
class FAudioSpectrumPlotGridAndLabelDrawingHelper
{
public:
	FAudioSpectrumPlotGridAndLabelDrawingHelper(const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, const FAudioSpectrumPlotScaleInfo& InScaleInfo);

	void DrawSoundLevelGridLines(const int32 LayerId, TConstArrayView<float> GridLineSoundLevels, const FLinearColor& LineColor) const;
	void DrawFrequencyGridLines(const int32 LayerId, TConstArrayView<float> GridLineFrequencies, const FLinearColor& LineColor) const;

	void DrawCrosshairWithLabels(const int32 LayerId, float Frequency, float SoundLevel, const FSlateFontInfo& Font, const FLinearColor& TextColor, const FLinearColor& LineColor);

	void DrawSoundLevelAxisLabels(const int32 LayerId, TConstArrayView<float> GridLineSoundLevels, const FSlateFontInfo& Font, const FLinearColor& TextColor);
	void DrawFrequencyAxisLabels(const int32 LayerId, TConstArrayView<float> GridLineFrequencies, const FSlateFontInfo& Font, const FLinearColor& TextColor);

	bool HasDrawnLabels() const { return !DrawnLabelRects.IsEmpty(); }

private:
	struct FSoundLevelFormattingOptions
	{
		int32 NumFractionalDigits = 0;
		bool bIncludeUnits = false;
	};

	struct FFreqFormattingOptions
	{
		bool bAlwaysDisplayMaximumFractionalDigits = false;
		bool bIncludeUnits = false;
	};

	static FString FormatSoundLevelString(const float SoundLevel, const FSoundLevelFormattingOptions& SoundLevelFormattingOptions);
	static FString FormatFreqString(const float Freq, const FFreqFormattingOptions& FreqFormattingOptions);

	void DrawVerticalArrowhead(int32 LayerId, FVector2f TipPosition, FVector2f Size, const FLinearColor& LineColor) const;
	void DrawLabelIfNoOverlap(const int32 LayerId, const float LabelLeft, const float LabelTop, const FVector2f& LabelDrawSize, const FString LabelText, const FSlateFontInfo& Font, const FLinearColor& TextColor);

	// Tweak the label Rect bounds to give space where it's needed for readability, while not wasting space where it's not needed.
	FSlateRect GetModifiedLabelRect(const FSlateRect& LabelRect) const;

	bool IsOverlappingPreviouslyDrawnLabel(const FSlateRect& LabelRect) const;

	const FGeometry& AllottedGeometry;
	FSlateWindowElementList& ElementList;
	const FAudioSpectrumPlotScaleInfo& ScaleInfo;
	const FSlateRect LocalBackgroundRect;
	const TSharedRef<FSlateFontMeasure> FontMeasureService;
	FVector2f SpaceDrawSize; // Cached draw size of a space character.
	TArray<FSlateRect> DrawnLabelRects; // Keep track of where text labels have been drawn.
};

FAudioSpectrumPlotGridAndLabelDrawingHelper::FAudioSpectrumPlotGridAndLabelDrawingHelper(const FGeometry& InAllottedGeometry, FSlateWindowElementList& OutDrawElements, const FAudioSpectrumPlotScaleInfo& InScaleInfo)
	: AllottedGeometry(InAllottedGeometry)
	, ElementList(OutDrawElements)
	, ScaleInfo(InScaleInfo)
	, LocalBackgroundRect(FVector2f::ZeroVector, InAllottedGeometry.GetLocalSize())
	, FontMeasureService(FSlateApplication::Get().GetRenderer()->GetFontMeasureService())
	, SpaceDrawSize(FVector2f::ZeroVector)
{
	//
}

void FAudioSpectrumPlotGridAndLabelDrawingHelper::DrawSoundLevelGridLines(const int32 LayerId, TConstArrayView<float> GridLineSoundLevels, const FLinearColor& LineColor) const
{
	TArray<FVector2f> LinePoints;
	LinePoints.SetNum(2);

	for (float SoundLevel : GridLineSoundLevels)
	{
		// Draw horizontal grid line:
		const float GridLineLocalY = ScaleInfo.SoundLevelToLocalY(SoundLevel);
		LinePoints[0] = { LocalBackgroundRect.Left, GridLineLocalY };
		LinePoints[1] = { LocalBackgroundRect.Right, GridLineLocalY };
		FSlateDrawElement::MakeLines(ElementList, LayerId, AllottedGeometry.ToPaintGeometry(), LinePoints, ESlateDrawEffect::None, LineColor);
	}
}

void FAudioSpectrumPlotGridAndLabelDrawingHelper::DrawFrequencyGridLines(const int32 LayerId, TConstArrayView<float> GridLineFrequencies, const FLinearColor& LineColor) const
{
	TArray<FVector2f> LinePoints;
	LinePoints.SetNum(2);

	for (const float Freq : GridLineFrequencies)
	{
		// Draw vertical grid line:
		const float GridLineLocalX = ScaleInfo.FrequencyToLocalX(Freq);
		LinePoints[0] = { GridLineLocalX, LocalBackgroundRect.Top };
		LinePoints[1] = { GridLineLocalX, LocalBackgroundRect.Bottom };
		FSlateDrawElement::MakeLines(ElementList, LayerId, AllottedGeometry.ToPaintGeometry(), LinePoints, ESlateDrawEffect::None, LineColor);
	}
}

void FAudioSpectrumPlotGridAndLabelDrawingHelper::DrawCrosshairWithLabels(const int32 LayerId, float Frequency, float SoundLevel, const FSlateFontInfo& Font, const FLinearColor& TextColor, const FLinearColor& LineColor)
{
	SpaceDrawSize = FontMeasureService->Measure(TEXT(" "), Font);

	const float CrosshairPosX = ScaleInfo.FrequencyToLocalX(Frequency);
	const float CrosshairPosY = ScaleInfo.SoundLevelToLocalY(SoundLevel);
	const bool bIsHorizontalCrosshairWithinVisibleRange = (CrosshairPosY >= LocalBackgroundRect.Top && CrosshairPosY <= LocalBackgroundRect.Bottom);
	const bool bIsVerticalCrosshairWithinVisibleRange = (CrosshairPosX >= LocalBackgroundRect.Left && CrosshairPosX <= LocalBackgroundRect.Right);

	if (!bIsVerticalCrosshairWithinVisibleRange)
	{
		return;
	}

	// If the horizontal crosshair is not within visible range then we shall be drawing arrowheads at the top or bottom to signify this.
	constexpr float ArrowheadHeight = 4.0f;
	constexpr float ArrowheadWidth = 6.0f;
	const float ArrowTipPosY = FMath::Clamp(CrosshairPosY, LocalBackgroundRect.Top, LocalBackgroundRect.Bottom);
	const float ArrowDirection = FMath::Sign(CrosshairPosY - ArrowTipPosY);

	TArray<FVector2f> VerticalCrosshairLinePoints;
	VerticalCrosshairLinePoints.Reserve(2);

	const FString FreqString = FormatFreqString(Frequency, { .bAlwaysDisplayMaximumFractionalDigits = true, .bIncludeUnits = true });
	const FVector2f FreqLabelDrawSize = FontMeasureService->Measure(FreqString, Font);
	const float FreqLabelLeft = FMath::Clamp(CrosshairPosX - 0.5f * FreqLabelDrawSize.X, LocalBackgroundRect.Left, LocalBackgroundRect.Right - FreqLabelDrawSize.X);
	const float TopLabelBottomSide = LocalBackgroundRect.Top + FreqLabelDrawSize.Y;
	const float BottomLabelTopSide = LocalBackgroundRect.Bottom - FreqLabelDrawSize.Y;

	if (CrosshairPosY >= TopLabelBottomSide)
	{
		// Draw label at the top:
		DrawLabelIfNoOverlap(LayerId, FreqLabelLeft, LocalBackgroundRect.Top, FreqLabelDrawSize, FreqString, Font, TextColor);

		// Start the vertical crosshair line below the top label:
		VerticalCrosshairLinePoints.Add({ CrosshairPosX, TopLabelBottomSide });
	}
	else
	{
		// Don't draw label at the top, as either the horizontal crosshair line or an arrowhead will be drawn at the top.

		// Start the vertical crosshair line at the very top:
		VerticalCrosshairLinePoints.Add({ CrosshairPosX, LocalBackgroundRect.Top });
	}

	if (CrosshairPosY <= BottomLabelTopSide)
	{
		// Draw label at the bottom:
		DrawLabelIfNoOverlap(LayerId, FreqLabelLeft, BottomLabelTopSide, FreqLabelDrawSize, FreqString, Font, TextColor);

		// End the crosshair line above the bottom label:
		VerticalCrosshairLinePoints.Add({ CrosshairPosX, BottomLabelTopSide });
	}
	else
	{
		// Don't draw label at the bottom, as either the horizontal crosshair line or an arrowhead will be drawn at the bottom.

		// End the vertical crosshair line at the very bottom:
		VerticalCrosshairLinePoints.Add({ CrosshairPosX, LocalBackgroundRect.Bottom });
	}

	// Draw the vertical crosshair:
	FSlateDrawElement::MakeLines(ElementList, LayerId, AllottedGeometry.ToPaintGeometry(), VerticalCrosshairLinePoints, ESlateDrawEffect::None, LineColor);

	if (!bIsHorizontalCrosshairWithinVisibleRange)
	{
		// Draw an arrowhead at the top or bottom of the vertical crosshair:
		DrawVerticalArrowhead(LayerId, { CrosshairPosX, ArrowTipPosY }, { ArrowheadWidth, ArrowDirection * ArrowheadHeight }, LineColor);
	}



	TArray<FVector2f> HorizontalCrosshairLinePoints;
	HorizontalCrosshairLinePoints.Reserve(2);

	const FString SoundLevelString = FormatSoundLevelString(SoundLevel, { .NumFractionalDigits = 1, .bIncludeUnits = true });
	const FVector2f SoundLevelLabelDrawSize = FontMeasureService->Measure(SoundLevelString, Font);
	const float SoundLevelLabelTop = FMath::Clamp(CrosshairPosY - 0.5f * SoundLevelLabelDrawSize.Y, LocalBackgroundRect.Top, LocalBackgroundRect.Bottom - SoundLevelLabelDrawSize.Y);
	const float LeftLabelRightSide = LocalBackgroundRect.Left + SoundLevelLabelDrawSize.X;
	const float RightLabelLeftSide = LocalBackgroundRect.Right - SoundLevelLabelDrawSize.X;
	const float LeftLabelRightSidePadded = LeftLabelRightSide + SpaceDrawSize.X;
	const float RightLabelLeftSidePadded = RightLabelLeftSide - SpaceDrawSize.X;

	if (bIsHorizontalCrosshairWithinVisibleRange)
	{
		if (CrosshairPosX > LeftLabelRightSide)
		{
			// Draw label at the left end of the horizontal crosshair line:
			DrawLabelIfNoOverlap(LayerId, LocalBackgroundRect.Left, SoundLevelLabelTop, SoundLevelLabelDrawSize, SoundLevelString, Font, TextColor);

			// Start the horizontal crosshair line to the right of the left side label:
			HorizontalCrosshairLinePoints.Add({ FMath::Min(LeftLabelRightSidePadded, CrosshairPosX), CrosshairPosY });
		}
		else
		{
			// Start the horizontal crosshair line at the furthest left:
			HorizontalCrosshairLinePoints.Add({ LocalBackgroundRect.Left, CrosshairPosY });
		}

		if (CrosshairPosX < RightLabelLeftSide)
		{
			// Draw label at the right end of the horizontal crosshair line:
			DrawLabelIfNoOverlap(LayerId, RightLabelLeftSide, SoundLevelLabelTop, SoundLevelLabelDrawSize, SoundLevelString, Font, TextColor);

			// End the horizontal crosshair line to the left of the right side label:
			HorizontalCrosshairLinePoints.Add({ FMath::Max(RightLabelLeftSidePadded, CrosshairPosX), CrosshairPosY });
		}
		else
		{
			// End the horizontal crosshair line at the furthest right:
			HorizontalCrosshairLinePoints.Add({ LocalBackgroundRect.Right, CrosshairPosY });
		}

		// Draw the horizontal crosshair:
		FSlateDrawElement::MakeLines(ElementList, LayerId, AllottedGeometry.ToPaintGeometry(), HorizontalCrosshairLinePoints, ESlateDrawEffect::None, LineColor);
	}
	else
	{
		const float ArrowTailPosY = SoundLevelLabelTop + 0.5f * SoundLevelLabelDrawSize.Y;

		if (CrosshairPosX > LeftLabelRightSidePadded + 1.5f * ArrowheadWidth)
		{
			// Draw label in the left corner:
			DrawLabelIfNoOverlap(LayerId, LocalBackgroundRect.Left, SoundLevelLabelTop, SoundLevelLabelDrawSize, SoundLevelString, Font, TextColor);

			// Horizontal crosshair is out of visible range, draw a vertical arrow to the right of the sound level label to signify this:
			const float ArrowPosX = LeftLabelRightSidePadded + 0.5f * ArrowheadWidth;
			DrawVerticalArrowhead(LayerId, { ArrowPosX, ArrowTipPosY }, { ArrowheadWidth, ArrowDirection * ArrowheadHeight }, LineColor);

			TArray<FVector2f> ArrowShaftPoints;
			ArrowShaftPoints.Reserve(2);
			ArrowShaftPoints.Add({ ArrowPosX, ArrowTailPosY });
			ArrowShaftPoints.Add({ ArrowPosX, ArrowTipPosY });
			FSlateDrawElement::MakeLines(ElementList, LayerId, AllottedGeometry.ToPaintGeometry(), ArrowShaftPoints, ESlateDrawEffect::None, LineColor);
		}

		if (CrosshairPosX < RightLabelLeftSidePadded - 1.5f * ArrowheadWidth)
		{
			// Draw label in the right corner:
			DrawLabelIfNoOverlap(LayerId, RightLabelLeftSide, SoundLevelLabelTop, SoundLevelLabelDrawSize, SoundLevelString, Font, TextColor);

			// Horizontal crosshair is out of visible range, draw a vertical arrow to the left of the sound level label to signify this:
			const float ArrowPosX = RightLabelLeftSidePadded - 0.5f * ArrowheadWidth;
			DrawVerticalArrowhead(LayerId, { ArrowPosX, ArrowTipPosY }, { ArrowheadWidth, ArrowDirection * ArrowheadHeight }, LineColor);

			TArray<FVector2f> ArrowShaftPoints;
			ArrowShaftPoints.Reserve(2);
			ArrowShaftPoints.Add({ ArrowPosX, ArrowTailPosY });
			ArrowShaftPoints.Add({ ArrowPosX, ArrowTipPosY });
			FSlateDrawElement::MakeLines(ElementList, LayerId, AllottedGeometry.ToPaintGeometry(), ArrowShaftPoints, ESlateDrawEffect::None, LineColor);
		}
	}
}

void FAudioSpectrumPlotGridAndLabelDrawingHelper::DrawSoundLevelAxisLabels(const int32 LayerId, TConstArrayView<float> GridLineSoundLevels, const FSlateFontInfo& Font, const FLinearColor& TextColor)
{
	SpaceDrawSize = FontMeasureService->Measure(TEXT(" "), Font);

	for (const float SoundLevel : GridLineSoundLevels)
	{
		const FString SoundLevelString = FormatSoundLevelString(SoundLevel, { .NumFractionalDigits = 0, .bIncludeUnits = false });
		const FVector2f LabelDrawSize = FontMeasureService->Measure(SoundLevelString, Font);
		const float GridLineLocalY = ScaleInfo.SoundLevelToLocalY(SoundLevel);
		const float LabelTop = GridLineLocalY - 0.5f * LabelDrawSize.Y;
		const float LabelBottom = GridLineLocalY + 0.5f * LabelDrawSize.Y;
		if (LabelTop >= LocalBackgroundRect.Top && LabelBottom <= LocalBackgroundRect.Bottom)
		{
			// Draw label on the left hand side:
			DrawLabelIfNoOverlap(LayerId, LocalBackgroundRect.Left, LabelTop, LabelDrawSize, SoundLevelString, Font, TextColor);

			// Draw label on the right hand side:
			DrawLabelIfNoOverlap(LayerId, LocalBackgroundRect.Right - LabelDrawSize.X, LabelTop, LabelDrawSize, SoundLevelString, Font, TextColor);
		}
	}
}

void FAudioSpectrumPlotGridAndLabelDrawingHelper::DrawFrequencyAxisLabels(const int32 LayerId, TConstArrayView<float> GridLineFrequencies, const FSlateFontInfo& Font, const FLinearColor& TextColor)
{
	SpaceDrawSize = FontMeasureService->Measure(TEXT(" "), Font);

	for (float Freq : GridLineFrequencies)
	{
		const FString FreqString = FormatFreqString(Freq, { .bAlwaysDisplayMaximumFractionalDigits = false, .bIncludeUnits = false });
		const FVector2f LabelDrawSize = FontMeasureService->Measure(FreqString, Font);
		const float GridLineLocalX = ScaleInfo.FrequencyToLocalX(Freq);
		const float LabelLeft = GridLineLocalX - 0.5f * LabelDrawSize.X;
		const float LabelRight = GridLineLocalX + 0.5f * LabelDrawSize.X;
		if (LabelLeft >= LocalBackgroundRect.Left && LabelRight <= LocalBackgroundRect.Right)
		{
			// Draw label at the top:
			DrawLabelIfNoOverlap(LayerId, LabelLeft, LocalBackgroundRect.Top, LabelDrawSize, FreqString, Font, TextColor);

			// Draw label at the bottom:
			DrawLabelIfNoOverlap(LayerId, LabelLeft, LocalBackgroundRect.Bottom - LabelDrawSize.Y, LabelDrawSize, FreqString, Font, TextColor);
		}
	}
}

FString FAudioSpectrumPlotGridAndLabelDrawingHelper::FormatSoundLevelString(const float SoundLevel, const FSoundLevelFormattingOptions& SoundLevelFormattingOptions)
{
	FNumberFormattingOptions NumberFormattingOptions;
	NumberFormattingOptions.MinimumFractionalDigits = SoundLevelFormattingOptions.NumFractionalDigits;
	NumberFormattingOptions.MaximumFractionalDigits = SoundLevelFormattingOptions.NumFractionalDigits;
	if (SoundLevel != 0.0f)
	{
		NumberFormattingOptions.AlwaysSign = true;
	}

	const FText NumberText = FText::AsNumber(SoundLevel, &NumberFormattingOptions);
	if (SoundLevelFormattingOptions.bIncludeUnits)
	{
		return NumberText.ToString() + TEXT(" dB");
	}
	else
	{
		return NumberText.ToString();
	}
}

FString FAudioSpectrumPlotGridAndLabelDrawingHelper::FormatFreqString(const float Freq, const FFreqFormattingOptions& FreqFormattingOptions)
{
	FNumberFormattingOptions NumberFormattingOptions;

	if (Freq >= 1000.0f)
	{
		NumberFormattingOptions.MaximumFractionalDigits = (Freq < 10000.0f) ? 2 : 1; // Displaying a max of 3 significant figures.
		if (FreqFormattingOptions.bAlwaysDisplayMaximumFractionalDigits)
		{
			NumberFormattingOptions.MinimumFractionalDigits = NumberFormattingOptions.MaximumFractionalDigits;
		}

		const FText NumberText = FText::AsNumber(Freq / 1000.0f, &NumberFormattingOptions);
		if (FreqFormattingOptions.bIncludeUnits)
		{
			return NumberText.ToString() + TEXT(" kHz");
		}
		else
		{
			return NumberText.ToString() + TEXT(" k");
		}
	}
	else
	{
		NumberFormattingOptions.MaximumFractionalDigits = 0;
		NumberFormattingOptions.MinimumFractionalDigits = 0;

		const FText NumberText = FText::AsNumber(Freq, &NumberFormattingOptions);
		if (FreqFormattingOptions.bIncludeUnits)
		{
			return NumberText.ToString() + TEXT(" Hz");
		}
		else
		{
			return NumberText.ToString();
		}
	}
}

void FAudioSpectrumPlotGridAndLabelDrawingHelper::DrawVerticalArrowhead(int32 LayerId, FVector2f TipPosition, FVector2f Size, const FLinearColor& LineColor) const
{
	TArray<FVector2f> LinePoints;
	LinePoints.Reserve(3);
	LinePoints.Add({ TipPosition.X - 0.5f * Size.X, TipPosition.Y - Size.Y });
	LinePoints.Add({ TipPosition.X, TipPosition.Y });
	LinePoints.Add({ TipPosition.X + 0.5f * Size.X, TipPosition.Y - Size.Y });
	FSlateDrawElement::MakeLines(ElementList, LayerId, AllottedGeometry.ToPaintGeometry(), LinePoints, ESlateDrawEffect::None, LineColor);
}

void FAudioSpectrumPlotGridAndLabelDrawingHelper::DrawLabelIfNoOverlap(const int32 LayerId, const float LabelLeft, const float LabelTop, const FVector2f& LabelDrawSize, const FString LabelText, const FSlateFontInfo& Font, const FLinearColor& TextColor)
{
	const FSlateLayoutTransform LabelTransform(FVector2f(LabelLeft, LabelTop));
	const FSlateRect LabelRect(LabelTransform.TransformPoint(FVector2f::ZeroVector), LabelTransform.TransformPoint(LabelDrawSize));
	const FSlateRect ModifiedLabelRect = GetModifiedLabelRect(LabelRect);
	if (!IsOverlappingPreviouslyDrawnLabel(ModifiedLabelRect))
	{
		const FPaintGeometry LabelPaintGeometry = AllottedGeometry.MakeChild(LabelDrawSize, LabelTransform).ToPaintGeometry();
		FSlateDrawElement::MakeText(ElementList, LayerId, LabelPaintGeometry, LabelText, Font, ESlateDrawEffect::None, TextColor);
		DrawnLabelRects.Add(LabelRect);
	}
}

FSlateRect FAudioSpectrumPlotGridAndLabelDrawingHelper::GetModifiedLabelRect(const FSlateRect& LabelRect) const
{
	const float TightLabelTop = FMath::Lerp(LabelRect.Top, LabelRect.Bottom, 0.1f);
	const float TightLabelBottom = FMath::Lerp(LabelRect.Bottom, LabelRect.Top, 0.1f);
	const float PaddedLabelLeft = LabelRect.Left - 0.5 * SpaceDrawSize.X;
	const float PaddedLabelRight = LabelRect.Right + 0.5 * SpaceDrawSize.X;
	return FSlateRect(PaddedLabelLeft, TightLabelTop, PaddedLabelRight, TightLabelBottom);
}

bool FAudioSpectrumPlotGridAndLabelDrawingHelper::IsOverlappingPreviouslyDrawnLabel(const FSlateRect& LabelRect) const
{
	const FSlateRect* OverlappingDrawnLabel = DrawnLabelRects.FindByPredicate([&LabelRect](const FSlateRect& PrevDrawnLabelRect)
		{
			return (
				LabelRect.Top < PrevDrawnLabelRect.Bottom && LabelRect.Bottom > PrevDrawnLabelRect.Top &&
				LabelRect.Left < PrevDrawnLabelRect.Right && LabelRect.Right > PrevDrawnLabelRect.Left
				);
		});

	return (OverlappingDrawnLabel != nullptr);
};

const float SAudioSpectrumPlot::ClampMinSoundLevel = -200.0f;
FName SAudioSpectrumPlot::ContextMenuExtensionHook("SpectrumPlotDisplayOptions");

void SAudioSpectrumPlot::Construct(const FArguments& InArgs)
{
	check(InArgs._Style);

	Style = InArgs._Style;
	ViewMinFrequency = InArgs._ViewMinFrequency;
	ViewMaxFrequency = InArgs._ViewMaxFrequency;
	ViewMinSoundLevel = InArgs._ViewMinSoundLevel;
	ViewMaxSoundLevel = InArgs._ViewMaxSoundLevel;
	TiltExponent = InArgs._TiltExponent;
	TiltPivotFrequency = InArgs._TiltPivotFrequency;
	SelectedFrequency = InArgs._SelectedFrequency;
	bDisplayCrosshair = InArgs._DisplayCrosshair;
	bDisplayFrequencyAxisLabels = InArgs._DisplayFrequencyAxisLabels;
	bDisplaySoundLevelAxisLabels = InArgs._DisplaySoundLevelAxisLabels;
	bDisplayFrequencyGridLines = InArgs._DisplayFrequencyGridLines;
	bDisplaySoundLevelGridLines = InArgs._DisplaySoundLevelGridLines;
	FrequencyAxisScale = InArgs._FrequencyAxisScale;
	FrequencyAxisPixelBucketMode = InArgs._FrequencyAxisPixelBucketMode;
	BackgroundColor = InArgs._BackgroundColor;
	GridColor = InArgs._GridColor;
	AxisLabelColor = InArgs._AxisLabelColor;
	CrosshairColor = InArgs._CrosshairColor;
	SpectrumColor = InArgs._SpectrumColor;
	bAllowContextMenu = InArgs._AllowContextMenu;
	OnContextMenuOpening = InArgs._OnContextMenuOpening;
	OnTiltSpectrumMenuEntryClicked = InArgs._OnTiltSpectrumMenuEntryClicked;
	OnFrequencyAxisPixelBucketModeMenuEntryClicked = InArgs._OnFrequencyAxisPixelBucketModeMenuEntryClicked;
	OnFrequencyAxisScaleMenuEntryClicked = InArgs._OnFrequencyAxisScaleMenuEntryClicked;
	OnDisplayFrequencyAxisLabelsButtonToggled = InArgs._OnDisplayFrequencyAxisLabelsButtonToggled;
	OnDisplaySoundLevelAxisLabelsButtonToggled = InArgs._OnDisplaySoundLevelAxisLabelsButtonToggled;
	OnGetAudioSpectrumData = InArgs._OnGetAudioSpectrumData;
}

TSharedRef<const FExtensionBase> SAudioSpectrumPlot::AddContextMenuExtension(EExtensionHook::Position HookPosition, const TSharedPtr<FUICommandList>& CommandList, const FMenuExtensionDelegate& MenuExtensionDelegate)
{
	if (!ContextMenuExtender.IsValid())
	{
		ContextMenuExtender = MakeShared<FExtender>();
	}

	return ContextMenuExtender->AddMenuExtension(ContextMenuExtensionHook, HookPosition, CommandList, MenuExtensionDelegate);
}

void SAudioSpectrumPlot::RemoveContextMenuExtension(const TSharedRef<const FExtensionBase>& Extension)
{
	if (ensure(ContextMenuExtender.IsValid()))
	{
		ContextMenuExtender->RemoveExtension(Extension);
	}
}

FReply SAudioSpectrumPlot::OnMouseButtonDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (!HasMouseCapture())
	{
		if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			// Right clicking to summon context menu, but we'll do that on mouse-up.
			return FReply::Handled().CaptureMouse(AsShared()).SetUserFocus(AsShared(), EFocusCause::Mouse);
		}
	}

	return SCompoundWidget::OnMouseButtonDown(InMyGeometry, InMouseEvent);
}

FReply SAudioSpectrumPlot::OnMouseButtonUp(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	// The mouse must have been captured by mouse down before we'll process mouse ups
	if (HasMouseCapture())
	{
		if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			if (InMyGeometry.IsUnderLocation(InMouseEvent.GetScreenSpacePosition()) && bAllowContextMenu.Get())
			{
				TSharedPtr<SWidget> ContextMenu = OnContextMenuOpening.IsBound() ? OnContextMenuOpening.Execute() : BuildDefaultContextMenu();

				if (ContextMenu.IsValid())
				{
					const FWidgetPath WidgetPath = (InMouseEvent.GetEventPath() != nullptr) ? *InMouseEvent.GetEventPath() : FWidgetPath();

					FSlateApplication::Get().PushMenu(
						AsShared(),
						WidgetPath,
						ContextMenu.ToSharedRef(),
						InMouseEvent.GetScreenSpacePosition(),
						FPopupTransitionEffect::ESlideDirection::ContextMenu);
				}
			}

			return FReply::Handled().ReleaseMouseCapture();
		}
	}

	return SCompoundWidget::OnMouseButtonUp(InMyGeometry, InMouseEvent);
}

FReply SAudioSpectrumPlot::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!SelectedFrequency.IsBound())
	{
		// If not bound to an external function, set SelectedFrequency from the mouse hover position:
		const FAudioSpectrumPlotScaleInfo ScaleInfo(MyGeometry.GetLocalSize(), FrequencyAxisScale.Get(), ViewMinFrequency.Get(), ViewMaxFrequency.Get(), ViewMinSoundLevel.Get(), ViewMaxSoundLevel.Get());
		const FVector2f MouseMoveLocation = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		SelectedFrequency = ScaleInfo.LocalXToFrequency(MouseMoveLocation.X);
	}

	return FReply::Unhandled();
}

void SAudioSpectrumPlot::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	if (!SelectedFrequency.IsBound())
	{
		// If not bound to an external function, clear SelectedFrequency when mouse is no longer hovering over the plot widget:
		SelectedFrequency = NullOpt;
	}
}

FAudioSpectrumPlotScaleInfo SAudioSpectrumPlot::GetScaleInfo() const
{
	const FGeometry& AllottedGeometry = GetPaintSpaceGeometry();
	return FAudioSpectrumPlotScaleInfo(AllottedGeometry.GetLocalSize(), FrequencyAxisScale.Get(), ViewMinFrequency.Get(), ViewMaxFrequency.Get(), ViewMinSoundLevel.Get(), ViewMaxSoundLevel.Get());
}

int32 SAudioSpectrumPlot::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const FAudioSpectrumPlotScaleInfo ScaleInfo(AllottedGeometry.GetLocalSize(), FrequencyAxisScale.Get(), ViewMinFrequency.Get(), ViewMaxFrequency.Get(), ViewMinSoundLevel.Get(), ViewMaxSoundLevel.Get());

	LayerId = DrawSolidBackgroundRectangle(AllottedGeometry, OutDrawElements, LayerId, InWidgetStyle);

	LayerId = DrawGrid(AllottedGeometry, OutDrawElements, LayerId, InWidgetStyle, ScaleInfo);

	LayerId = DrawPowerSpectrum(AllottedGeometry, OutDrawElements, LayerId, InWidgetStyle, ScaleInfo);

	return LayerId;
}

int32 SAudioSpectrumPlot::DrawSolidBackgroundRectangle(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle) const
{
	const FSlateBrush BackgroundBrush;
	const FLinearColor BoxColor = GetBackgroundColor(InWidgetStyle);
	FSlateDrawElement::MakeBox(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), &BackgroundBrush, ESlateDrawEffect::None, BoxColor);

	return LayerId + 1;
}

int32 SAudioSpectrumPlot::DrawGrid(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, const FAudioSpectrumPlotScaleInfo& ScaleInfo) const
{
	TArray<float> GridLineSoundLevels;
	GetGridLineSoundLevels(GridLineSoundLevels);

	TArray<float> AllGridLineFrequencies;
	TArray<float> MajorGridLineFrequencies;
	GetGridLineFrequencies(AllGridLineFrequencies, MajorGridLineFrequencies);

	const FLinearColor LineColor = GetGridColor(InWidgetStyle);

	FAudioSpectrumPlotGridAndLabelDrawingHelper GridAndLabelDrawingHelper(AllottedGeometry, OutDrawElements, ScaleInfo);

	if (bDisplaySoundLevelGridLines.Get())
	{
		GridAndLabelDrawingHelper.DrawSoundLevelGridLines(LayerId, GridLineSoundLevels, LineColor);
	}

	if (bDisplayFrequencyGridLines.Get())
	{
		GridAndLabelDrawingHelper.DrawFrequencyGridLines(LayerId, AllGridLineFrequencies, LineColor);
	}

	return LayerId + 1;
}

void SAudioSpectrumPlot::GetGridLineSoundLevels(TArray<float>& GridLineSoundLevels) const
{
	// Define grid line sound levels (dB scale):
	const float MaxSoundLevel = ViewMaxSoundLevel.Get();
	const float MinSoundLevel = ViewMinSoundLevel.Get();
	const float SoundLevelIncrement = 20.0f * FMath::LogX(10.0f, 2.0f);

	// Add grid lines from 0dB up to MaxSoundLevel:
	float SoundLevel = 0.0f;
	while (SoundLevel <= MaxSoundLevel)
	{
		GridLineSoundLevels.Add(SoundLevel);
		SoundLevel += SoundLevelIncrement;
	}

	// Add grid lines from below 0dB down to MinSoundLevel:
	SoundLevel = 0.0f - SoundLevelIncrement;
	while (SoundLevel >= MinSoundLevel)
	{
		GridLineSoundLevels.Add(SoundLevel);
		SoundLevel -= SoundLevelIncrement;
	}
}

void SAudioSpectrumPlot::GetGridLineFrequencies(TArray<float>& AllGridLineFrequencies, TArray<float>& MajorGridLineFrequencies) const
{
	if (FrequencyAxisScale.Get() == EAudioSpectrumPlotFrequencyAxisScale::Logarithmic)
	{
		// Define grid line frequencies (log scale):

		const float MinGridFreq = ViewMinFrequency.Get();
		const float MaxGridFreq = ViewMaxFrequency.Get();
		const float Log10MinGridFreq = FMath::LogX(10.0f, MinGridFreq);

		float Freq = FMath::Pow(10.0f, FMath::Floor(Log10MinGridFreq));
		while (Freq <= MaxGridFreq)
		{
			if (Freq >= MinGridFreq)
			{
				MajorGridLineFrequencies.Add(Freq);
			}

			const float FreqIncrement = Freq;
			const float NextJump = 10.0f * FreqIncrement;
			while (Freq < NextJump && Freq <= MaxGridFreq)
			{
				if (Freq >= MinGridFreq)
				{
					AllGridLineFrequencies.Add(Freq);
				}

				Freq += FreqIncrement;
			}
		}
	}
	else
	{
		// Define grid line frequencies (linear scale):
		const float ViewFrequencyRange = ViewMaxFrequency.Get() - ViewMinFrequency.Get();

		// Find grid spacing to draw around 10 grid lines:
		const float Log10ApproxGridSpacing = FMath::LogX(10.0f, ViewFrequencyRange / 10.0f);
		const float GridSpacing = FMath::Pow(10.0f, FMath::Floor(Log10ApproxGridSpacing));

		// Add frequencies to the grid line arrays:
		const float StartFrequency = GridSpacing * FMath::CeilToDouble(ViewMinFrequency.Get() / GridSpacing);
		for (float Freq = StartFrequency; Freq <= ViewMaxFrequency.Get(); Freq += GridSpacing)
		{
			AllGridLineFrequencies.Add(Freq);
			MajorGridLineFrequencies.Add(Freq);
		}
	}
}

int32 SAudioSpectrumPlot::DrawPowerSpectrum(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, const FAudioSpectrumPlotScaleInfo& ScaleInfo) const
{
	// Get the power spectrum data if available:
	const FAudioPowerSpectrumData PowerSpectrum = GetPowerSpectrum();
	ensure(PowerSpectrum.CenterFrequencies.Num() == PowerSpectrum.SquaredMagnitudes.Num());
	const int NumFrequencies = FMath::Min(PowerSpectrum.CenterFrequencies.Num(), PowerSpectrum.SquaredMagnitudes.Num());
	if (NumFrequencies > 0)
	{
		// Convert to array of data points with X == frequency, Y == sound level in dB.
		TArray<FVector2f> DataPoints;
		DataPoints.Reserve(NumFrequencies);

		const float TiltExponentValue = TiltExponent.Get();
		const float TiltPivotFrequencyValue = TiltPivotFrequency.Get();
		const float ClampMinFrequency = (FrequencyAxisScale.Get() == EAudioSpectrumPlotFrequencyAxisScale::Logarithmic) ? 0.00001f : -FLT_MAX; // Cannot plot DC with log scale.
		const float ClampMinMagnitudeSquared = FMath::Pow(10.0f, ClampMinSoundLevel / 10.0f); // Clamp at -200dB
		for (int Index = 0; Index < NumFrequencies; Index++)
		{
			const float Frequency = FMath::Max(PowerSpectrum.CenterFrequencies[Index], ClampMinFrequency);
			const float TiltPowerGain = FMath::Pow(Frequency / TiltPivotFrequencyValue, TiltExponentValue);
			const float MagnitudeSquared = FMath::Max(TiltPowerGain * PowerSpectrum.SquaredMagnitudes[Index], ClampMinMagnitudeSquared);
			const float SoundLevel = 10.0f * FMath::LogX(10.0f, MagnitudeSquared);
			DataPoints.Add({ Frequency, SoundLevel });
		}

		// Line points to plot will be added to this array:
		TArray<FVector2f> LinePoints;

		switch (FrequencyAxisPixelBucketMode.Get())
		{
			case EAudioSpectrumPlotFrequencyAxisPixelBucketMode::Sample:
			{
				// For DataPoints that map to the same frequency axis pixel bucket, choose the one that is nearest the bucket center:
				const auto CostFunction = [&ScaleInfo](const FVector2f& DataPoint)
				{
					const float LocalX = ScaleInfo.FrequencyToLocalX(DataPoint.X);
					return FMath::Abs(LocalX - FMath::RoundToInt32(LocalX));
				};

				// Get the line points to plot:
				GetSpectrumLinePoints(LinePoints, DataPoints, ScaleInfo, CostFunction);
			}
			break;
			case EAudioSpectrumPlotFrequencyAxisPixelBucketMode::Peak:
			{
				// For DataPoints that map to the same frequency axis pixel bucket, choose the one with the highest sound level:
				const auto CostFunction = [](const FVector2f& DataPoint) { return -DataPoint.Y; };

				// Get the line points to plot:
				GetSpectrumLinePoints(LinePoints, DataPoints, ScaleInfo, CostFunction);
			}
			break;
			case EAudioSpectrumPlotFrequencyAxisPixelBucketMode::Average:
			{
				// For DataPoints that map to the same frequency axis pixel bucket, take the average:

				int32 CurrFreqAxisPixelBucket = INT32_MIN;
				FVector2f CurrSum = FVector2f::ZeroVector;
				int CurrCount = 0;
				for (FVector2f DataPoint : DataPoints)
				{
					const float LocalX = ScaleInfo.FrequencyToLocalX(DataPoint.X);
					const float LocalY = ScaleInfo.SoundLevelToLocalY(DataPoint.Y);

					const int32 FreqAxisPixelBucket = FMath::RoundToInt32(LocalX);
					if (FreqAxisPixelBucket != CurrFreqAxisPixelBucket && CurrCount > 0)
					{
						// New DataPoint is not at the same frequency axis pixel bucket.

						// Add current average to line plot:
						LinePoints.Add(CurrSum / CurrCount);

						// Reset current average:
						CurrSum = FVector2f::ZeroVector;
						CurrCount = 0;
					}

					// Set the current frequency axis pixel bucket, and add to the average:
					CurrFreqAxisPixelBucket = FreqAxisPixelBucket;
					CurrSum += FVector2f(LocalX, LocalY);
					CurrCount++;
				}

				// Add remaining average to line plot:
				check(CurrCount > 0);
				LinePoints.Add(CurrSum / CurrCount);
			}
			break;
		}

		// Draw crosshair and axis labels (horizontal crosshair position depends on the spectrum line points to be plotted):
		LayerId = DrawCrosshairAndAxisLabels(AllottedGeometry, OutDrawElements, LayerId, InWidgetStyle, ScaleInfo, LinePoints);

		// Actually draw the line points:
		const FLinearColor& LineColor = GetSpectrumColor(InWidgetStyle);
		FSlateDrawElement::MakeLines(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), LinePoints, ESlateDrawEffect::None, LineColor, true, 1.0f);
		LayerId++;
	}

	return LayerId;
}

int32 SAudioSpectrumPlot::DrawCrosshairAndAxisLabels(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, const FAudioSpectrumPlotScaleInfo& ScaleInfo, TConstArrayView<FVector2f> LinePoints) const
{
	FAudioSpectrumPlotGridAndLabelDrawingHelper GridAndLabelDrawingHelper(AllottedGeometry, OutDrawElements, ScaleInfo);

	FLinearColor TextColor = GetAxisLabelColor(InWidgetStyle);

	if (bDisplayCrosshair.Get())
	{
		const TOptional<float> SelectedFrequencyOptional = SelectedFrequency.Get();
		if (SelectedFrequencyOptional.IsSet())
		{
			const float CrosshairFrequency = SelectedFrequencyOptional.GetValue();
			const float CrosshairPosX = ScaleInfo.FrequencyToLocalX(CrosshairFrequency);
			const int32 PointIndex = LinePoints.FindLastByPredicate([=](FVector2f Point) { return Point.X <= CrosshairPosX; });
			if (PointIndex != INDEX_NONE && PointIndex + 1 < LinePoints.Num())
			{
				const FVector2f& PointL = LinePoints[PointIndex];
				const FVector2f& PointR = LinePoints[PointIndex + 1];
				const float LerpParam = (CrosshairPosX - PointL.X) / (PointR.X - PointL.X);
				const float CrosshairPosY = FMath::Lerp(PointL.Y, PointR.Y, LerpParam);
				const float CrosshairSoundLevel = ScaleInfo.LocalYToSoundLevel(CrosshairPosY);
				if (CrosshairSoundLevel > ClampMinSoundLevel)
				{
					const FSlateFontInfo& CrosshairLabelFont = Style->CrosshairLabelFont;
					const FLinearColor CrosshairLineColor = GetCrosshairColor(InWidgetStyle);
					GridAndLabelDrawingHelper.DrawCrosshairWithLabels(LayerId, CrosshairFrequency, CrosshairSoundLevel, CrosshairLabelFont, TextColor, CrosshairLineColor);
				}
			}
		}
	}

	TArray<float> GridLineSoundLevels;
	GetGridLineSoundLevels(GridLineSoundLevels);

	TArray<float> AllGridLineFrequencies;
	TArray<float> MajorGridLineFrequencies;
	GetGridLineFrequencies(AllGridLineFrequencies, MajorGridLineFrequencies);

	const FSlateFontInfo& AxisLabelFont = Style->AxisLabelFont;

	if (GridAndLabelDrawingHelper.HasDrawnLabels())
	{
		// De-emphasize grid axis labels if we are displaying crosshair labels:
		TextColor.A *= 0.5;
	}

	if (bDisplaySoundLevelAxisLabels.Get())
	{
		// Draw sound level axis labels for all grid lines.
		GridAndLabelDrawingHelper.DrawSoundLevelAxisLabels(LayerId, GridLineSoundLevels, AxisLabelFont, TextColor);
	}

	if (bDisplayFrequencyAxisLabels.Get())
	{
		// Draw frequency axis labels for all major grid lines.
		GridAndLabelDrawingHelper.DrawFrequencyAxisLabels(LayerId, MajorGridLineFrequencies, AxisLabelFont, TextColor);
	}

	if (GridAndLabelDrawingHelper.HasDrawnLabels())
	{
		// We drew some labels, so increment layer ID:
		LayerId++;
	}

	return LayerId;
}

FAudioPowerSpectrumData SAudioSpectrumPlot::GetPowerSpectrum() const
{
	if (OnGetAudioSpectrumData.IsBound())
	{
		return OnGetAudioSpectrumData.Execute();
	}

	return FAudioPowerSpectrumData();
}

void SAudioSpectrumPlot::GetSpectrumLinePoints(TArray<FVector2f>& OutLinePoints, TConstArrayView<FVector2f> DataPoints, const FAudioSpectrumPlotScaleInfo& ScaleInfo, TFunctionRef<float(const FVector2f& DataPoint)> CostFunction)
{
	// Function to find whether two data points map to the same frequency axis pixel bucket:
	const auto IsSameFreqAxisPixelBucket = [&ScaleInfo](FVector2f DataPoint1, FVector2f DataPoint2)
	{
		const int32 PixelBucket1 = FMath::RoundToInt32(ScaleInfo.FrequencyToLocalX(DataPoint1.X));
		const int32 PixelBucket2 = FMath::RoundToInt32(ScaleInfo.FrequencyToLocalX(DataPoint2.X));
		return (PixelBucket1 == PixelBucket2);
	};

	TOptional<FVector2f> CurrBestDataPoint;

	for (FVector2f DataPoint : DataPoints)
	{
		if (CurrBestDataPoint.IsSet() && !IsSameFreqAxisPixelBucket(DataPoint, CurrBestDataPoint.GetValue()))
		{
			// New DataPoint is not at the same frequency axis pixel bucket as CurrBestDataPoint.

			// Add CurrBestDataPoint to line plot:
			const FVector2f LocalPosCurrBestDataPoint = ScaleInfo.ToLocalPos(CurrBestDataPoint.GetValue());
			OutLinePoints.Add(LocalPosCurrBestDataPoint);

			// Reset best value: 
			CurrBestDataPoint.Reset();
		}

		if (!CurrBestDataPoint.IsSet() || CostFunction(DataPoint) < CostFunction(CurrBestDataPoint.GetValue()))
		{
			// New DataPoint is either at a new frequency axis pixel bucket or is better than CurrBestDataPoint.
			CurrBestDataPoint = DataPoint;
		}
	}

	{
		// Add final CurrBestDataPoint to line plot:
		const FVector2f LocalPosCurrBestDataPoint = ScaleInfo.ToLocalPos(CurrBestDataPoint.GetValue());
		OutLinePoints.Add(LocalPosCurrBestDataPoint);
	}
}

FLinearColor SAudioSpectrumPlot::GetBackgroundColor(const FWidgetStyle& InWidgetStyle) const
{
	const FSlateColor& SlateColor = (BackgroundColor.Get() != FSlateColor::UseStyle()) ? BackgroundColor.Get() : Style->BackgroundColor;
	return SlateColor.GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint();
}

FLinearColor SAudioSpectrumPlot::GetGridColor(const FWidgetStyle& InWidgetStyle) const
{
	const FSlateColor& SlateColor = (GridColor.Get() != FSlateColor::UseStyle()) ? GridColor.Get() : Style->GridColor;
	return SlateColor.GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint();
}

FLinearColor SAudioSpectrumPlot::GetAxisLabelColor(const FWidgetStyle& InWidgetStyle) const
{
	const FSlateColor& SlateColor = (AxisLabelColor.Get() != FSlateColor::UseStyle()) ? AxisLabelColor.Get() : Style->AxisLabelColor;
	return SlateColor.GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint();
}

FLinearColor SAudioSpectrumPlot::GetCrosshairColor(const FWidgetStyle& InWidgetStyle) const
{
	const FSlateColor& SlateColor = (CrosshairColor.Get() != FSlateColor::UseStyle()) ? CrosshairColor.Get() : Style->CrosshairColor;
	return SlateColor.GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint();
}

FLinearColor SAudioSpectrumPlot::GetSpectrumColor(const FWidgetStyle& InWidgetStyle) const
{
	const FSlateColor& SlateColor = (SpectrumColor.Get() != FSlateColor::UseStyle()) ? SpectrumColor.Get() : Style->SpectrumColor;
	return SlateColor.GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint();
}

float SAudioSpectrumPlot::GetTiltExponentValue(const EAudioSpectrumPlotTilt InTilt)
{
	switch (InTilt)
	{
	default:
	case EAudioSpectrumPlotTilt::NoTilt:
		return 0.0f;
	case EAudioSpectrumPlotTilt::Plus1_5dBPerOctave:
		return 0.5f;
	case EAudioSpectrumPlotTilt::Plus3dBPerOctave:
		return 1.0f;
	case EAudioSpectrumPlotTilt::Plus4_5dBPerOctave:
		return 1.5f;
	case EAudioSpectrumPlotTilt::Plus6dBPerOctave:
		return 2.0f;
	}
}

TSharedRef<SWidget> SAudioSpectrumPlot::BuildDefaultContextMenu()
{
	constexpr bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr, ContextMenuExtender);

	MenuBuilder.BeginSection(ContextMenuExtensionHook, LOCTEXT("DisplayOptions", "Display Options"));

	if (OnTiltSpectrumMenuEntryClicked.IsBound() || !TiltExponent.IsBound())
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("TiltSpectrum", "Tilt Spectrum"),
			FText(),
			FNewMenuDelegate::CreateSP(this, &SAudioSpectrumPlot::BuildTiltSpectrumSubMenu));
	}

	if (OnFrequencyAxisPixelBucketModeMenuEntryClicked.IsBound() || !FrequencyAxisPixelBucketMode.IsBound())
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("FrequencyAxisPixelBucketMode", "Pixel Plot Mode"),
			FText(),
			FNewMenuDelegate::CreateSP(this, &SAudioSpectrumPlot::BuildFrequencyAxisPixelBucketModeSubMenu));
	}

	if (OnFrequencyAxisScaleMenuEntryClicked.IsBound() || !FrequencyAxisScale.IsBound())
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("FrequencyAxisScale", "Frequency Scale"),
			FText(),
			FNewMenuDelegate::CreateSP(this, &SAudioSpectrumPlot::BuildFrequencyAxisScaleSubMenu));
	}

	if (OnDisplayFrequencyAxisLabelsButtonToggled.IsBound() || !bDisplayFrequencyAxisLabels.IsBound())
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("DisplayFrequencyAxisLabels", "Display Frequency Axis Labels"),
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSPLambda(this, [this]()
					{
						if (!bDisplayFrequencyAxisLabels.IsBound())
						{
							bDisplayFrequencyAxisLabels = !bDisplayFrequencyAxisLabels.Get();
						}

						OnDisplayFrequencyAxisLabelsButtonToggled.ExecuteIfBound();
					}),
				FCanExecuteAction(),
				FIsActionChecked::CreateSPLambda(this, [this]() { return bDisplayFrequencyAxisLabels.Get(); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}

	if (OnDisplaySoundLevelAxisLabelsButtonToggled.IsBound() || !bDisplaySoundLevelAxisLabels.IsBound())
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("DisplaySoundLevelAxisLabels", "Display Sound Level Axis Labels"),
			FText(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSPLambda(this, [this]()
					{
						if (!bDisplaySoundLevelAxisLabels.IsBound())
						{
							bDisplaySoundLevelAxisLabels = !bDisplaySoundLevelAxisLabels.Get();
						}

						OnDisplaySoundLevelAxisLabelsButtonToggled.ExecuteIfBound();
					}),
				FCanExecuteAction(),
				FIsActionChecked::CreateSPLambda(this, [this]() { return bDisplaySoundLevelAxisLabels.Get(); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SAudioSpectrumPlot::BuildTiltSpectrumSubMenu(FMenuBuilder& SubMenu)
{
	const UEnum* EnumClass = StaticEnum<EAudioSpectrumPlotTilt>();
	const int32 NumEnumValues = EnumClass->NumEnums() - 1; // Exclude 'MAX' enum value.
	for (int32 Index = 0; Index < NumEnumValues; Index++)
	{
		const auto EnumValue = static_cast<EAudioSpectrumPlotTilt>(EnumClass->GetValueByIndex(Index));
		const float TiltExponentValue = GetTiltExponentValue(EnumValue);

		SubMenu.AddMenuEntry(
			EnumClass->GetDisplayNameTextByIndex(Index),
#if WITH_EDITOR
			EnumClass->GetToolTipTextByIndex(Index),
#else
			FText(),
#endif
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSPLambda(this, [this, EnumValue, TiltExponentValue]()
					{
						if (!TiltExponent.IsBound())
						{
							TiltExponent = TiltExponentValue;
						}

						OnTiltSpectrumMenuEntryClicked.ExecuteIfBound(EnumValue);
					}),
				FCanExecuteAction(),
				FIsActionChecked::CreateSPLambda(this, [this, TiltExponentValue]() { return (TiltExponent.Get() == TiltExponentValue); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}
}

void SAudioSpectrumPlot::BuildFrequencyAxisScaleSubMenu(FMenuBuilder& SubMenu)
{
	const UEnum* EnumClass = StaticEnum<EAudioSpectrumPlotFrequencyAxisScale>();
	const int32 NumEnumValues = EnumClass->NumEnums() - 1; // Exclude 'MAX' enum value.
	for (int32 Index = 0; Index < NumEnumValues; Index++)
	{
		const auto EnumValue = static_cast<EAudioSpectrumPlotFrequencyAxisScale>(EnumClass->GetValueByIndex(Index));

		SubMenu.AddMenuEntry(
			EnumClass->GetDisplayNameTextByIndex(Index),
#if WITH_EDITOR
			EnumClass->GetToolTipTextByIndex(Index),
#else
			FText(),
#endif
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSPLambda(this, [this, EnumValue]()
					{
						if (!FrequencyAxisScale.IsBound())
						{
							FrequencyAxisScale = EnumValue;
						}

						OnFrequencyAxisScaleMenuEntryClicked.ExecuteIfBound(EnumValue);
					}),
				FCanExecuteAction(),
				FIsActionChecked::CreateSPLambda(this, [this, EnumValue]() { return (FrequencyAxisScale.Get() == EnumValue); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}
}

void SAudioSpectrumPlot::BuildFrequencyAxisPixelBucketModeSubMenu(FMenuBuilder& SubMenu)
{
	const UEnum* EnumClass = StaticEnum<EAudioSpectrumPlotFrequencyAxisPixelBucketMode>();
	const int32 NumEnumValues = EnumClass->NumEnums() - 1; // Exclude 'MAX' enum value.
	for (int32 Index = 0; Index < NumEnumValues; Index++)
	{
		const auto EnumValue = static_cast<EAudioSpectrumPlotFrequencyAxisPixelBucketMode>(EnumClass->GetValueByIndex(Index));

		SubMenu.AddMenuEntry(
			EnumClass->GetDisplayNameTextByIndex(Index),
#if WITH_EDITOR
			EnumClass->GetToolTipTextByIndex(Index),
#else
			FText(),
#endif
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSPLambda(this, [this, EnumValue]()
					{
						if (!FrequencyAxisPixelBucketMode.IsBound())
						{
							FrequencyAxisPixelBucketMode = EnumValue;
						}

						OnFrequencyAxisPixelBucketModeMenuEntryClicked.ExecuteIfBound(EnumValue);
					}),
				FCanExecuteAction(),
				FIsActionChecked::CreateSPLambda(this, [this, EnumValue]() { return (FrequencyAxisPixelBucketMode.Get() == EnumValue); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}
}

#undef LOCTEXT_NAMESPACE

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
