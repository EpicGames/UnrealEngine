// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAudioTextBox.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "Rendering/SlateRenderer.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Layout/SBorder.h"

const FVariablePrecisionNumericInterface SAudioTextBox::NumericInterface = FVariablePrecisionNumericInterface();

void SAudioTextBox::Construct(const SAudioTextBox::FArguments& InArgs)
{
	ShowLabelOnlyOnHover = InArgs._ShowLabelOnlyOnHover;
	ShowUnitsText = InArgs._ShowUnitsText;
	OnValueTextCommitted = InArgs._OnValueTextCommitted;
	Style = InArgs._Style;
	LabelBackgroundColor = InArgs._LabelBackgroundColor;

	const ISlateStyle* AudioWidgetsStyle = FSlateStyleRegistry::FindSlateStyle("AudioWidgetsStyle");
	if (AudioWidgetsStyle)
	{
		Style = &AudioWidgetsStyle->GetWidgetStyle<FAudioTextBoxStyle>("AudioTextBox.Style");
		LabelBackgroundColor = Style->BackgroundColor;

		SAssignNew(ValueText, SEditableText)
			.Text(FText::AsNumber(0.0f))
			.Justification(ETextJustify::Right)
			.OnTextCommitted(OnValueTextCommitted)
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis);
		SAssignNew(UnitsText, SEditableText)
			.Visibility_Lambda([this]() 
			{
				return ShowUnitsText.Get() ? EVisibility::Visible : EVisibility::Collapsed;
			})
			.Text(FText::FromString("units"))
			.Justification(ETextJustify::Left)
			.OverflowPolicy(ETextOverflowPolicy::Ellipsis);

		SAssignNew(LabelBorder, SBorder)
			.BorderImage(&Style->BackgroundImage)
			.BorderBackgroundColor(Style->BackgroundColor)
			.Padding(2.0f)
			.Visibility_Lambda([this]()
			{
				return (!ShowLabelOnlyOnHover.Get() || this->IsHovered()) ? EVisibility::Visible : EVisibility::Hidden;
			})
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Expose(ValueTextSlot)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				[
					ValueText.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f, 0.0f, 0.0f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					UnitsText.ToSharedRef()
				]
			];
	}

	ChildSlot
	[
		LabelBorder.ToSharedRef()
	];
}

void SAudioTextBox::SetLabelBackgroundColor(FSlateColor InColor)
{
	SetAttribute(LabelBackgroundColor, TAttribute<FSlateColor>(InColor), EInvalidateWidgetReason::Paint);
	LabelBorder->SetBorderBackgroundColor(InColor.GetSpecifiedColor());
}

void SAudioTextBox::SetValueText(const float OutputValue)
{
	ValueText->SetText(FText::FromString(NumericInterface.ToString(OutputValue)));
}

void SAudioTextBox::SetUnitsText(const FText Units)
{
	UnitsText->SetText(Units);
}

void SAudioTextBox::SetUnitsTextReadOnly(const bool bIsReadOnly)
{
	UnitsText->SetIsReadOnly(bIsReadOnly);
}

void SAudioTextBox::SetValueTextReadOnly(const bool bIsReadOnly)
{
	ValueText->SetIsReadOnly(bIsReadOnly);
}

void SAudioTextBox::SetShowLabelOnlyOnHover(const bool bShowLabelOnlyOnHover)
{
	SetAttribute(ShowLabelOnlyOnHover, TAttribute<bool>(bShowLabelOnlyOnHover), EInvalidateWidgetReason::Visibility);
}

void SAudioTextBox::SetShowUnitsText(const bool bShowUnitsText)
{
	SetAttribute(ShowUnitsText, TAttribute<bool>(bShowUnitsText), EInvalidateWidgetReason::Layout);

	if (bShowUnitsText)
	{
		ValueText->SetJustification(ETextJustify::Right);
	}
	else
	{
		ValueText->SetJustification(ETextJustify::Center);
	}
	UpdateValueTextWidth(OutputRange);
}

// Update value text size to accommodate the largest numbers possible within the output range
void SAudioTextBox::UpdateValueTextWidth(const FVector2D InOutputRange)
{
	OutputRange = InOutputRange;

	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const FText OutputRangeXText = FText::FromString(SAudioTextBox::NumericInterface.ToString(OutputRange.X));
	const FText OutputRangeYText = FText::FromString(SAudioTextBox::NumericInterface.ToString(OutputRange.Y));
	const FSlateFontInfo Font = FTextBlockStyle::GetDefault().Font;
	const float OutputRangeXTextWidth = FontMeasureService->Measure(OutputRangeXText, Font).X;
	const float OutputRangeYTextWidth = FontMeasureService->Measure(OutputRangeYText, Font).X;
	// add 1 digit of padding
	const float Padding = FontMeasureService->Measure(FText::FromString("-"), Font).X;
	float MaxValueLabelWidth = FMath::Max(OutputRangeXTextWidth, OutputRangeYTextWidth) + Padding;

	if (!ShowUnitsText.Get())
	{
		// set to max of label background width and calculated max width
		MaxValueLabelWidth = FMath::Max(MaxValueLabelWidth, Style->BackgroundImage.ImageSize.X);
	}
	else
	{
		const float UnitsTextWidth = FontMeasureService->Measure(UnitsText->GetText(), Font).X;
		MaxValueLabelWidth = FMath::Max(MaxValueLabelWidth, Style->BackgroundImage.ImageSize.X - UnitsTextWidth);
	}
	ValueText->SetMinDesiredWidth(MaxValueLabelWidth);
	if (ValueTextSlot)
	{	
		ValueTextSlot->SetMaxWidth(MaxValueLabelWidth);
	}
}
