// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAudioVectorscopePanelWidget.h"

#include "AudioWidgetsLog.h"
#include "DSP/Dsp.h"
#include "SAudioRadialSlider.h"
#include "SFixedSampledSequenceVectorViewer.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

#define LOCTEXT_NAMESPACE "SAudioVectorscopePanelWidget"

void SAudioVectorscopePanelWidget::Construct(const FArguments& InArgs, const FFixedSampledSequenceView& InData)
{
	ValueGridMaxDivisionParameter = InArgs._ValueGridMaxDivisionParameter;
	bHideValueGrid = InArgs._HideGrid.Get();

	check(InArgs._PanelStyle);
	PanelStyle = InArgs._PanelStyle;

	BuildWidget(InData, InArgs._PanelLayoutType);
}

void SAudioVectorscopePanelWidget::BuildWidget(const FFixedSampledSequenceView& InData, const EAudioPanelLayoutType InPanelLayoutType)
{
	DataView        = InData;
	PanelLayoutType = InPanelLayoutType;

	CreateBackground(PanelStyle->VectorViewerStyle);

	ValueGridOverlayXAxis = CreateValueGridOverlay(ValueGridMaxDivisionParameter,
		SampledSequenceValueGridOverlay::EGridDivideMode::MidSplit,
		PanelStyle->ValueGridStyle,
		SampledSequenceDrawingUtils::ESampledSequenceDrawOrientation::Horizontal);

	ValueGridOverlayYAxis = CreateValueGridOverlay(ValueGridMaxDivisionParameter,
		SampledSequenceValueGridOverlay::EGridDivideMode::MidSplit,
		PanelStyle->ValueGridStyle,
		SampledSequenceDrawingUtils::ESampledSequenceDrawOrientation::Vertical);
	
	CreateSequenceVectorViewer(DataView, PanelStyle->VectorViewerStyle);

	if (PanelLayoutType == EAudioPanelLayoutType::Advanced)
	{
		CreateVectorscopeControls();
	}

	CreateLayout();
}

void SAudioVectorscopePanelWidget::CreateLayout()
{
	// Vectorscope view
	auto CreateVectorscopeViewContainer = [this]()
	{
		TSharedPtr<SOverlay> VectorscopeViewOverlays = SNew(SOverlay);

		VectorscopeViewOverlays->AddSlot()
		[
			BackgroundBorder.ToSharedRef()
		];

		if (!bHideValueGrid)
		{
			VectorscopeViewOverlays->AddSlot()
			[
				ValueGridOverlayXAxis.ToSharedRef()
			];

			VectorscopeViewOverlays->AddSlot()
			[
				ValueGridOverlayYAxis.ToSharedRef()
			];
		}

		VectorscopeViewOverlays->AddSlot()
		[
			SequenceVectorViewer.ToSharedRef()
		];

		TSharedPtr<SVerticalBox> VectorscopeViewContainer = SNew(SVerticalBox);

		VectorscopeViewContainer->AddSlot()
		[
			VectorscopeViewOverlays.ToSharedRef()
		];

		VectorscopeViewContainer->SetClipping(EWidgetClipping::ClipToBounds);

		return VectorscopeViewContainer;
	};

	if (PanelLayoutType == EAudioPanelLayoutType::Advanced)
	{
		// Vectorscope controls
		auto CreateVectorscopeControlsContainer = [this]()
		{
			TSharedPtr<SVerticalBox> VectorscopeControlsContainer = SNew(SVerticalBox);

			// Time Window Knob
			VectorscopeControlsContainer->AddSlot()
			.FillHeight(0.2f)
			.Padding(0.0f, 0.0f, 0.0f, 5.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(0.2f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Vectorscope_DisplayPersistence_Display_Label", "Persistence"))
					.Justification(ETextJustify::Center)
				]
				+ SVerticalBox::Slot()
				.FillHeight(0.8f)
				[
					DisplayPersistenceKnob.ToSharedRef()
				]
			];

			// Scale Knob
			VectorscopeControlsContainer->AddSlot()
			.FillHeight(0.2f)
			.Padding(0.0f, 0.0f, 0.0f, 5.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(0.2f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Vectorscope_Scale_Display_Label", "Scale"))
					.Justification(ETextJustify::Center)
				]
				+ SVerticalBox::Slot()
				.FillHeight(0.8f)
				[
					ScaleKnob.ToSharedRef()
				]
			];

			VectorscopeControlsContainer->AddSlot()
			.FillHeight(0.6f)
			[
				SNew(SVerticalBox)
			];

			return VectorscopeControlsContainer;
		};

		VectorscopeViewProportion = 0.9f;

		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(VectorscopeViewProportion)
			[
				CreateVectorscopeViewContainer().ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f - VectorscopeViewProportion)
			[
				CreateVectorscopeControlsContainer().ToSharedRef()
			]
		];
	}
	else
	{
		ChildSlot
		[
			CreateVectorscopeViewContainer().ToSharedRef()
		];
	}
}

void SAudioVectorscopePanelWidget::CreateBackground(const FSampledSequenceVectorViewerStyle& VectorViewerStyle)
{
	BackgroundBorder = SNew(SBorder)
	.BorderImage(&VectorViewerStyle.BackgroundBrush)
	.BorderBackgroundColor(VectorViewerStyle.BackgroundColor);
}

TSharedPtr<SSampledSequenceValueGridOverlay> SAudioVectorscopePanelWidget::CreateValueGridOverlay(const uint32 MaxDivisionParameter,
	const SampledSequenceValueGridOverlay::EGridDivideMode DivideMode,
	const FSampledSequenceValueGridOverlayStyle& ValueGridStyle,
	const SampledSequenceDrawingUtils::ESampledSequenceDrawOrientation GridOrientation)
{
	using namespace SampledSequenceDrawingUtils;

	FSampledSequenceDrawingParams ValueGridOverlayDrawingParams;
	ValueGridOverlayDrawingParams.Orientation = GridOrientation;

	return SNew(SSampledSequenceValueGridOverlay)
		   .MaxDivisionParameter(MaxDivisionParameter)
		   .DivideMode(DivideMode)
		   .HideLabels(true)
		   .Style(&ValueGridStyle)
		   .SequenceDrawingParams(MoveTemp(ValueGridOverlayDrawingParams))
		   .NumDimensions(1);
}

void SAudioVectorscopePanelWidget::CreateSequenceVectorViewer(const FFixedSampledSequenceView& InData, const FSampledSequenceVectorViewerStyle& VectorViewerStyle)
{
	SequenceVectorViewer = SNew(SFixedSampledSequenceVectorViewer, InData.SampleData, InData.NumDimensions)
	.Style(&VectorViewerStyle)
	.SequenceDrawingParams(SampledSequenceDrawingUtils::FSampledSequenceDrawingParams());
}

void SAudioVectorscopePanelWidget::CreateDisplayPersistenceKnob()
{
	auto OnValueChangedLambda = [this](float Value)
	{
		if (DisplayPersistenceKnob.IsValid())
		{
			if (!bIsInputWidgetTransacting)
			{
			#if WITH_EDITOR
				if (GEditor)
				{
					GEditor->BeginTransaction(LOCTEXT("Vectorscope_DisplayPersistence_Knob_Changed_Msg", "Set vectorscope Time Window value."));
				}
			#endif
				bIsInputWidgetTransacting = true;
			}

			if (const float DisplayPersistenceKnobValue = DisplayPersistenceKnob->GetOutputValue(Value);
				DisplayPersistenceKnobValue != DisplayPersistenceValue)
			{
				DisplayPersistenceValue = DisplayPersistenceKnobValue;

				OnDisplayPersistenceValueChanged.Broadcast(DisplayPersistenceValue);
			}
		}
	};

	auto OnRadialSliderMouseCaptureBeginLambda = [this]()
	{
		if (!bIsInputWidgetTransacting)
		{
		#if WITH_EDITOR
			if (GEditor)
			{
				GEditor->BeginTransaction(LOCTEXT("Vectorscope_DisplayPersistence_Knob_CaptureBegin_Msg", "Set vectorscope Time Window value."));
			}
		#endif
			bIsInputWidgetTransacting = true;
		}
	};

	auto OnRadialSliderMouseCaptureEndLambda = [this]()
	{
		if (bIsInputWidgetTransacting)
		{
		#if WITH_EDITOR
			if (GEditor)
			{
				GEditor->EndTransaction();
			}
		#endif
			bIsInputWidgetTransacting = false;
		}
		else
		{
			UE_LOGF(LogAudioWidgets, Warning, "Unmatched vectorscope widget transaction.");
		}
	};

	DisplayPersistenceKnob = SNew(SAudioRadialSlider)
	.OnMouseCaptureBegin_Lambda(OnRadialSliderMouseCaptureBeginLambda)
	.OnMouseCaptureEnd_Lambda(OnRadialSliderMouseCaptureEndLambda)
	.SliderValue(0.0f);

	DisplayPersistenceKnob->SetOutputRange(DisplayPersistenceKnobOutputRange);
	DisplayPersistenceKnob->SetUnitsText(FText::FromString("ms"));

	DisplayPersistenceKnob->OnValueChanged.BindLambda(OnValueChangedLambda);
}

void SAudioVectorscopePanelWidget::CreateScaleKnob()
{
	auto OnValueChangedLambda = [this](float Value)
	{
		if (ScaleKnob.IsValid())
		{
			if (!bIsInputWidgetTransacting)
			{
			#if WITH_EDITOR
				if (GEditor)
				{
					GEditor->BeginTransaction(LOCTEXT("Vectorscope_Scale_Knob_Changed_Msg", "Set vectorscope Scale value."));
				}
			#endif
				bIsInputWidgetTransacting = true;
			}

			if (const float ScaleKnobValue = ScaleKnob->GetOutputValue(Value);
				ScaleKnobValue != ScaleValue)
			{
				ScaleValue = ScaleKnobValue;

				SequenceVectorViewer->SetScaleFactor(ScaleValue);
			}
		}
	};

	auto OnRadialSliderMouseCaptureBeginLambda = [this]()
	{
		if (!bIsInputWidgetTransacting)
		{
		#if WITH_EDITOR
			if (GEditor)
			{
				GEditor->BeginTransaction(LOCTEXT("Vectorscope_Scale_Knob_CaptureBegin_Msg", "Set vectorscope Scale value."));
			}
		#endif
			bIsInputWidgetTransacting = true;
		}
	};

	auto OnRadialSliderMouseCaptureEndLambda = [this]()
	{
		if (bIsInputWidgetTransacting)
		{
		#if WITH_EDITOR
			if (GEditor)
			{
				GEditor->EndTransaction();
			}
		#endif
			bIsInputWidgetTransacting = false;
		}
		else
		{
			UE_LOGF(LogAudioWidgets, Warning, "Unmatched vectorscope widget transaction.");
		}
	};

	ScaleKnob = SNew(SAudioRadialSlider)
	.OnMouseCaptureBegin_Lambda(OnRadialSliderMouseCaptureBeginLambda)
	.OnMouseCaptureEnd_Lambda(OnRadialSliderMouseCaptureEndLambda)
	.SliderValue(1.0f);

	ScaleKnob->SetOutputRange(ScaleFactorOutputKnobRange);
	ScaleKnob->SetShowUnitsText(false);

	ScaleKnob->OnValueChanged.BindLambda(OnValueChangedLambda);
}

void SAudioVectorscopePanelWidget::CreateVectorscopeControls()
{
	CreateDisplayPersistenceKnob();
	CreateScaleKnob();
}

void SAudioVectorscopePanelWidget::SetGridVisibility(const bool InbIsVisible)
{
	ValueGridOverlayXAxis->SetHideGrid(!InbIsVisible);
	ValueGridOverlayYAxis->SetHideGrid(!InbIsVisible);
}

void SAudioVectorscopePanelWidget::SetValueGridOverlayMaxNumDivisions(const uint32 InGridMaxNumDivisions)
{
	ValueGridOverlayXAxis->SetMaxDivisionParameter(InGridMaxNumDivisions);
	ValueGridOverlayYAxis->SetMaxDivisionParameter(InGridMaxNumDivisions);
}

void SAudioVectorscopePanelWidget::SetMaxDisplayPersistence(const float InMaxDisplayPersistenceInMs)
{
	DisplayPersistenceKnobOutputRange.Y = InMaxDisplayPersistenceInMs;

	if (DisplayPersistenceKnob.IsValid())
	{
		DisplayPersistenceKnob->SetOutputRange(DisplayPersistenceKnobOutputRange);
	}
}

void SAudioVectorscopePanelWidget::SetDisplayPersistence(const float InDisplayPersistenceInMs)
{
	if (DisplayPersistenceKnob.IsValid())
	{
		DisplayPersistenceKnob->SetSliderValue(FMath::GetMappedRangeValueUnclamped(DisplayPersistenceKnobOutputRange, { 0.0, 1.0 }, InDisplayPersistenceInMs));
	}
}

void SAudioVectorscopePanelWidget::SetVectorViewerScaleFactor(const float InScaleFactor)
{
	SequenceVectorViewer->SetScaleFactor(InScaleFactor);

	if (ScaleKnob.IsValid())
	{
		ScaleKnob->SetSliderValue(FMath::GetMappedRangeValueUnclamped(ScaleFactorOutputKnobRange, { 0.0, 1.0 }, InScaleFactor));
	}
}

void SAudioVectorscopePanelWidget::UpdateValueGridOverlayStyle(const FSampledSequenceValueGridOverlayStyle UpdatedStyle)
{
	ValueGridOverlayXAxis->OnStyleUpdated(UpdatedStyle);
	ValueGridOverlayYAxis->OnStyleUpdated(UpdatedStyle);
}

void SAudioVectorscopePanelWidget::UpdateSequenceVectorViewerStyle(const FSampledSequenceVectorViewerStyle UpdatedStyle)
{
	SequenceVectorViewer->OnStyleUpdated(UpdatedStyle);
	BackgroundBorder->SetBorderBackgroundColor(UpdatedStyle.BackgroundColor);
}

void SAudioVectorscopePanelWidget::ReceiveSequenceView(const FFixedSampledSequenceView InData, const uint32 FirstSampleIndex)
{
	if (SequenceVectorViewer)
	{
		SequenceVectorViewer->UpdateView(InData.SampleData, InData.NumDimensions);
	}
}

#undef LOCTEXT_NAMESPACE