// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFixedSampledSequenceRuler.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SparseSampledSequenceTransportCoordinator.h"

#define LOCTEXT_NAMESPACE "WaveformEditorTimeRuler"

void SFixedSampledSequenceRuler::Construct(const FArguments& InArgs, TSharedRef<IFixedSampledSequenceGridService> InGridService)
{
	GridService = InGridService;
	UpdateGridMetrics();
	DisplayUnit = InArgs._DisplayUnit;

	if (InArgs._DisplayPlayhead.IsSet())
	{
		bDisplayPlayhead = InArgs._DisplayPlayhead.Get();
	}

	check(InArgs._Style);

	HandleColor = InArgs._Style->HandleColor;
	HandleWidth = InArgs._Style->HandleWidth;
	HandleBrush = InArgs._Style->HandleBrush;
	BackgroundColor = InArgs._Style->BackgroundColor;
	BackgroundBrush = InArgs._Style->BackgroundBrush;
	TicksColor = InArgs._Style->TicksColor;
	TicksTextColor = InArgs._Style->TicksTextColor;
	TicksTextFont = InArgs._Style->TicksTextFont;
	DesiredWidth = InArgs._Style->DesiredWidth;
	DesiredHeight = InArgs._Style->DesiredHeight;
	TicksTextOffset = InArgs._Style->TicksTextOffset;
}

FVector2D SFixedSampledSequenceRuler::ComputeDesiredSize(float) const
{
	return FVector2D(DesiredWidth, DesiredHeight);
}

int32 SFixedSampledSequenceRuler::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		++LayerId,
		AllottedGeometry.ToPaintGeometry(),
		&BackgroundBrush,
		ESlateDrawEffect::None,
		BackgroundColor.GetSpecifiedColor());
	
	DrawRulerTicks(AllottedGeometry, OutDrawElements, LayerId);
	
	if (bDisplayPlayhead)
	{
		DrawPlayheadHandle(AllottedGeometry, OutDrawElements, LayerId);
	}
	
	return LayerId;
}

void SFixedSampledSequenceRuler::DrawRulerTicks(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32& LayerId) const
{
	const double MinorGridXStep = GridMetrics.MajorGridXStep / GridMetrics.NumMinorGridDivisions;
	const double MajorTickY = AllottedGeometry.Size.Y * 0.25;
	const double MinorTickY = AllottedGeometry.Size.Y * 0.75;
	
	TArray<FVector2D> LinePoints;
	LinePoints.SetNumUninitialized(2);
	
	for (double CurrentMajorTickX = GridMetrics.FirstMajorTickX; CurrentMajorTickX < AllottedGeometry.Size.X; CurrentMajorTickX += GridMetrics.MajorGridXStep)
	{
		const double MajorTickX = CurrentMajorTickX;
		LinePoints[0] = FVector2D(MajorTickX, MajorTickY);
		LinePoints[1] = FVector2D(MajorTickX, AllottedGeometry.Size.Y);
		
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			TicksColor.GetSpecifiedColor(),
			false);
		
		for (int32 MinorTickIndex = 1; MinorTickIndex < GridMetrics.NumMinorGridDivisions; ++MinorTickIndex)
		{
			const double MinorTickX = MajorTickX + MinorGridXStep * MinorTickIndex;

			if (MinorTickX < AllottedGeometry.Size.X)
			{
				LinePoints[0] = FVector2D(MinorTickX, MinorTickY);
				LinePoints[1] = FVector2D(MinorTickX, AllottedGeometry.Size.Y);

				FSlateDrawElement::MakeLines(
					OutDrawElements,
					++LayerId,
					AllottedGeometry.ToPaintGeometry(),
					LinePoints,
					ESlateDrawEffect::None,
					TicksColor.GetSpecifiedColor(),
					false);
			}
		}
		
		uint32 TickFrame = FMath::RoundToInt32(MajorTickX / GridMetrics.PixelsPerFrame) + GridMetrics.StartFrame;
		DrawTickTimeString(TickFrame, MajorTickX, MajorTickY, OutDrawElements, LayerId, AllottedGeometry);
	}
}

void SFixedSampledSequenceRuler::DrawTickTimeString(uint32 TickFrame, const double TickX, const double TickY, FSlateWindowElementList& OutDrawElements, int32& LayerId, const FGeometry& AllottedGeometry) const
{
	FString TimeString;
	
	switch (DisplayUnit)
	{
	case ESampledSequenceDisplayUnit::Samples:
	{
		uint32 SampleTime = TickFrame;
		TimeString = FString::Printf(TEXT("%d"), SampleTime);
	}
	break;
	case ESampledSequenceDisplayUnit::Seconds:
		TimeString = FString::Printf(TEXT("%.3f"), TickFrame / (float) GridMetrics.SampleRate);
		break;
	default:
		TimeString = FString::Printf(TEXT("%.3f"), TickFrame / (float)GridMetrics.SampleRate);
		break;
	}
	
	FVector2D TextOffset(TickX + TicksTextOffset, TickY);
	
	FSlateDrawElement::MakeText(
		OutDrawElements,
		++LayerId,
		AllottedGeometry.ToPaintGeometry(AllottedGeometry.Size, FSlateLayoutTransform(TextOffset)),
		TimeString,
		TicksTextFont,
		ESlateDrawEffect::None,
		TicksTextColor.GetSpecifiedColor()
	);
}

TSharedRef<SWidget> SFixedSampledSequenceRuler::MakeContextMenu()
{
	const bool bCloseAfterSelection = false;
	const bool bCloseSelfOnly = false;
	FMenuBuilder MenuBuilder(bCloseAfterSelection, NULL, TSharedPtr<FExtender>(), bCloseSelfOnly, &FCoreStyle::Get());
	MenuBuilder.BeginSection("Ruler Options", LOCTEXT("RulerOptionsHeading", "Options"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("TimeUnitsSubMenuLabel", "Time Unit"),
			LOCTEXT("TimeUnitsSubMenuLabelToolTip", "Select the time unit shown by the grid"),
			FNewMenuDelegate::CreateSP(this, &SFixedSampledSequenceRuler::MakeTimeUnitsSubMenu)
		);
	}
	MenuBuilder.EndSection();
	return MenuBuilder.MakeWidget();
}

void SFixedSampledSequenceRuler::MakeTimeUnitsSubMenu(FMenuBuilder& SubMenuBuilder)
{
	SubMenuBuilder.BeginSection(NAME_None);

	SubMenuBuilder.AddMenuEntry(
		LOCTEXT("TimeUnitsSubMenuEntry_Seconds", "Seconds"),
		LOCTEXT("TimeUnitsSubMenuEntry_Seconds", "Seconds"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SFixedSampledSequenceRuler::NotifyTimeUnitMenuSelection, ESampledSequenceDisplayUnit::Seconds),
			FCanExecuteAction::CreateLambda([this] { return DisplayUnit != ESampledSequenceDisplayUnit::Seconds; }),
			FIsActionChecked::CreateLambda([this] { return DisplayUnit == ESampledSequenceDisplayUnit::Seconds;})
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	SubMenuBuilder.AddMenuEntry(
		LOCTEXT("TimeUnitsSubMenuEntry_Frames", "Frames"),
		LOCTEXT("TimeUnitsSubMenuEntry_Frames", "Frames"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SFixedSampledSequenceRuler::NotifyTimeUnitMenuSelection, ESampledSequenceDisplayUnit::Samples),
			FCanExecuteAction::CreateLambda([this] { return DisplayUnit != ESampledSequenceDisplayUnit::Samples; }),
			FIsActionChecked::CreateLambda([this] { return DisplayUnit == ESampledSequenceDisplayUnit::Samples; })
		),
		NAME_None,
		EUserInterfaceActionType::ToggleButton
	);

	SubMenuBuilder.EndSection();
 }

void SFixedSampledSequenceRuler::NotifyTimeUnitMenuSelection(const ESampledSequenceDisplayUnit SelectedDisplayUnit) const
{
	if (SelectedDisplayUnit == DisplayUnit)
	{
		return;
	}

	if (OnTimeUnitMenuSelection.IsBound())
	{
		OnTimeUnitMenuSelection.Broadcast(SelectedDisplayUnit);
	}
}

void SFixedSampledSequenceRuler::UpdateGridMetrics()
{
	check(GridService)
	GridMetrics = GridService->GetGridMetrics();
}

void SFixedSampledSequenceRuler::UpdateDisplayUnit(const ESampledSequenceDisplayUnit InDisplayUnit)
{
	DisplayUnit = InDisplayUnit;
}

void SFixedSampledSequenceRuler::SetPlayheadPosition(const float InNewPosition)
{
	PlayheadPosition = InNewPosition;
}

void SFixedSampledSequenceRuler::OnStyleUpdated(const FFixedSampleSequenceRulerStyle UpdatedStyle)
{
	HandleColor = UpdatedStyle.HandleColor;
	HandleWidth = UpdatedStyle.HandleWidth;
	HandleBrush = UpdatedStyle.HandleBrush;
	BackgroundColor = UpdatedStyle.BackgroundColor;
	BackgroundBrush = UpdatedStyle.BackgroundBrush;
	TicksColor = UpdatedStyle.TicksColor;
	TicksTextColor = UpdatedStyle.TicksTextColor;
	TicksTextFont = UpdatedStyle.TicksTextFont;
	DesiredWidth = UpdatedStyle.DesiredWidth;
	DesiredHeight = UpdatedStyle.DesiredHeight;
	TicksTextOffset = UpdatedStyle.TicksTextOffset;
}

FReply SFixedSampledSequenceRuler::LaunchContextMenu()
{
	FSlateApplication::Get().PushMenu(AsShared(), FWidgetPath(), MakeContextMenu(), FSlateApplication::Get().GetCursorPos(), FPopupTransitionEffect::ContextMenu);
	return FReply::Handled();
}

void SFixedSampledSequenceRuler::SetPlayheadVisible(bool Enable)
{
	bDisplayPlayhead = Enable;
}

void SFixedSampledSequenceRuler::DrawPlayheadHandle(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32& LayerId) const
{
	const float WindowWidth = AllottedGeometry.Size.X;
	const float PlayheadCenter = PlayheadPosition;
	const float	HandleStart = PlayheadCenter - HandleWidth / 2;
	const float	HandleEnd = HandleStart + HandleWidth;
 	FPaintGeometry HandleGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(HandleEnd - HandleStart, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2f(HandleStart, 0)));
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		HandleGeometry,
		&HandleBrush,
		ESlateDrawEffect::None,
		HandleColor.GetSpecifiedColor()
	);
	++LayerId;
}

#undef LOCTEXT_NAMESPACE