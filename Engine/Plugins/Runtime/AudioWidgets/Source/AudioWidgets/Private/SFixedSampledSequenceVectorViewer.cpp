// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFixedSampledSequenceVectorViewer.h"

#include "AudioWidgetsSlateTypes.h"

#define LOCTEXT_NAMESPACE "OscilloscopeWidget"

void SFixedSampledSequenceVectorViewer::Construct(const FArguments& InArgs, TArrayView<const float> InSampleData, const uint8 InNumChannels)
{
	UpdateView(InSampleData, InNumChannels);
	
	check(InArgs._Style);

	DrawingParams = InArgs._SequenceDrawingParams;

	LineColor     = InArgs._Style->LineColor;
	LineThickness = InArgs._Style->LineThickness;

	ScaleFactor = InArgs._ScaleFactor;
}

void SFixedSampledSequenceVectorViewer::UpdateView(TArrayView<const float> InSampleData, const uint8 InNumChannels)
{
	SampleData   = InSampleData;
	NumChannels  = InNumChannels;
	bForceRedraw = true;
}

FVector2D SFixedSampledSequenceVectorViewer::ComputeDesiredSize(float) const
{
	return FVector2D(DesiredWidth, DesiredHeight);
}

int32 SFixedSampledSequenceVectorViewer::OnPaint(const FPaintArgs& Args,
	const FGeometry& AllottedGeometry, 
	const FSlateRect& MyCullingRect, 
	FSlateWindowElementList& OutDrawElements, 
	int32 LayerId, 
	const FWidgetStyle& InWidgetStyle, 
	bool bParentEnabled) const
{
	check(NumChannels > 0);

	TArray<FVector2D> PointsToDraw;

	PointsToDraw.Reserve(SampleData.Num() / NumChannels);

	const float MaxScaleFactor   = DrawingParams.MaxSequenceHeightRatio * 0.5f;
	const float FinalScaleFactor = ScaleFactor * MaxScaleFactor;

	const FScale2D Scale(AllottedGeometry.GetLocalSize() * FVector2D(FinalScaleFactor, -FinalScaleFactor));
	const FVector2D Translation(AllottedGeometry.GetLocalSize().X / 2.0, AllottedGeometry.GetLocalSize().Y / 2.0);

	const FTransform2D PointsTransform(Scale, Translation);

	if (NumChannels == 1)
	{
		for (int32 Index = 0; Index < SampleData.Num(); ++Index)
		{
			PointsToDraw.Add(PointsTransform.TransformPoint(FVector2D(SampleData[Index], 0.0)));
		}
	}
	else
	{
		// For more than 2 channels it will only take the first 2 channels of audio data
		for (int32 Index = 0; Index < SampleData.Num() - NumChannels + 1; Index += NumChannels)
		{
			PointsToDraw.Add(PointsTransform.TransformPoint(FVector2D(SampleData[Index], SampleData[Index + 1])));
		}
	}

	FSlateDrawElement::MakeLines(OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(),
		PointsToDraw,
		ESlateDrawEffect::None,
		LineColor,
		true,
		LineThickness);

	++LayerId;

	return LayerId;
}

void SFixedSampledSequenceVectorViewer::OnStyleUpdated(const FSampledSequenceVectorViewerStyle UpdatedStyle)
{
	LineColor     = UpdatedStyle.LineColor;
	LineThickness = UpdatedStyle.LineThickness;
}

void SFixedSampledSequenceVectorViewer::SetScaleFactor(const float InScaleFactor)
{
	ScaleFactor = InScaleFactor;
}

#undef LOCTEXT_NAMESPACE
