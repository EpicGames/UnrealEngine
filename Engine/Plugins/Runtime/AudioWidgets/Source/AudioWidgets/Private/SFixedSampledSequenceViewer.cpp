// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFixedSampledSequenceViewer.h"

void SFixedSampledSequenceViewer::Construct(const FArguments& InArgs, TArrayView<const float> InSampleData, const uint8 InNumChannels, TSharedRef<IFixedSampledSequenceGridService> InGridService)
{
	GridService = InGridService;
	UpdateView(InSampleData, InNumChannels);

	DrawingParams = InArgs._SequenceDrawingParams;
	bHideBackground = InArgs._HideBackground;
	bHideGrid = InArgs._HideGrid;

	check(InArgs._Style);
	SequenceColor = InArgs._Style->SequenceColor;
	MajorGridLineColor = InArgs._Style->MajorGridLineColor;
	MinorGridLineColor = InArgs._Style->MinorGridLineColor;
	BackgroundColor = InArgs._Style->SequenceBackgroundColor;
	BackgroundBrush = InArgs._Style->BackgroundBrush;
	DesiredWidth = InArgs._Style->DesiredWidth;
	DesiredHeight = InArgs._Style->DesiredHeight;
	SampleMarkersSize = InArgs._Style->SampleMarkersSize;
	SequenceLineThickness = InArgs._Style->SequenceLineThickness;
	ZeroCrossingLineColor = InArgs._Style->ZeroCrossingLineColor;
	ZeroCrossingLineThickness = InArgs._Style->ZeroCrossingLineThickness;
}

void SFixedSampledSequenceViewer::UpdateView(TArrayView<const float> InSampleData, const uint8 InNumChannels)
{
	UpdateGridMetrics();

	SampleData = InSampleData;
	NumChannels = InNumChannels;

	bForceRedraw = true;
}

int32 SFixedSampledSequenceViewer::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const 
{
	float PixelWidth = MyCullingRect.GetSize().X;

	if (!bHideBackground)
	{
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(),
			&BackgroundBrush,
			ESlateDrawEffect::None,
			BackgroundColor.GetSpecifiedColor()
		);
	}


	if (PixelWidth > 0)
	{
		if (!bHideGrid)
		{
			DrawGridLines(AllottedGeometry, OutDrawElements, LayerId);
		}
		
		if (SequenceDrawMode == ESequenceDrawMode::BinnedPeaks)
		{
			TArray<FVector2D> BinDrawPoints;
			BinDrawPoints.SetNumUninitialized(2);

			for (int32 i = 0; i < CachedBinsDrawCoordinates.Num(); ++i)
			{
				BinDrawPoints[0] = CachedBinsDrawCoordinates[i].A;
				BinDrawPoints[1] = CachedBinsDrawCoordinates[i].B;

				FSlateDrawElement::MakeLines(
					OutDrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(),
					BinDrawPoints,
					ESlateDrawEffect::None,
					SequenceColor.GetSpecifiedColor(),
					true,
					SequenceLineThickness
				);

				const int32 NextBinCoordinates = i + NumChannels;

				if (NextBinCoordinates < CachedBinsDrawCoordinates.Num())
				{
					BinDrawPoints[0] = CachedBinsDrawCoordinates[i].B;
					BinDrawPoints[1] = CachedBinsDrawCoordinates[NextBinCoordinates].A;

					FSlateDrawElement::MakeLines(
						OutDrawElements,
						LayerId,
						AllottedGeometry.ToPaintGeometry(),
						BinDrawPoints,
						ESlateDrawEffect::None,
						SequenceColor.GetSpecifiedColor(),
						true,
						SequenceLineThickness
					);

				}
			}

		}
		else if (SequenceDrawMode == ESequenceDrawMode::SequenceLine)
		{
			TArray<FVector2D> SamplesDrawCoordinates;
			const uint16 NChannels = NumChannels;

			for (uint16 Channel = 0; Channel < NChannels; ++Channel)
			{
				for (int32 SampleIndex = Channel; SampleIndex < CachedSampleDrawCoordinates.Num(); SampleIndex += NChannels)
				{
					SamplesDrawCoordinates.Add(CachedSampleDrawCoordinates[SampleIndex]);
					
					const FVector2D SampleBoxSize = FVector2D(SampleMarkersSize, SampleMarkersSize);
					const FVector2D SampleBoxTransform = FVector2D(CachedSampleDrawCoordinates[SampleIndex] - (SampleBoxSize / 2.f));

					const FPaintGeometry SampleBoxGeometry = AllottedGeometry.ToPaintGeometry(
						SampleBoxSize,
						FSlateLayoutTransform(SampleBoxTransform)
					);

					FSlateDrawElement::MakeBox(
						OutDrawElements,
						LayerId,
						SampleBoxGeometry,
						&BackgroundBrush,
						ESlateDrawEffect::None,
						SequenceColor.GetSpecifiedColor()
					);
				}

				FSlateDrawElement::MakeLines(
					OutDrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(),
					SamplesDrawCoordinates,
					ESlateDrawEffect::None,
					SequenceColor.GetSpecifiedColor(),
					true,
					SequenceLineThickness
				);

				SamplesDrawCoordinates.Empty();
			}
		}

		++LayerId;
	}
	
	return LayerId;
}

FVector2D SFixedSampledSequenceViewer::ComputeDesiredSize(float) const
{
	return FVector2D(DesiredWidth, DesiredHeight);
}

void SFixedSampledSequenceViewer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	uint32 DiscretePixelWidth = FMath::FloorToInt(AllottedGeometry.GetLocalSize().X);
	if (DiscretePixelWidth <= 0)
	{
		return;
	}

	if (DiscretePixelWidth != CachedPixelWidth || bForceRedraw)
	{
		CachedPixelWidth = DiscretePixelWidth;
		CachedPixelHeight = AllottedGeometry.GetLocalSize().Y;
		const float NumSamplesToDisplay = SampleData.Num() / (float) NumChannels;
		SequenceDrawMode = NumSamplesToDisplay > CachedPixelWidth ? ESequenceDrawMode::BinnedPeaks : ESequenceDrawMode::SequenceLine;

		if (SequenceDrawMode == ESequenceDrawMode::BinnedPeaks)
		{
			SampledSequenceDrawingUtils::GroupInterleavedSampledTSIntoMinMaxBins<float>(CachedPeaks, CachedPixelWidth, SampleData.GetData(), SampleData.Num(), GridMetrics.SampleRate, NumChannels);
			SampledSequenceDrawingUtils::GenerateSampleBinsCoordinatesForGeometry(CachedBinsDrawCoordinates, AllottedGeometry, CachedPeaks, NumChannels, DrawingParams);
		}
		else
		{
			SampledSequenceDrawingUtils::GenerateSequencedSamplesCoordinatesForGeometry(CachedSampleDrawCoordinates, SampleData, AllottedGeometry, NumChannels, GridMetrics, DrawingParams);
		}
		
		bForceRedraw = false;
	}
	else if (CachedPixelHeight != AllottedGeometry.GetLocalSize().Y)
	{
		CachedPixelHeight = AllottedGeometry.GetLocalSize().Y;

		if (SequenceDrawMode == ESequenceDrawMode::BinnedPeaks)
		{
			SampledSequenceDrawingUtils::GenerateSampleBinsCoordinatesForGeometry(CachedBinsDrawCoordinates, AllottedGeometry, CachedPeaks, NumChannels, DrawingParams);
		}
		else
		{
			SampledSequenceDrawingUtils::GenerateSequencedSamplesCoordinatesForGeometry(CachedSampleDrawCoordinates, SampleData, AllottedGeometry, NumChannels, GridMetrics, DrawingParams);
		}
	}
}

void SFixedSampledSequenceViewer::DrawGridLines(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32& LayerId) const
{
	TArray<FVector2D> LinePoints;
	LinePoints.SetNumUninitialized(2);

	const double MinorGridXStep = GridMetrics.MajorGridXStep / GridMetrics.NumMinorGridDivisions;

	for (double CurrentMajorLineX = GridMetrics.FirstMajorTickX; CurrentMajorLineX < AllottedGeometry.Size.X; CurrentMajorLineX += GridMetrics.MajorGridXStep)
	{
		const double MajorLineX = CurrentMajorLineX;

		LinePoints[0] = FVector2D(MajorLineX, 0.f);
		LinePoints[1] = FVector2D(MajorLineX, AllottedGeometry.Size.Y);

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			MajorGridLineColor.GetSpecifiedColor(),
			false);

		for (int32 MinorLineIndex = 1; MinorLineIndex < GridMetrics.NumMinorGridDivisions; ++MinorLineIndex)
		{
			const double MinorLineX = MajorLineX + MinorGridXStep * MinorLineIndex;

			if (MinorLineX < AllottedGeometry.Size.X)
			{
				LinePoints[0] = FVector2D(MinorLineX, 0.);
				LinePoints[1] = FVector2D(MinorLineX, AllottedGeometry.Size.Y);

				FSlateDrawElement::MakeLines(
					OutDrawElements,
					++LayerId,
					AllottedGeometry.ToPaintGeometry(),
					LinePoints,
					ESlateDrawEffect::None,
					MinorGridLineColor.GetSpecifiedColor(),
					false);
			}
		}
	}

	const uint16 NChannels = NumChannels;

	for (uint16 Channel = 0; Channel < NChannels; ++Channel)
	{
		const SampledSequenceDrawingUtils::FDimensionSlot ChannelBoundaries(Channel, NChannels, AllottedGeometry, DrawingParams);

		LinePoints[0] = FVector2D(0.f, ChannelBoundaries.Center);
		LinePoints[1] = FVector2D(AllottedGeometry.Size.X, ChannelBoundaries.Center);

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			ZeroCrossingLineColor.GetSpecifiedColor(),
			false,
			ZeroCrossingLineThickness);

	}
}

void SFixedSampledSequenceViewer::UpdateGridMetrics()
{
	check(GridService)
	GridMetrics = GridService->GetGridMetrics();
}

void SFixedSampledSequenceViewer::SetHideGrid(const bool InHideGrid)
{
	bHideGrid = InHideGrid;
}

void SFixedSampledSequenceViewer::OnStyleUpdated(const FSampledSequenceViewerStyle UpdatedStyle)
{
	SequenceColor = UpdatedStyle.SequenceColor;
	MajorGridLineColor = UpdatedStyle.MajorGridLineColor;
	MinorGridLineColor = UpdatedStyle.MinorGridLineColor;
	ZeroCrossingLineColor = UpdatedStyle.ZeroCrossingLineColor;
	ZeroCrossingLineThickness = UpdatedStyle.ZeroCrossingLineThickness;
	BackgroundColor = UpdatedStyle.SequenceBackgroundColor;
	BackgroundBrush = UpdatedStyle.BackgroundBrush;
	SampleMarkersSize = UpdatedStyle.SampleMarkersSize;
	SequenceLineThickness = UpdatedStyle.SequenceLineThickness;
	DesiredWidth = UpdatedStyle.DesiredWidth;
	DesiredHeight = UpdatedStyle.DesiredHeight;
}