// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFixedSampledSequenceViewerPanel.h"

#include "FixedSampledSequenceGridData.h"
#include "SFixedSampledSequenceRuler.h"
#include "SFixedSampledSequenceViewer.h"
#include "SparseSampledSequenceTransportCoordinator.h"
#include "SPlayheadOverlay.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"


void SFixedSampledSequenceViewerPanel::Construct(const FArguments& InArgs, const FFixedSampledSequenceView& InData)
{
	DataView = InData;
	
	DrawingParams = InArgs._SequenceDrawingParams;
	SamplesRulerDisplayUnit = InArgs._SamplesRulerDisplayUnit;
	bHidePlayhead = InArgs._HidePlayhead.Get();
	bHideSamplesRuler = InArgs._HideSamplesRuler.Get();
	bHideSamplesGrid = InArgs._HideSamplesGrid.Get();
	bHideValueGrid = InArgs._HideValueGrid.Get();

	CreateBackground(InArgs._SequenceViewerStyle);
	CreateGridData(InArgs._SamplesRulerStyle);
	CreateSequenceViewer(SamplesGridData.ToSharedRef(), DataView, InArgs._SequenceViewerStyle);
	CreatePlayheadOverlay(InArgs._PlayheadOverlayStyle);
	CreateSamplesRuler(SamplesGridData.ToSharedRef(), InArgs._SamplesRulerStyle);
	CreateValueGridOverlay(InArgs._ValueGridMaxDivisionParameter, InArgs._ValueGridDivideMode, InArgs._ValueGridLabelGenerator, InArgs._ValueGridStyle);

	CreateLayout();
}

void SFixedSampledSequenceViewerPanel::CreateLayout()
{
	check(SamplesRuler);
	check(SequenceViewer);
	check(PlayheadOverlay);
	check(ValueGridOverlay);
	check(BackgroundBorder);

	TSharedPtr<SVerticalBox> MainBox = SNew(SVerticalBox);

	if (!bHideSamplesRuler)
	{
		MainBox->AddSlot().AutoHeight()
		[
			SamplesRuler.ToSharedRef()
		];
	}

	TSharedPtr<SOverlay> SequenceView = SNew(SOverlay);

	SequenceView->AddSlot()
	[
		BackgroundBorder.ToSharedRef()
	];

	if (!bHideValueGrid)
	{
		SequenceView->AddSlot()
		[
			ValueGridOverlay.ToSharedRef()
		];
	}

	SequenceView->AddSlot()
	[
		SequenceViewer.ToSharedRef()
	];

	if (!bHidePlayhead)
	{
		SequenceView->AddSlot()
		[
			PlayheadOverlay.ToSharedRef()
		];
	}

	MainBox->AddSlot()
	[
		SequenceView.ToSharedRef()
	];
	

	ChildSlot
	[
		MainBox.ToSharedRef()
	];
}

void SFixedSampledSequenceViewerPanel::CreateSamplesRuler(TSharedRef<FFixedSampledSequenceGridData> InGridData, const FFixedSampleSequenceRulerStyle* RulerStyle)
{
	check(RulerStyle)
	SamplesRuler = SNew(SFixedSampledSequenceRuler, InGridData).DisplayUnit(SamplesRulerDisplayUnit).Style(RulerStyle).DisplayPlayhead(!bHidePlayhead);
	SamplesGridData->OnGridMetricsUpdated.AddSP(SamplesRuler.ToSharedRef(), &SFixedSampledSequenceRuler::UpdateGridMetrics);
	SamplesRuler->OnTimeUnitMenuSelection.AddSP(this, &SFixedSampledSequenceViewerPanel::UpdateSamplesRulerDisplayUnit);
}


void SFixedSampledSequenceViewerPanel::CreatePlayheadOverlay(const FPlayheadOverlayStyle* PlayheadOverlayStyle)
{
	check(PlayheadOverlayStyle)
	PlayheadOverlay = SNew(SPlayheadOverlay).Style(PlayheadOverlayStyle);
}

void SFixedSampledSequenceViewerPanel::CreateValueGridOverlay(const uint32 MaxDivisionParameter, const SampledSequenceValueGridOverlay::EGridDivideMode DivideMode, const TFunction<FText(const double)>& LabelGenerator, const FSampledSequenceValueGridOverlayStyle* ValueGridStyle)
{
	check(ValueGridStyle)

	ValueGridOverlay = SNew(SSampledSequenceValueGridOverlay)
	.MaxDivisionParameter(MaxDivisionParameter)
	.DivideMode(DivideMode)
	.ValueGridLabelGenerator(LabelGenerator)
	.Style(ValueGridStyle)
	.SequenceDrawingParams(DrawingParams)
	.NumDimensions(DataView.NumDimensions);
	
}

void SFixedSampledSequenceViewerPanel::CreateSequenceViewer(TSharedRef<FFixedSampledSequenceGridData> InGridData, const FFixedSampledSequenceView& InData, const FSampledSequenceViewerStyle* ViewerStyle)
{
	check(ViewerStyle)

	SequenceViewer = SNew(SFixedSampledSequenceViewer, InData.SampleData, InData.NumDimensions, InGridData)
		.Style(ViewerStyle)
		.SequenceDrawingParams(DrawingParams)
		.HideBackground(true)
		.HideGrid(bHideSamplesGrid);

	SamplesGridData->OnGridMetricsUpdated.AddSP(SequenceViewer.ToSharedRef(), &SFixedSampledSequenceViewer::UpdateGridMetrics);
}

void SFixedSampledSequenceViewerPanel::CreateBackground(const FSampledSequenceViewerStyle* ViewerStyle)
{
	check(ViewerStyle)

	BackgroundBorder = SNew(SBorder)
		.BorderImage(&ViewerStyle->BackgroundBrush)
		.BorderBackgroundColor(ViewerStyle->SequenceBackgroundColor);
}

void SFixedSampledSequenceViewerPanel::CreateGridData(const FFixedSampleSequenceRulerStyle* RulerStyle)
{
	check(RulerStyle)

	SamplesGridData = MakeShared<FFixedSampledSequenceGridData>(DataView.SampleData.Num() / DataView.NumDimensions, DataView.SampleRate, RulerStyle->TicksTextFont, RulerStyle->DesiredWidth);
}

void SFixedSampledSequenceViewerPanel::ReceiveSequenceView(const FFixedSampledSequenceView InData, const uint32 FirstSampleIndex)
{
	if (SamplesGridData)
	{
		const uint32 FirstRenderedFrame = FirstSampleIndex / InData.NumDimensions;
		const uint32 NumFrames = InData.SampleData.Num() / InData.NumDimensions;
		SamplesGridData->UpdateDisplayRange(TRange<uint32>(FirstRenderedFrame, FirstRenderedFrame + NumFrames));
	}

	if (SequenceViewer)
	{
		SequenceViewer->UpdateView(InData.SampleData, InData.NumDimensions);
	}
}

void SFixedSampledSequenceViewerPanel::SetPlayheadRatio(const float InRatio)
{
	CachedPlayheadRatio = InRatio;
}

void SFixedSampledSequenceViewerPanel::UpdateSequenceViewerStyle(const FSampledSequenceViewerStyle UpdatedStyle)
{
	check(SequenceViewer);
	SequenceViewer->OnStyleUpdated(UpdatedStyle);
}

void SFixedSampledSequenceViewerPanel::UpdateRulerStyle(const FFixedSampleSequenceRulerStyle UpdatedStyle)
{
	check(SamplesRuler);
	SamplesRuler->OnStyleUpdated(UpdatedStyle);
}

void SFixedSampledSequenceViewerPanel::UpdatePlayheadOverlayStyle(const FPlayheadOverlayStyle UpdatedStyle)
{
	check(PlayheadOverlay);
	PlayheadOverlay->OnStyleUpdated(UpdatedStyle);
}

void SFixedSampledSequenceViewerPanel::UpdateValueGridOverlayStyle(const FSampledSequenceValueGridOverlayStyle UpdatedStyle)
{
	check(ValueGridOverlay);
	ValueGridOverlay->OnStyleUpdated(UpdatedStyle);
}

void SFixedSampledSequenceViewerPanel::UpdateSamplesRulerDisplayUnit(const ESampledSequenceDisplayUnit InDisplayUnit)
{
	SamplesRulerDisplayUnit = InDisplayUnit;
	SamplesRuler->UpdateDisplayUnit(SamplesRulerDisplayUnit);
}

void SFixedSampledSequenceViewerPanel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	const float PaintedWidth = AllottedGeometry.GetLocalSize().X;

	if (PaintedWidth != CachedPixelWidth)
	{
		CachedPixelWidth = PaintedWidth;

		if (SamplesGridData)
		{
			SamplesGridData->UpdateGridMetrics(PaintedWidth);
		}
	}

	UpdatePlayheadPosition(PaintedWidth);
}

void SFixedSampledSequenceViewerPanel::UpdatePlayheadPosition(const float PaintedWidth)
{
	float PlayheadX = CachedPlayheadRatio * PaintedWidth;
	PlayheadX = SamplesGridData ? SamplesGridData->SnapPositionToClosestFrame(PlayheadX) : PlayheadX;


	if (!bHidePlayhead && PlayheadOverlay)
	{
		PlayheadOverlay->SetPlayheadPosition(PlayheadX);
	}

	if (!bHideSamplesRuler && SamplesRuler)
	{
		SamplesRuler->SetPlayheadPosition(PlayheadX);
	}
}
