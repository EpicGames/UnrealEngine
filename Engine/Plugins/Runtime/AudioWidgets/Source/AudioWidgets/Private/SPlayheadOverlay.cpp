// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlayheadOverlay.h"

void SPlayheadOverlay::Construct(const FArguments& InArgs)
{
	check(InArgs._Style);
	PlayheadColor = InArgs._Style->PlayheadColor;
	PlayheadWidth = InArgs._Style->PlayheadWidth;
	DesiredWidth = InArgs._Style->DesiredWidth;
	DesiredHeight = InArgs._Style->DesiredHeight;
	bIsVisible = true;
}

void SPlayheadOverlay::SetPlayheadPosition(const float InNewPosition)
{
	PlayheadPosition = InNewPosition;
}

void SPlayheadOverlay::OnStyleUpdated(const FPlayheadOverlayStyle UpdatedStyle)
{
	PlayheadColor = UpdatedStyle.PlayheadColor;
	PlayheadWidth = UpdatedStyle.PlayheadWidth;
	DesiredWidth = UpdatedStyle.DesiredWidth;
	DesiredHeight = UpdatedStyle.DesiredHeight;
}

void SPlayheadOverlay::SetPlayheadVisible(bool Enable)
{
	bIsVisible = Enable;
}

int32 SPlayheadOverlay::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	if (bIsVisible)
	{
		LayerId = DrawPlayhead(AllottedGeometry, OutDrawElements, LayerId);
	}

	return LayerId;
}

int32 SPlayheadOverlay::DrawPlayhead(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	const float PlayHeadX = PlayheadPosition;

	TArray<FVector2D> LinePoints{ FVector2D(PlayHeadX, 0.0f), FVector2D(PlayHeadX, AllottedGeometry.Size.Y) };
	

	FSlateDrawElement::MakeLines(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(),
		LinePoints,
		ESlateDrawEffect::None,
		PlayheadColor.GetSpecifiedColor(),
		true,
		PlayheadWidth
	);

	return ++LayerId;
}

FVector2D SPlayheadOverlay::ComputeDesiredSize(float) const
{
	return FVector2D(DesiredWidth, DesiredHeight);
}