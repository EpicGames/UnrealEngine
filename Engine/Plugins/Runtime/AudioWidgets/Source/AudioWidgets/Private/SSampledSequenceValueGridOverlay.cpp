// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSampledSequenceValueGridOverlay.h"

#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"

namespace SampledSequenceValueGridOverlayPrivate
{
	TFunction<FText(const double)> DefaultLabelGenerator = [](const double GridLineValue)
	{
		FNumberFormattingOptions DefaultFormatting;
		return FText::AsNumber(GridLineValue, &DefaultFormatting);
	};
}

void SSampledSequenceValueGridOverlay::Construct(const FArguments& InArgs)
{
	check(InArgs._Style);

	GridSplitMode = InArgs._DivideMode;
	MaxDivisionParameter = InArgs._MaxDivisionParameter;
	OnValueGridLabel = InArgs._ValueGridLabelGenerator ? InArgs._ValueGridLabelGenerator : SampledSequenceValueGridOverlayPrivate::DefaultLabelGenerator;
	NumDimensions = InArgs._NumDimensions;
	DrawingParams = InArgs._SequenceDrawingParams;
	bHideLabels = InArgs._HideLabels;
	bHideGrid = InArgs._HideGrid;
	GridColor = InArgs._Style->GridColor;
	LabelTextFont = InArgs._Style->LabelTextFont;
	LabelTextColor = InArgs._Style->LabelTextColor;
	DesiredWidth = InArgs._Style->DesiredWidth;
	DesiredHeight = InArgs._Style->DesiredHeight;
	GridThickness = InArgs._Style->GridThickness;
}

int32 SSampledSequenceValueGridOverlay::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	if (bHideGrid && bHideLabels)
	{
		return LayerId;
	}

	TArray<FVector2D> LinePoints;
	LinePoints.SetNumUninitialized(2);
	TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	for (int32 SlotIndex = 0; SlotIndex < CachedGridSlotData.Num(); ++SlotIndex)
	{
		const SampledSequenceDrawingUtils::FGridData& SlotData = CachedGridSlotData[SlotIndex];
		
		for (int32 LineIndex = 0; LineIndex < SlotData.DrawCoordinates.Num(); ++LineIndex)
		{
			LinePoints[0] = SlotData.DrawCoordinates[LineIndex].A;
			LinePoints[1] = SlotData.DrawCoordinates[LineIndex].B;

			if (!bHideLabels)
			{
				const FGridLabelData& LabelData = CachedLabelData[SlotIndex];
				const FVector2D LabelSize = FontMeasureService->Measure(LabelData.LabelTexts[LineIndex], FSlateFontInfo(FAppStyle::GetFontStyle("Regular")));
				const FSlateLayoutTransform LabelLayoutTransform(1.0f, LabelData.LabelCoordinates[LineIndex]);
				float LabelRotation = 0.f;

				switch (DrawingParams.Orientation)
				{
				case SampledSequenceDrawingUtils::ESampledSequenceDrawOrientation::Horizontal:
					LabelRotation = 0.f;
					break;
				case SampledSequenceDrawingUtils::ESampledSequenceDrawOrientation::Vertical:
					LabelRotation = 90.f;
				default:
					static_assert(static_cast<int32>(SampledSequenceDrawingUtils::ESampledSequenceDrawOrientation::COUNT) == 2, "Possible missing switch case coverage for 'ESampledSequenceDrawOrientation'");
					break;
				}
				
				const FSlateRenderTransform LabelRotationTransform = FSlateRenderTransform(FQuat2D(FMath::DegreesToRadians(LabelRotation)));


				const FGeometry LabelGeometry = AllottedGeometry.MakeChild(
					LabelSize,
					LabelLayoutTransform,
					LabelRotationTransform,
					FVector2D(0.0f, 0.0f)
				);

				FSlateDrawElement::MakeText(
					OutDrawElements,
					LayerId,
					LabelGeometry.ToPaintGeometry(),
					LabelData.LabelTexts[LineIndex],
					LabelTextFont,
					ESlateDrawEffect::None,
					LabelTextColor.GetSpecifiedColor()
				);
			}

			if (!bHideGrid)
			{
				FSlateDrawElement::MakeLines(
					OutDrawElements,
					LayerId,
					AllottedGeometry.ToPaintGeometry(),
					LinePoints,
					ESlateDrawEffect::None,
					GridColor.GetSpecifiedColor(),
					false,
					GridThickness
				);
			}
		}
	}

	return ++LayerId;
}

FVector2D SSampledSequenceValueGridOverlay::ComputeDesiredSize(float) const
{
	return FVector2D(DesiredWidth, DesiredHeight);
}

void SSampledSequenceValueGridOverlay::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (AllottedGeometry.GetLocalSize() != CachedLocalSize || bForceRedraw)
	{
		CachedLocalSize = AllottedGeometry.GetLocalSize();
		CacheDrawElements(AllottedGeometry, MaxDivisionParameter);
		bForceRedraw = false;
	}
}

void SSampledSequenceValueGridOverlay::OnStyleUpdated(const FSampledSequenceValueGridOverlayStyle UpdatedStyle)
{
	GridColor = UpdatedStyle.GridColor;
	LabelTextFont = UpdatedStyle.LabelTextFont;
	LabelTextColor = UpdatedStyle.LabelTextColor;
	DesiredWidth = UpdatedStyle.DesiredWidth;
	DesiredHeight = UpdatedStyle.DesiredHeight;
	GridThickness = UpdatedStyle.GridThickness;

	ForceRedraw();
}

void SSampledSequenceValueGridOverlay::SetLabelGenerator(TFunction<FText(const double)> InLabelGenerator)
{
	OnValueGridLabel = InLabelGenerator;
	ForceRedraw();
}

void SSampledSequenceValueGridOverlay::SetMaxDivisionParameter(const uint32 InDivisionParameter)
{
	MaxDivisionParameter = InDivisionParameter;
	ForceRedraw();
}

void SSampledSequenceValueGridOverlay::SetHideLabels(const bool InHideLabels)
{
	bHideLabels = InHideLabels;
	ForceRedraw();
}

void SSampledSequenceValueGridOverlay::SetHideGrid(const bool InHideGrid)
{
	bHideGrid = InHideGrid;
	ForceRedraw();
}

void SSampledSequenceValueGridOverlay::ForceRedraw() 
{
	bForceRedraw = true;
}

void SSampledSequenceValueGridOverlay::CacheDrawElements(const FGeometry& AllottedGeometry, const uint32 InDivisionParameter)
{
	switch (GridSplitMode)
	{
	case SampledSequenceValueGridOverlay::EGridDivideMode::EvenSplit:
		SampledSequenceDrawingUtils::GenerateEvenlySplitGridForGeometry(CachedGridSlotData, AllottedGeometry, NumDimensions, InDivisionParameter, DrawingParams);
		break;
	case SampledSequenceValueGridOverlay::EGridDivideMode::MidSplit:
		SampledSequenceDrawingUtils::GenerateMidpointSplitGridForGeometry(CachedGridSlotData, AllottedGeometry, NumDimensions, InDivisionParameter, DrawingParams);
		break;
	default:
		static_assert(static_cast<int32>(SampledSequenceValueGridOverlay::EGridDivideMode::COUNT) == 2, "Possible missing switch case coverage for 'ValueGridOverlay::EGridDivideMode'");
		break;
	}
	
	if (bHideLabels)
	{
		return;
	}

	switch (DrawingParams.Orientation)
	{
	case SampledSequenceDrawingUtils::ESampledSequenceDrawOrientation::Horizontal:
		GenerateHorizontalGridLabels(InDivisionParameter, AllottedGeometry);
		break;
	case SampledSequenceDrawingUtils::ESampledSequenceDrawOrientation::Vertical:
		GenerateVerticalGridLabels(InDivisionParameter, AllottedGeometry);
		break;
	default:
		static_assert(static_cast<int32>(SampledSequenceDrawingUtils::ESampledSequenceDrawOrientation::COUNT) == 2, "Possible missing switch case coverage for 'ESampledSequenceDrawOrientation'");
		break;
	}
	return;
}

void SSampledSequenceValueGridOverlay::GenerateVerticalGridLabels(const uint32 InDivisionParameter, const FGeometry& AllottedGeometry)
{
	CachedLabelData.Empty();
	TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	for (int32 SlotIndex = 0; SlotIndex < CachedGridSlotData.Num(); ++SlotIndex)
	{
		FGridLabelData LabelData;
		const SampledSequenceDrawingUtils::FGridData& SlotData = CachedGridSlotData[SlotIndex];

		for (int32 LineIndex = 0; LineIndex < SlotData.DrawCoordinates.Num(); ++LineIndex)
		{
			FText LabelText = OnValueGridLabel(SlotData.PositionRatios[LineIndex]);
			const FVector2D TextSize = FontMeasureService->Measure(LabelText, LabelTextFont);
			const FVector2D& DrawCoordinate = SlotData.DrawCoordinates[LineIndex].A;

			const float TextX = SlotData.PositionRatios[LineIndex] <= 0.5 ? DrawCoordinate.X + LabelToGridPixelDistance + TextSize.Y : DrawCoordinate.X - LabelToGridPixelDistance;
			const FVector2D TextOffset = FVector2D(TextX, DrawCoordinate.Y + LabelToGridPixelDistance);

			if (SlotData.DrawCoordinates.Num() > 1)
			{
				const int32 ComparedSlotIndex = LineIndex == SlotData.DrawCoordinates.Num() - 1 ? LineIndex - 1 : LineIndex + 1;
				const float GridLinesSpacing = FMath::Abs(DrawCoordinate.X - SlotData.DrawCoordinates[ComparedSlotIndex].A.X);

				if (GridLinesSpacing < TextSize.Y * 2 && InDivisionParameter != 0)
				{
					const float NewDivisionParameter = FMath::Clamp(InDivisionParameter - 1, 0, MaxDivisionParameter);
					CacheDrawElements(AllottedGeometry, NewDivisionParameter);
					return;
				}
			}

			LabelData.LabelTexts.Emplace(LabelText);
			LabelData.LabelCoordinates.Emplace(TextOffset);
		}

		CachedLabelData.Emplace(LabelData);
	}
}

void SSampledSequenceValueGridOverlay::GenerateHorizontalGridLabels(const uint32 InDivisionParameter, const FGeometry& AllottedGeometry)
{
	CachedLabelData.Empty();
	TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

	for (int32 SlotIndex = 0; SlotIndex < CachedGridSlotData.Num(); ++SlotIndex)
	{
		FGridLabelData LabelData;
		const SampledSequenceDrawingUtils::FGridData& SlotData = CachedGridSlotData[SlotIndex];

		for (int32 LineIndex = 0; LineIndex < SlotData.DrawCoordinates.Num(); ++LineIndex)
		{
			FText LabelText = OnValueGridLabel(SlotData.PositionRatios[LineIndex]);
			const FVector2D TextSize = FontMeasureService->Measure(LabelText, LabelTextFont);
			const FVector2D& DrawCoordinate = SlotData.DrawCoordinates[LineIndex].A;

			const float TextY = SlotData.PositionRatios[LineIndex] <= 0.5 ? DrawCoordinate.Y + LabelToGridPixelDistance : DrawCoordinate.Y - TextSize.Y - LabelToGridPixelDistance;
			const FVector2D TextOffset = FVector2D(DrawCoordinate.X + LabelToGridPixelDistance, TextY);

			if (SlotData.DrawCoordinates.Num() > 1)
			{
				const int32 ComparedSlotIndex = LineIndex == SlotData.DrawCoordinates.Num() - 1 ? LineIndex - 1 : LineIndex + 1;
				const float GridLinesSpacing = FMath::Abs(DrawCoordinate.Y - SlotData.DrawCoordinates[ComparedSlotIndex].A.Y);

				if (GridLinesSpacing < TextSize.Y * 2 && InDivisionParameter != 0)
				{
					const float NewDivisionParameter = FMath::Clamp(InDivisionParameter - 1, 0, MaxDivisionParameter);
					CacheDrawElements(AllottedGeometry, NewDivisionParameter);
					return;
				}
			}

			LabelData.LabelTexts.Emplace(LabelText);
			LabelData.LabelCoordinates.Emplace(TextOffset);
		}

		CachedLabelData.Emplace(LabelData);
	}
}