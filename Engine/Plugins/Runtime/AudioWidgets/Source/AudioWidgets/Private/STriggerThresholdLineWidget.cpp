// Copyright Epic Games, Inc. All Rights Reserved.

#include "STriggerThresholdLineWidget.h"

void STriggerThresholdLineWidget::Construct(const FArguments& InArgs)
{
	TriggerThreshold = InArgs._TriggerThreshold;
	LineColor        = InArgs._Style->LineColor;

	DrawingParams = InArgs._SequenceDrawingParams;
}

int32 STriggerThresholdLineWidget::OnPaint(const FPaintArgs& Args,
	const FGeometry& AllottedGeometry, 
	const FSlateRect& MyCullingRect, 
	FSlateWindowElementList& OutDrawElements, 
	int32 LayerId, 
	const FWidgetStyle& InWidgetStyle, 
	bool bParentEnabled) const
{
	const SampledSequenceDrawingUtils::FDimensionSlot DimensionSlot(0, 1, AllottedGeometry, DrawingParams);

	const FVector2D TriggerThresholdRange(-1.0, 1.0);
	const FVector2D DrawingAreaRange(DimensionSlot.Bottom, DimensionSlot.Top);

	const double TriggerThresholdMappedValue = FMath::GetMappedRangeValueUnclamped(TriggerThresholdRange, DrawingAreaRange, TriggerThreshold);

	const FVector2D StartLinePoint(0.0, TriggerThresholdMappedValue);
	const FVector2D EndLinePoint(AllottedGeometry.GetLocalSize().X, TriggerThresholdMappedValue);

	const TArray<FVector2D> TriggerLine({ StartLinePoint, EndLinePoint });

	FSlateDrawElement::MakeLines(OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), TriggerLine, ESlateDrawEffect::None, LineColor, true, 1.0f);

	return LayerId;
}

FVector2D STriggerThresholdLineWidget::ComputeDesiredSize(float) const
{
	return FVector2D();
}

void STriggerThresholdLineWidget::OnStyleUpdated(const FTriggerThresholdLineStyle UpdatedStyle)
{
	LineColor = UpdatedStyle.LineColor;
}

void STriggerThresholdLineWidget::SetTriggerThreshold(float InTriggerThreshold)
{
	TriggerThreshold = InTriggerThreshold;
}
