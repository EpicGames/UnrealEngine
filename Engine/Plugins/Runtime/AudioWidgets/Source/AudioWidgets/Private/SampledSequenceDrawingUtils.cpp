// Copyright Epic Games, Inc. All Rights Reserved.

#include "SampledSequenceDrawingUtils.h"

#include "Layout/Geometry.h"

namespace SampledSequenceDrawingUtilsPrivate
{
	void DivideLineIteratively(const float StartPoint, const float EndPoint, int32 Depth, TArray<double>& HalfPoints)
	{
		const double HalfPoint = (StartPoint + EndPoint) / 2.f;
		HalfPoints.Add(HalfPoint);

		if (--Depth > 0)
		{
			SampledSequenceDrawingUtilsPrivate::DivideLineIteratively(StartPoint, HalfPoint, Depth, HalfPoints);
			SampledSequenceDrawingUtilsPrivate::DivideLineIteratively(HalfPoint, EndPoint, Depth, HalfPoints);
		}
	}

	void GenerateGridDataFromMidPoints(const uint16 NDimensions, const FGeometry& InAllottedGeometry, const SampledSequenceDrawingUtils::FSampledSequenceDrawingParams& Params, const TArray<double>& MidPoints, const TArray<float>& MidPointsRatios, TArray<SampledSequenceDrawingUtils::FGridData>& OutGridData)
	{
		for (uint16 Dimension = 0; Dimension < NDimensions; ++Dimension)
		{
			SampledSequenceDrawingUtils::FGridData DimensionGridData;
			SampledSequenceDrawingUtils::FDimensionSlot DimensionSlot(Dimension, NDimensions, InAllottedGeometry, Params);

			for (int32 PointIndex = 0; PointIndex < MidPoints.Num(); ++PointIndex)
			{
				const float MidPointCoordinate = MidPoints[PointIndex] + DimensionSlot.Top;

				FVector2D PointA = Params.Orientation == SampledSequenceDrawingUtils::ESampledSequenceDrawOrientation::Horizontal ? FVector2D(0, MidPointCoordinate) : FVector2D(MidPointCoordinate, 0);
				FVector2D PointB = Params.Orientation == SampledSequenceDrawingUtils::ESampledSequenceDrawOrientation::Horizontal ? FVector2D(InAllottedGeometry.GetLocalSize().X, MidPointCoordinate) : FVector2D(MidPointCoordinate, InAllottedGeometry.GetLocalSize().Y);

				DimensionGridData.DrawCoordinates.Add({ PointA, PointB });
				DimensionGridData.PositionRatios.Add(MidPointsRatios[PointIndex]);
			}

			OutGridData.Add(DimensionGridData);
		}
	}
}

SampledSequenceDrawingUtils::FDimensionSlot::FDimensionSlot(const uint16 DimensionToDraw, const uint16 TotalNumDimensions, const FGeometry& InAllottedGeometry, const FSampledSequenceDrawingParams& Params)
{
	check(TotalNumDimensions > 0)

	const float& GeometryLength = Params.Orientation == ESampledSequenceDrawOrientation::Horizontal ? InAllottedGeometry.GetLocalSize().Y : InAllottedGeometry.GetLocalSize().X;

	const double FullSlotHeight = GeometryLength / TotalNumDimensions;
	const double MarginedTop = (FullSlotHeight * DimensionToDraw) + Params.DimensionSlotMargin;
	Height = FMath::Clamp(FullSlotHeight - (Params.DimensionSlotMargin * 2), FullSlotHeight * Params.MinSequenceHeightRatio, FullSlotHeight);
	const double MarginedBottom = MarginedTop + Height;

	Center = MarginedTop + ((MarginedBottom - MarginedTop) / 2.f);
	Height *= Params.MaxSequenceHeightRatio;
	Top = Center - Height / 2;
	Bottom = Center + Height / 2;
}

void SampledSequenceDrawingUtils::GenerateSampleBinsCoordinatesForGeometry(TArray<F2DLineCoordinates>& OutDrawCoordinates, const FGeometry& InAllottedGeometry, const TArray<TRange<float>>& InSampleBins, const uint16 NDimensions, const FSampledSequenceDrawingParams Params)
{
	if (!ensure(NDimensions != 0))
	{
		return;
	}

	const uint32 PixelWidth = FMath::FloorToInt(InAllottedGeometry.GetLocalSize().X);
	check(PixelWidth * NDimensions == InSampleBins.Num());

	const FDimensionSlot SampleSlotForScale(0, NDimensions, InAllottedGeometry, Params);
	const float HeightScale = SampleSlotForScale.Height / (2.f * Params.MaxDisplayedValue) * Params.VerticalZoomFactor;

	OutDrawCoordinates.SetNumUninitialized(InSampleBins.Num());
	F2DLineCoordinates* OutCoordinatesData = OutDrawCoordinates.GetData();

	for (uint16 Channel = 0; Channel < NDimensions; ++Channel)
	{
		const FDimensionSlot ChannelBoundaries(Channel, NDimensions, InAllottedGeometry, Params);

		for (uint32 Pixel = 0; Pixel < PixelWidth; ++Pixel)
		{
			uint32 PeakIndex = Pixel * NDimensions + Channel;
			const TRange<float>& MinMaxBin = InSampleBins[PeakIndex];

			const float SampleMaxScaled = MinMaxBin.GetUpperBoundValue() * HeightScale;
			const float SampleMinScaled = MinMaxBin.GetLowerBoundValue() * HeightScale;

			float Top = FMath::Max(ChannelBoundaries.Center - SampleMaxScaled, ChannelBoundaries.Top);
			float Bottom = FMath::Min(ChannelBoundaries.Center - SampleMinScaled, ChannelBoundaries.Bottom);

			const bool bResizeBin = FMath::Abs(Top - Bottom) < Params.MinScaledBinValue;

			if (bResizeBin)
			{
				const float MinHeightHalfSize = Params.MinScaledBinValue / 2.f;
				Top = Top <= Bottom ? Top - MinHeightHalfSize : Top + MinHeightHalfSize;
				Bottom = Top <= Bottom ? Bottom + MinHeightHalfSize : Top - MinHeightHalfSize;
			}

			OutCoordinatesData[PeakIndex].A = FVector2D(Pixel, Top);
			OutCoordinatesData[PeakIndex].B = FVector2D(Pixel, Bottom);
		}
	}
}

void SampledSequenceDrawingUtils::GenerateSequencedSamplesCoordinatesForGeometry(TArray<FVector2D>& OutDrawCoordinates, TArrayView<const float> InSampleData, const FGeometry& InAllottedGeometry, const uint16 NDimensions, const FFixedSampledSequenceGridMetrics InGridMetrics, const FSampledSequenceDrawingParams Params)
{
	OutDrawCoordinates.Empty();

	if (!ensure(NDimensions != 0))
	{
		return;
	}

	const uint32 NumFramesToDisplay = InSampleData.Num() / NDimensions;

	TArray<FDimensionSlot> ChannelSlotsBoundaries;

	for (uint16 Channel = 0; Channel < NDimensions; ++Channel)
	{
		ChannelSlotsBoundaries.Emplace(Channel, NDimensions, InAllottedGeometry, Params);
	}

	uint32 FrameIndex = 0;
	const float XDrawMargin = InAllottedGeometry.Size.X + InGridMetrics.PixelsPerFrame;

	for (double FrameX = InGridMetrics.FirstMajorTickX; FrameX < XDrawMargin; FrameX += InGridMetrics.PixelsPerFrame)
	{
		const double XRatio = FrameX / InAllottedGeometry.Size.X;

		if (FrameX >= 0 && FrameIndex < NumFramesToDisplay)
		{
			for (uint16 Channel = 0; Channel < NDimensions; ++Channel)
			{
				const FDimensionSlot& ChannelBoundaries = ChannelSlotsBoundaries[Channel];
				const uint32 SampleIndex = FrameIndex * NDimensions + Channel;
				const float SampleValue = InSampleData[SampleIndex];

				const float SampleValueRatio = SampleValue / Params.MaxDisplayedValue * Params.VerticalZoomFactor;
				const float HeightRatio = SampleValueRatio * ChannelBoundaries.Height / 2.f;
				const float SampleY = FMath::Clamp(ChannelBoundaries.Center - HeightRatio, ChannelBoundaries.Top, ChannelBoundaries.Bottom);

				const FVector2D SampleCoordinates(FrameX, SampleY);

				OutDrawCoordinates.Add(SampleCoordinates);
			}

			FrameIndex++;
		}
	}
}

void SampledSequenceDrawingUtils::GenerateEvenlySplitGridForGeometry(TArray<FGridData>& OutGridData, const FGeometry& InAllottedGeometry, const uint16 NDimensions, const uint32 NumGridDivisions /*= 5*/, const FSampledSequenceDrawingParams Params /*= FSampledSequenceDrawingParams()*/)
{
	if (!ensure(NDimensions != 0))
	{
		return;
	}

	OutGridData.Empty();

	FDimensionSlot FirstDimensionSlot(0, NDimensions, InAllottedGeometry, Params);
	TArray<double> MidPoints;
	TArray<float> MidPointsRatios;
	SampledSequenceDrawingUtils::GenerateEvenlySplitGridForLine(MidPoints, MidPointsRatios, FirstDimensionSlot.Height, NumGridDivisions);
	SampledSequenceDrawingUtilsPrivate::GenerateGridDataFromMidPoints(NDimensions, InAllottedGeometry, Params, MidPoints, MidPointsRatios, OutGridData);

}

void SampledSequenceDrawingUtils::GenerateMidpointSplitGridForGeometry(TArray<FGridData>& OutGridData, const FGeometry& InAllottedGeometry, const uint16 NDimensions, const uint32 DivisionDepth /*= 3*/, const FSampledSequenceDrawingParams Params /*= FSampledSequenceDrawingParams()*/)
{
	if (!ensure(NDimensions != 0))
	{
		return;
	}

	OutGridData.Empty();

	FDimensionSlot FirstDimensionSlot(0, NDimensions, InAllottedGeometry, Params);
	TArray<double> MidPoints;
	TArray<float> MidPointsRatios;
	SampledSequenceDrawingUtils::GenerateMidpointSplitGridForLine(MidPoints, MidPointsRatios, FirstDimensionSlot.Height, DivisionDepth);
	SampledSequenceDrawingUtilsPrivate::GenerateGridDataFromMidPoints(NDimensions, InAllottedGeometry, Params, MidPoints, MidPointsRatios, OutGridData);
}


void SampledSequenceDrawingUtils::GenerateMidpointSplitGridForLine(TArray<double>& OutDrawCoordinates, TArray<float>& OutLinePositionRatios, const float LineLength, const uint32 DivisionDepth /*= 3*/, bool bEmptyOutArrays /*= true*/)
{
	if (bEmptyOutArrays)
	{
		OutDrawCoordinates.Empty();
		OutLinePositionRatios.Empty();
	}

	const bool bAddBorders = DivisionDepth > 0;
	
	if (bAddBorders)
	{
		OutDrawCoordinates.Add(0);
		OutLinePositionRatios.Add(0.f);
	}


	TArray<double> MidPoints;
	SampledSequenceDrawingUtilsPrivate::DivideLineIteratively(0, LineLength, DivisionDepth, MidPoints);

	MidPoints.Sort();

	for (const double MidPoint : MidPoints)
	{
		const float MidPointRatio = MidPoint / LineLength;
		OutLinePositionRatios.Add(MidPointRatio);
		OutDrawCoordinates.Add(MidPoint);
	}

	if (bAddBorders)
	{
		OutDrawCoordinates.Add(LineLength);
		OutLinePositionRatios.Add(1);
	}

	
}

void SampledSequenceDrawingUtils::GenerateEvenlySplitGridForLine(TArray<double>& OutDrawCoordinates, TArray<float>& OutLinePositionRatios, const float LineLength, const uint32 NumGridDivisions /*= 2*/, bool bEmptyOutArrays /*= true*/)
{
	if (bEmptyOutArrays)
	{
		OutDrawCoordinates.Empty();
		OutLinePositionRatios.Empty();
	}

	const double GridStep = LineLength / NumGridDivisions;

	for (double GridLineCoordinate = 0; GridLineCoordinate < LineLength; GridLineCoordinate += GridStep)
	{
		const float GridValue = GridLineCoordinate / LineLength;
		OutLinePositionRatios.Add(GridValue);
		OutDrawCoordinates.Add(GridLineCoordinate);
	}

	OutLinePositionRatios.Add(1.f);
	OutDrawCoordinates.Add(LineLength);
}