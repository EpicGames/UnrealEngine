// Copyright Epic Games, Inc. All Rights Reserved.

#include "SparseSampledSequenceTransportCoordinator.h"

/** Zoom Formula

E.g: Coordinator is receiving a playback ratio (PBR) in the range 0 to 1
To position the focus point accordingly, this would the formula to follow:

FP = (PBR - LB) / ZR

FP = Focus Point
ZR = Zoom Ratio
LB = Lower Displayed Waveform Bound
from here, you can rotate as needed. 
e.g to find PBR
PBR = (FP * ZR) + LB



This formula can be used to position any element (e.g. a marker) given in a 0 to 1 position range where
0 is the beginning time of the waveform and 1 the end time

Zoom is defined as a ratio = displayed length / total length

*/

void FSparseSampledSequenceTransportCoordinator::UpdatePlaybackRange(const TRange<double>& NewRange)
{
	ProgressRange = NewRange;
}

const bool FSparseSampledSequenceTransportCoordinator::IsScrubbing() const
{
	return bIsScrubbing;
}

const bool FSparseSampledSequenceTransportCoordinator::IsScrolling() const
{
	return bIsScrolling;
}

void FSparseSampledSequenceTransportCoordinator::SetScrolling(const bool bInIsScrolling)
{
	bIsScrolling = bInIsScrolling;
}

float FSparseSampledSequenceTransportCoordinator::ConvertAbsoluteRatioToZoomed(const float InAbsoluteRatio) const
{
	return (InAbsoluteRatio - DisplayRange.GetLowerBoundValue()) / ZoomRatio;
}

float FSparseSampledSequenceTransportCoordinator::ConvertZoomedRatioToAbsolute(const float InZoomedRatio) const
{
	return InZoomedRatio * ZoomRatio + DisplayRange.GetLowerBoundValue();
}

const TRange<double> FSparseSampledSequenceTransportCoordinator::GetDisplayRange() const
{
	return DisplayRange;
}

void FSparseSampledSequenceTransportCoordinator::MoveFocusPoint(const double InFocusPoint)
{
	FocusPoint = InFocusPoint;
	OnFocusPointMoved.Broadcast(FocusPoint);
}

void FSparseSampledSequenceTransportCoordinator::UpdateZoomRatioAndDisplayRange(const float NewZoomRatio)
{
	const float ClampedZoomRatio = FMath::Clamp(NewZoomRatio, UE_KINDA_SMALL_NUMBER, 1.f);
	const float ZoomRatioDelta = ZoomRatio - ClampedZoomRatio;

 	const double DeltaOutsideBoundaries = DisplayRange.GetUpperBoundValue() - (DisplayRange.GetUpperBoundValue() - ZoomRatioDelta);

	const double DeltaStepL = DeltaOutsideBoundaries * FocusPoint;
	const double DeltaStepR = DeltaOutsideBoundaries * (1.f - FocusPoint);

	double MinDisplayRangeValue = DeltaStepL + DisplayRange.GetLowerBoundValue();
	double MaxDisplayRangeValue = DisplayRange.GetUpperBoundValue() - DeltaStepR;

	const double CurrentPlaybackPosition = (ZoomRatio * FocusPoint) + DisplayRange.GetLowerBoundValue();

	//if Min < 0 or Max > 1 we can't zoom out further on that side

	//hence we shift the zooming entirely on the opposite side
	if (MinDisplayRangeValue < 0.f)
	{
		const double ExceedingDelta = FMath::Abs(MinDisplayRangeValue);
		MaxDisplayRangeValue = MaxDisplayRangeValue + ExceedingDelta;
		MinDisplayRangeValue = 0.f;
	}

	else if (MaxDisplayRangeValue > 1.f)
	{
		const double ExceedingDelta = MaxDisplayRangeValue - 1.f;

		MinDisplayRangeValue = MinDisplayRangeValue - ExceedingDelta;
		MaxDisplayRangeValue = 1.f;
	}

	const double NewPlayheadPosition = (CurrentPlaybackPosition - MinDisplayRangeValue) / ClampedZoomRatio;

	ZoomRatio = ClampedZoomRatio;
	UpdateDisplayRange(MinDisplayRangeValue, MaxDisplayRangeValue);
}

void FSparseSampledSequenceTransportCoordinator::Stop()
{
	SetProgressRatio(0.f);
}

void FSparseSampledSequenceTransportCoordinator::MoveWaveformView(float MoveAmount)
{
	double MinDisplayRangeValue = DisplayRange.GetLowerBoundValue() + (MoveAmount * ZoomRatio);
	double MaxDisplayRangeValue = DisplayRange.GetUpperBoundValue() + (MoveAmount * ZoomRatio);

	//Clamp zoom bounds while maintaining same width if bounds go outside 0-1 range
	if (MinDisplayRangeValue < 0.0)
	{
		const double ExceedingDelta = FMath::Abs(MinDisplayRangeValue);
		MaxDisplayRangeValue = FMath::Clamp(MaxDisplayRangeValue + ExceedingDelta, 0.0, 1.0);
		MinDisplayRangeValue = 0.0;
	}

	if (MaxDisplayRangeValue > 1.0)
	{
		const double ExceedingDelta = MaxDisplayRangeValue - 1.0;

		MinDisplayRangeValue = FMath::Clamp(MinDisplayRangeValue - ExceedingDelta, 0.0, 1.0);
		MaxDisplayRangeValue = 1.0;
	}

	UpdateDisplayRange(MinDisplayRangeValue, MaxDisplayRangeValue);
}

const double FSparseSampledSequenceTransportCoordinator::GetFocusPoint() const
{
	return FocusPoint;
}

void FSparseSampledSequenceTransportCoordinator::ScrubFocusPoint(const float InTargetFocusPoint, const bool bIsMoving)
{
	const float LowerPlaybackBoundPosition = (ProgressRange.GetLowerBoundValue() - DisplayRange.GetLowerBoundValue()) / ZoomRatio;
	const float UpperPlaybackBoundPosition = (ProgressRange.GetUpperBoundValue() - DisplayRange.GetLowerBoundValue()) / ZoomRatio;

	float ClampedTargetPosition = FMath::Clamp(InTargetFocusPoint, LowerPlaybackBoundPosition, UpperPlaybackBoundPosition);
	float NewPosition = FMath::Clamp(ClampedTargetPosition, 0.f, 1.f);

	if (ClampedTargetPosition > FocusPointLockPosition)
	{
		const float BoundsDelta = (ClampedTargetPosition - FocusPointLockPosition) * ZoomRatio;
		const float NewDisplayUpBound = FMath::Clamp(DisplayRange.GetUpperBoundValue() + BoundsDelta, 0.f, 1.f);

		if (NewDisplayUpBound < 1.f)
		{
			const float NewDisplayLowBound = FMath::Clamp(DisplayRange.GetLowerBoundValue() + BoundsDelta, 0.f, 1.f);
			UpdateDisplayRange(NewDisplayLowBound, NewDisplayUpBound);
			NewPosition = FocusPointLockPosition;
		}
	}
	if (ClampedTargetPosition < 0.f)
	{
		const float BoundsDelta = ClampedTargetPosition * ZoomRatio;
		const float NewDisplayLowBound = FMath::Clamp(DisplayRange.GetLowerBoundValue() + BoundsDelta, 0.f, 1.f);

		if (NewDisplayLowBound > 0.f)
		{

			const float NewDisplayUpBound = FMath::Clamp(DisplayRange.GetUpperBoundValue() + BoundsDelta, 0.f, 1.f);

			UpdateDisplayRange(NewDisplayLowBound, NewDisplayUpBound);

			NewPosition = 0.f;
		}
	}

	MoveFocusPoint(NewPosition);

	if (OnFocusPointScrubUpdate.IsBound())
	{
		OnFocusPointScrubUpdate.Broadcast(GetPlayBackRatioFromFocusPoint(NewPosition), bIsMoving);
	}

	bIsScrubbing = bIsMoving;
}



void FSparseSampledSequenceTransportCoordinator::SetProgressRatio(const float NewRatio)
{	 
	if (bIsScrubbing || bIsScrolling)
	{
		return;
	}

	check(NewRatio >= 0.f && NewRatio <= 1.f);

	const double PlaybackRangeScale = ProgressRange.Size<double>();
	const double ScaledPlaybackRatio = NewRatio * PlaybackRangeScale + ProgressRange.GetLowerBoundValue();

	const double NewPlayheadPosition = (ScaledPlaybackRatio - DisplayRange.GetLowerBoundValue()) / ZoomRatio;

	if (NewPlayheadPosition < 0.f)
	{
		// If the playhead is off-screen to the left (e.g. loop restarted or user scrolled away),
		// recenter the display range with the playback position on the left edge

		double NewDisplayLowBound = FMath::Clamp(ScaledPlaybackRatio, 0.0, 1.0 - ZoomRatio);
		double NewDisplayUpBound = NewDisplayLowBound + ZoomRatio;

		// Update DisplayRange first so the Playhead is placed correctly at the left edge
		UpdateDisplayRange(NewDisplayLowBound, NewDisplayUpBound);
		
		constexpr double LeftBound = 0.0;
		MoveFocusPoint(LeftBound);
	}
	else if (NewPlayheadPosition >= FocusPointLockPosition)
	{
		const double NewDisplayUpBound = FMath::Clamp(ScaledPlaybackRatio + (ZoomRatio * (1.f - FocusPointLockPosition)), 0.f, 1.f);
		double NewDisplayLowBound = NewDisplayUpBound < 1.f ? ScaledPlaybackRatio - (ZoomRatio * FocusPointLockPosition) : 1.f - ZoomRatio;

		NewDisplayLowBound = FMath::Clamp(NewDisplayLowBound, 0, 1);
		
		UpdateDisplayRange(NewDisplayLowBound, NewDisplayUpBound);

		if (NewDisplayUpBound < 1.f)
		{
			// We only lock the playhead if the display upper bound is before the end of the waveform
			// otherwise we let the playhead go to the end

			MoveFocusPoint(FocusPointLockPosition);
		}
		else
		{
			// If playhead is far off screen, the NewPlayheadPosition could be > 1.0
			constexpr double RightBound = 1.0;
			MoveFocusPoint(FMath::Min(NewPlayheadPosition, RightBound));
		}
	}
	else
	{
		MoveFocusPoint(NewPlayheadPosition);
	}
}

void FSparseSampledSequenceTransportCoordinator::SetZoomRatio(const float NewLevel)
{
	UpdateZoomRatioAndDisplayRange(NewLevel);
}

void FSparseSampledSequenceTransportCoordinator::UpdateDisplayRange(const double MinValue, const double MaxValue)
{
	check(MinValue < MaxValue);

	DisplayRange.SetLowerBoundValue(FMath::Clamp(MinValue, 0.f, MaxValue));
	DisplayRange.SetUpperBoundValue(FMath::Clamp(MaxValue, MinValue, 1.f));

	if (OnDisplayRangeUpdated.IsBound())
	{
		OnDisplayRangeUpdated.Broadcast(DisplayRange);
	}
}

bool FSparseSampledSequenceTransportCoordinator::IsRatioWithinDisplayRange(const double Ratio) const
{
	return DisplayRange.Contains(Ratio);
}

double FSparseSampledSequenceTransportCoordinator::GetPlayBackRatioFromFocusPoint(const double InFocusPoint) const
{
	check(InFocusPoint >= 0.f && InFocusPoint <= 1.f);

	const double PlayBackRatio = ((InFocusPoint * ZoomRatio) + DisplayRange.GetLowerBoundValue() - ProgressRange.GetLowerBoundValue()) / ProgressRange.Size<float>();
	return PlayBackRatio;
}