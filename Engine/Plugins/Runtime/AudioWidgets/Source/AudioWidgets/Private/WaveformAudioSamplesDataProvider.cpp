// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformAudioSamplesDataProvider.h"

#include "AudioBusSubsystem.h"
#include "AudioMixerDevice.h"
#include "AudioWidgetsLog.h"
#include "DSP/DeinterleaveView.h"

namespace AudioWidgets
{
	namespace FWaveformAudioSamplesDataProviderPrivate
	{
		int32 GetTriggerThresholdBufferIndex(const EAudioOscilloscopeTriggerMode InTriggerMode, const float InTriggerThreshold, const TArray<float>& InAudioSamples)
		{
			for (int32 Index = 1; Index < InAudioSamples.Num(); ++Index)
			{
				if (InTriggerMode == EAudioOscilloscopeTriggerMode::Rising)
				{
					if (InAudioSamples[Index - 1] < InTriggerThreshold && InAudioSamples[Index] >= InTriggerThreshold)
					{
						return Index - 1;
					}
				}
				else if (InTriggerMode == EAudioOscilloscopeTriggerMode::Falling)
				{
					if (InAudioSamples[Index - 1] > InTriggerThreshold && InAudioSamples[Index] <= InTriggerThreshold)
					{
						return Index - 1;
					}
				}
			}

			return -1;
		};
	}

	FWaveformAudioSamplesDataProvider::FWaveformAudioSamplesDataProvider(const Audio::FDeviceId InAudioDeviceId,
		UAudioBus* InAudioBus,
		const uint32 InNumChannelToProvide,
		const float InTimeWindowMs,
		const float InMaxTimeWindowMs,
		const float InAnalysisPeriodMs)
	{
		if (!InAudioBus)
		{
			UE_LOGF(LogAudioWidgets, Error, "Unable to obtain audio samples for visualization without a valid audio bus.");
			return;
		}

		const FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get();
		if (!AudioDeviceManager)
		{
			UE_LOGF(LogAudioWidgets, Error, "Unable to obtain audio samples for visualization without a valid audio device manager.");
			return;
		}

		MixerDevice = static_cast<const Audio::FMixerDevice*>(AudioDeviceManager->GetAudioDeviceRaw(InAudioDeviceId));
		if (!MixerDevice)
		{
			UE_LOGF(LogAudioWidgets, Error, "Unable to obtain audio samples for visualization without a valid audio mixer.");
			return;
		}

		AudioBus = InAudioBus;

		NumChannelsToProvide = InNumChannelToProvide;
		NumChannels          = InAudioBus->GetNumChannels();

		SampleRate = static_cast<uint32>(MixerDevice->GetSampleRate());

		// Init ticker handle
		if (!TickerHandle.IsValid())
		{
			TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FWaveformAudioSamplesDataProvider::Tick), 0.0f);
		}

		// Init audio buffers
		MaxTimeWindowMs = InMaxTimeWindowMs;

		const float MaxTimeWindowSamples = FMath::RoundToInt(MaxTimeWindowMs / 1000.0f * NumChannelsToProvide * SampleRate);

		AudioSamplesForView.Init(0.0f, MaxTimeWindowSamples);
		DataView = FFixedSampledSequenceView{ MakeArrayView(AudioSamplesForView.GetData(), AudioSamplesForView.Num()), NumChannelsToProvide, SampleRate };

		AudioSamplesCircularBuffer.SetCapacity(MaxTimeWindowSamples * 2); // Twice of the amount needed for display 

		SetTimeWindow(InTimeWindowMs);
		SetAnalysisPeriod(InAnalysisPeriodMs);
	}


	FWaveformAudioSamplesDataProvider::~FWaveformAudioSamplesDataProvider()
	{
		if (TickerHandle.IsValid())
		{
			FTSTicker::RemoveTicker(TickerHandle);
			TickerHandle.Reset();
		}

		StopProcessing();
	}

	void FWaveformAudioSamplesDataProvider::ResetAudioBuffers()
	{
		const float MaxTimeWindowSamples = FMath::RoundToInt(MaxTimeWindowMs / 1000.0f * NumChannelsToProvide * SampleRate);

		AudioSamplesCircularBuffer.Reset(MaxTimeWindowSamples * 2);
		AudioSamplesForView.Init(0.0f, MaxTimeWindowSamples);

		AudioSamplesCircularBuffer.SetNum(FMath::Clamp(TimeWindowSamples, 0.0f, MaxTimeWindowSamples));
	}

	void FWaveformAudioSamplesDataProvider::StartProcessing()
	{
		if (bIsProcessing)
		{
			return;
		}

		check(AudioBus);
		check(MixerDevice);

		ResetAudioBuffers();

		// Start the audio bus. This won't do anything if the bus is already started elsewhere.
		const uint32 AudioBusId = AudioBus->GetUniqueID();

		UAudioBusSubsystem* AudioBusSubsystem = MixerDevice->GetSubsystem<UAudioBusSubsystem>();
		check(AudioBusSubsystem);

		const Audio::FAudioBusKey AudioBusKey = Audio::FAudioBusKey(AudioBusId);

		const FString WaveformAudioSamplesDataProviderAudioBusName = FString::Format(TEXT("_WaveformAudioSamplesDataProvider_AudioBusId_{0}"), { AudioBusId });
		AudioBusSubsystem->StartAudioBus(AudioBusKey, WaveformAudioSamplesDataProviderAudioBusName, NumChannels, false);

		// Get an output patch for the audio bus
		const uint32 NumFramesPerBufferToAnalyze = MixerDevice->GetNumOutputFrames();
		PatchOutput = AudioBusSubsystem->AddPatchOutputForAudioBus(AudioBusKey, NumFramesPerBufferToAnalyze, NumChannels);

		bIsProcessing = true;
	}

	void FWaveformAudioSamplesDataProvider::StopProcessing()
	{
		if (bIsProcessing)
		{
			PatchOutput   = nullptr;
			bIsProcessing = false;
		}
	}

	void FWaveformAudioSamplesDataProvider::SetMaxTimeWindowMs(const float InMaxTimeWindowMs)
	{
		MaxTimeWindowMs = InMaxTimeWindowMs;
		ResetAudioBuffers();
	}

	void FWaveformAudioSamplesDataProvider::SetChannelToAnalyze(const int32 InChannel)
	{
		const int32 ChannelIndex = InChannel - 1;

		if (ChannelIndex != ChannelIndexToAnalyze)
		{
			ChannelIndexToAnalyze = ChannelIndex;
			ResetAudioBuffers();
		}
	}

	void FWaveformAudioSamplesDataProvider::SetTriggerMode(const EAudioOscilloscopeTriggerMode InTriggerMode)
	{
		TriggerMode   = InTriggerMode;
		bHasTriggered = false;
	}

	void FWaveformAudioSamplesDataProvider::SetTriggerThreshold(const float InTriggerThreshold)
	{
		TriggerThreshold = InTriggerThreshold;
		bHasTriggered    = false;
	}

	void FWaveformAudioSamplesDataProvider::SetTimeWindow(const float InTimeWindowMs)
	{
		TimeWindowSamples = FMath::RoundToInt(InTimeWindowMs / 1000.0f * NumChannelsToProvide * SampleRate);
		DataView          = FFixedSampledSequenceView{ MakeArrayView(AudioSamplesForView.GetData(), AudioSamplesForView.Num()).Slice(0, TimeWindowSamples), NumChannelsToProvide, SampleRate };

		if (NumChannelsToProvide == 1)
		{
			AudioSamplesCircularBuffer.SetNum(TimeWindowSamples);
		}
		else
		{
			// Adjust read pointer so it falls in a sample of the first channel
			const uint32 ChannelsModulo              = TimeWindowSamples % NumChannelsToProvide;
			const uint32 TimeWindowSamplesWithOffset = TimeWindowSamples - ChannelsModulo;

			AudioSamplesCircularBuffer.SetNum(TimeWindowSamplesWithOffset);
		}
	}

	void FWaveformAudioSamplesDataProvider::SetAnalysisPeriod(const float InAnalysisPeriodMs)
	{
		AnalysisPeriodSamples = InAnalysisPeriodMs * 0.001f * SampleRate;
		AudioSamplesCircularBuffer.SetNum(TimeWindowSamples);
	}

	void FWaveformAudioSamplesDataProvider::PushAudioSamplesToCircularBuffer()
	{
		if (const int32 NumSamplesAvailable = PatchOutput->GetNumSamplesAvailable();
			NumSamplesAvailable > 0)
		{
			TempAudioBuffer.Reset();
			TempAudioBuffer.AddUninitialized(NumSamplesAvailable);

			PatchOutput->PopAudio(TempAudioBuffer.GetData(), NumSamplesAvailable, false);

			if (NumChannelsToProvide == 1)
			{
				// Deinterleave so we push into the circular buffer the channel of interest
				TArray<float> ArrayToFill;
				Audio::TAutoDeinterleaveView<float> DeinterleaveView(TempAudioBuffer, ArrayToFill, NumChannels);

				for (auto Channel : DeinterleaveView)
				{
					if (Channel.ChannelIndex == ChannelIndexToAnalyze)
					{
						NumSamplesPushedToCircularBuffer += AudioSamplesCircularBuffer.Push(Channel.Values.GetData(), Channel.Values.Num());

						// Check if a trigger has occurred
						if (TriggerMode != EAudioOscilloscopeTriggerMode::None)
						{
							if (const int32 TriggerThresholdBufferIndex = FWaveformAudioSamplesDataProviderPrivate::GetTriggerThresholdBufferIndex(TriggerMode, TriggerThreshold, Channel.Values);
								TriggerThresholdBufferIndex != -1 && !bHasTriggered)
							{
								const int32 TriggerScreenPosition = 0; // TODO alex.perez: tweakable param? can be from 0 to TimeWindowSamples - 1;

								// Move the read pointer at the trigger point
								AudioSamplesCircularBuffer.SetNum(NumSamplesPushedToCircularBuffer - TriggerThresholdBufferIndex + TriggerScreenPosition);

								bHasTriggered = true;
							}
						}
					}
				}
			}
			else
			{
				NumSamplesPushedToCircularBuffer += AudioSamplesCircularBuffer.Push(TempAudioBuffer.GetData(), TempAudioBuffer.Num());
			}
		}
	}

	bool FWaveformAudioSamplesDataProvider::Tick(float DeltaTime)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FWaveformAudioSamplesDataProvider::Tick);

		if (PatchOutput.IsValid())
		{
			PushAudioSamplesToCircularBuffer();

			const uint32 NumAvailableSamplesToReadCircularBuffer = AudioSamplesCircularBuffer.Num();

			if (TriggerMode == EAudioOscilloscopeTriggerMode::None || !bHasTriggered)
			{
				if (NumAvailableSamplesToReadCircularBuffer >= TimeWindowSamples)
				{
					// Obtain the audio samples we want to display
					AudioSamplesCircularBuffer.Peek(AudioSamplesForView.GetData(), TimeWindowSamples);

					// Advance the read pointer
					AudioSamplesCircularBuffer.Pop(AnalysisPeriodSamples * NumChannelsToProvide);
				}
			}
			else
			{
				// Keep refreshing the trigger view with new samples until we reach TimeWindowSamples
				if (NumAvailableSamplesToReadCircularBuffer < TimeWindowSamples)
				{
					AudioSamplesCircularBuffer.Peek(AudioSamplesForView.GetData(), NumAvailableSamplesToReadCircularBuffer);
				}
				else
				{
					AudioSamplesCircularBuffer.Pop(AudioSamplesForView.GetData(), NumAvailableSamplesToReadCircularBuffer);
					bHasTriggered = false;
				}
			}

			// Send data if we have reached the analysis period time
			if (NumSamplesPushedToCircularBuffer >= AnalysisPeriodSamples * NumChannelsToProvide)
			{
				RequestSequenceView(TRange<double>::Inclusive(0, 1));
				NumSamplesPushedToCircularBuffer = 0;
			}
		}

		return true;
	}

	FFixedSampledSequenceView FWaveformAudioSamplesDataProvider::RequestSequenceView(const TRange<double> DataRatioRange)
	{
		// Broadcast audio samples data view
		OnDataViewGenerated.Broadcast(DataView, /*FirstRenderedSample*/ 0);

		return DataView;
	}
} // namespace AudioWidgets
