// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "Framework/Docking/TabManager.h"
#include "Sound/AudioBus.h"

#define UE_API AUDIOWIDGETS_API

class FUICommandList;

namespace AudioWidgets
{
	/**
	 * FAudioBusInfo encapsulates the required info that describes the audio bus that is to be analyzed.
	 */
	struct FAudioBusInfo
	{
		static constexpr Audio::FDeviceId InvalidAudioDeviceId = static_cast<Audio::FDeviceId>(INDEX_NONE);

		Audio::FDeviceId AudioDeviceId = InvalidAudioDeviceId;
		TObjectPtr<UAudioBus> AudioBus = nullptr;

		int32 GetNumChannels() const { return (AudioBus) ? AudioBus->GetNumChannels() : 0; }
	};

	/**
	 * Interface for something that can be used in an analyzer rack.
	 */
	class IAudioAnalyzerRackUnit : public TSharedFromThis<IAudioAnalyzerRackUnit>
	{
	public:
		virtual ~IAudioAnalyzerRackUnit() = default;

		/** If the Audio Bus to analyze changes (due to channel count change), handle this here. */
		virtual void SetAudioBusInfo(const FAudioBusInfo& AudioBusInfo) = 0;

		/** Spawn the actual analyzer Widget in a DockTab. */
		virtual TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args) const = 0;

		/** Be notified of request to start processing. */
		virtual void StartProcessing() {};

		/** Be notified of request to stop processing. */
		virtual void StopProcessing() {};
	};

	/**
	 * Parameters for the FOnMakeAudioAnalyzerRackUnit delegate.
	 */
	struct FAudioAnalyzerRackUnitConstructParams
	{
		FAudioBusInfo AudioBusInfo;
		const ISlateStyle* StyleSet = nullptr;
		const UClass* EditorSettingsClass = nullptr;
	};

	DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<IAudioAnalyzerRackUnit>, FOnMakeAudioAnalyzerRackUnit, const FAudioAnalyzerRackUnitConstructParams&);

	/**
	 * Static type descriptor data for a rack unit type.
	 */
	struct FAudioAnalyzerRackUnitTypeInfo
	{
		FName TypeName;
		FText DisplayName;
		FSlateIcon Icon;
		FOnMakeAudioAnalyzerRackUnit OnMakeAudioAnalyzerRackUnit;
		float VerticalSizeCoefficient;
	};

	/**
	 * Manages display of audio analyzer rack units. Rack units can be shown, hidden, and reordered by the user.
	 */
	class FAudioAnalyzerRack : public TSharedFromThis<FAudioAnalyzerRack>
	{
	public:
		struct FRackConstructParams
		{
			/** The rack layout can be saved using the given name. */
			FName TabManagerLayoutName;

			/** An ISlateStyle can be provided to override AudioWidgetsStyle. */
			const ISlateStyle* StyleSet = nullptr;

			/** An Editor Settings class can be provided for rack units that require one for saving settings. */
			const UClass* EditorSettingsClass = nullptr;

			/** Rack unit type names to exclude from the rack (they won't appear in the menu or layout). */
			TSet<FName> ExcludedRackUnitTypes;
		};

		UE_API FAudioAnalyzerRack(const FRackConstructParams& Params);
		UE_API virtual ~FAudioAnalyzerRack();

		UE_API void Init(int32 InNumChannels, Audio::FDeviceId InAudioDeviceId);
		UE_API void DestroyAnalyzers();

		UE_API TSharedRef<SWidget> CreateWidget(TSharedRef<SDockTab> DockTab, const FSpawnTabArgs& SpawnTabArgs);

		UE_API UAudioBus* GetAudioBus() const;

		UE_API void StartProcessing();
		UE_API void StopProcessing();

	protected:
		UE_API virtual TSharedRef<FTabManager::FArea> CreatePrimaryArea(const TArray<const FAudioAnalyzerRackUnitTypeInfo*>& RackUnitTypes);

	private:
		static TSharedRef<SWidget> MakeVisibleAnalyzersMenu(TSharedRef<FUICommandList> InCommandList, TWeakPtr<FWorkspaceItem> InWorkspaceGroup, TWeakPtr<FTabManager> InTabManager);

		static void SaveTabLayout(const TSharedRef<FTabManager::FLayout>& InLayout);
		TSharedRef<FTabManager::FLayout> LoadTabLayout();
		TSharedRef<FTabManager::FLayout> GetDefaultTabLayout();

		TSharedRef<IAudioAnalyzerRackUnit> MakeRackUnit(FName RackUnitTypeName);

		FName TabManagerLayoutName;
		TSet<FName> ExcludedRackUnitTypes;

		TStrongObjectPtr<UAudioBus> AudioBus;
		FAudioAnalyzerRackUnitConstructParams RackUnitConstructParams;

		TMap<FName, TSharedRef<IAudioAnalyzerRackUnit>> RackUnits;
		TSharedPtr<FTabManager> TabManager;
		
		bool bIsProcessingStarted = false;
	};
}

#undef UE_API
