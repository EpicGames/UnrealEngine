// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Color.h"
#include "UObject/ObjectMacros.h"
#include "AudioColorMapper.generated.h"

UENUM(BlueprintType)
enum class EAudioColorGradient : uint8
{
	BlackToWhite,
	WhiteToBlack,
};

/**
 * Helper class for normalizing and applying a colormapping to unnormalized data.
 */
class FAudioColorMapper
{
public:
	FAudioColorMapper(const float InMinValue, const float InMaxValue, const EAudioColorGradient InColorMap)
		: ValueScaling(1.0f / (InMaxValue - InMinValue))
		, MinValue(InMinValue)
		, ColorMap(InColorMap)
	{
		//
	}

	FColor GetColorFromValue(const float Value) const;

	static FColor ColorMapBlackToWhite(const float T);
	static FColor ColorMapWhiteToBlack(const float T);
	
private:
	const float ValueScaling;
	const float MinValue;
	const EAudioColorGradient ColorMap;
};
