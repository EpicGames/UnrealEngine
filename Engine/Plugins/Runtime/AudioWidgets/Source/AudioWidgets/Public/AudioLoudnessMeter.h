// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioAnalyzerRack.h"
#include "Containers/Ticker.h"
#include "LKFS.h"
#include "LoudnessMeterWidgetView.h"
#include "Meter.h"

#define UE_API AUDIOWIDGETS_API

namespace AudioWidgets
{
	/**
	 * Constructor parameters for the analyzer.
	 */
	struct FAudioLoudnessMeterParams
	{
		int32 NumChannels = 1;
		Audio::FDeviceId AudioDeviceId = INDEX_NONE;
		TObjectPtr<UAudioBus> ExternalAudioBus = nullptr;
	};

	/**
	 * Owns an analyzer and a corresponding Slate widget for displaying the loudness stats.
	 * Can either create an Audio Bus to analyze, or analyze the given Bus.
	 */
	class FAudioLoudnessMeter : public IAudioAnalyzerRackUnit
	{
	public:
		static UE_API const FAudioAnalyzerRackUnitTypeInfo RackUnitTypeInfo;
		
		UE_API FAudioLoudnessMeter(const FAudioLoudnessMeterParams& Params);
		UE_API ~FAudioLoudnessMeter();

		// Begin IAudioAnalyzerRackUnit overrides.
		UE_API virtual void SetAudioBusInfo(const FAudioBusInfo& AudioBusInfo) override;
		UE_API virtual TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args) const override;
		// End IAudioAnalyzerRackUnit overrides.

	private:
		struct FLoudnessMeterMaxResults
		{
			float ShortTermLoudness = MIN_VOLUME_DECIBELS;
			float MomentaryLoudness = MIN_VOLUME_DECIBELS;
			float TruePeakDb        = MIN_VOLUME_DECIBELS;
		};

		static TSharedRef<IAudioAnalyzerRackUnit> MakeRackUnit(const FAudioAnalyzerRackUnitConstructParams& Params);

		void Init(int32 InNumChannels, Audio::FDeviceId InAudioDeviceId, TObjectPtr<UAudioBus> InExternalAudioBus);
		void Teardown();

		void CreateLKFSAnalyzer();
		void ReleaseLKFSAnalyzer();
		void ResetLKFSAnalyzer();

		void CreateTruePeakAnalyzer();
		void ReleaseTruePeakAnalyzer();
		void ResetTruePeakAnalyzer();

		void InitLoudnessMeterWidgetView(const FAudioAnalyzerRackUnitConstructParams& Params);

		void OnLoudnessOutput(ULKFSAnalyzer* InLKFSAnalyzer, const FLKFSResults& InLoudnessResults);
		void OnTruePeakOutput(UMeterAnalyzer* InMeterAnalyzer, int32 InChannelIndex, const TArray<FMeterResults>& InMeterResultsArray);

		FTimespan GetAnalysisTime() const;
		FReply HandleResetButtonClicked();

		/** Creates the meters widget */
		AudioWidgetsCore::FLoudnessMeterWidgetView LoudnessMeterWidgetView;
		
		/** Loudness analyzer object. */
		TStrongObjectPtr<ULKFSAnalyzer> LKFSAnalyzer;

		/** Loudness analyzer settings. */
		TStrongObjectPtr<ULKFSSettings> LKFSSettings;

		/** Handle for results delegate for loudness analyzer. */
		FDelegateHandle LatestOverallLKFSResultsDelegateHandle;

		/** Most recent results from the loudness analyzer. */
		TOptional<FLKFSResults> LatestLoudnessResults;

		/** True peak analyzer object. */
		TStrongObjectPtr<UMeterAnalyzer> TruePeakAnalyzer;

		/** True peak analyzer settings. */
		TStrongObjectPtr<UMeterSettings> TruePeakSettings;

		/** Handle for results delegate for true peak analyzer. */
		FDelegateHandle PerChannelMeterResultsNativeDelegateHandle;

		/** Most recent results from the true peak analyzer. */
		TOptional<FMeterResults> LatestTruePeakResults;

		/** Loudness meter max results (only for short term loudness, momentary loudness and true peak). */
		FLoudnessMeterMaxResults LoudnessMeterMaxResults;

		Audio::FDeviceId AudioDeviceId = INDEX_NONE;

		FTSTicker::FDelegateHandle WidgetRefreshTicker;

		/** The audio bus used for analysis. */
		TStrongObjectPtr<UAudioBus> AudioBus;
		bool bUseExternalAudioBus = false;
	};
} // namespace AudioWidgets

#undef UE_API
