// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "Components/Widget.h"
#include "Delegates/Delegate.h"
#include "AudioMaterialButton.generated.h"

#define UE_API AUDIOWIDGETS_API

class SAudioMaterialButton;
struct FAudioMaterialButtonStyle;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnButtonMouseCaptureBeginEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnButtonMouseCaptureEndEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnButtonPressedChangedEvent, bool, bIsPressed);


/**
 * A simple widget that shows a button
 * Button is rendered by using material instead of texture.
 *
 * * No Children
 */
UCLASS(MinimalAPI)
class UAudioMaterialButton : public UWidget
{
	GENERATED_BODY()

public:

	UE_API UAudioMaterialButton();

	/** The button's style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style", meta = (DisplayName = "Style", ShowOnlyInnerProperties))
	FAudioMaterialButtonStyle WidgetStyle;

public:
#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif // WITH_EDITOR


	// UWidget
	UE_API virtual void SynchronizeProperties() override;
	// End of UWidget

	// UVisual
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UVisual

		/** Gets the current value of the slider.*/
	UFUNCTION(BlueprintPure, Category = "Audio Widgets| Audio Material Button")
	UE_API bool GetIsPressed() const;

	/** Sets the current value of the slider. InValue is Clamped between 0.f - 1.f */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Material Button")
	UE_API void SetIsPressed(bool InPressed);

public:

	/** Called when the value is changed by button. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnButtonPressedChangedEvent OnButtonPressedChangedEvent;

	/** Called when mouse capture begins. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnButtonMouseCaptureBeginEvent OnMouseCaptureBegin;

	/** Called when mouse capture ends (mouse release or any other capture loss). */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnButtonMouseCaptureEndEvent OnMouseCaptureEnd;

protected:

	// UWidget
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget

	UE_API void HandleOnPressedValueChanged(bool InPressedSate);
	UE_API void HandleOnMouseCaptureBegin();
	UE_API void HandleOnMouseCaptureEnd();

private:

	/**Default Value of the button*/
	UPROPERTY(EditAnywhere, BlueprintSetter = SetIsPressed, BlueprintGetter = GetIsPressed, Category = "Appearance")
	bool bIsPressed = false;

private:

	/** Native Slate Widget */
	TSharedPtr<SAudioMaterialButton> Button;

};

#undef UE_API
