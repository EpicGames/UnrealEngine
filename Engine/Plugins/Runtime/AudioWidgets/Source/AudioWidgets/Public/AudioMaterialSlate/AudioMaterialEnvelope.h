// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMaterialEnvelopeSettings.h"
#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "Components/Widget.h"
#include "Delegates/Delegate.h"
#include "AudioMaterialEnvelope.generated.h"

#define UE_API AUDIOWIDGETS_API

class SAudioMaterialEnvelope;

/**
 * A simple widget that shows a envelope curve Depending on given AudioMaterialEnvelopeSetings
 * Rendered by using material instead of texture.
 *
 * * No Children
 */
UCLASS(MinimalAPI)
class UAudioMaterialEnvelope : public UWidget
{
	GENERATED_BODY()

public:

	UE_API UAudioMaterialEnvelope();

	/** The Envelope's style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style", meta = (DisplayName = "Style", ShowOnlyInnerProperties))
	FAudioMaterialEnvelopeStyle WidgetStyle;

	/**Envelope settings*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Appearance")
	FAudioMaterialEnvelopeSettings EnvelopeSettings;

public:

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif

	// UWidget
	UE_API virtual void SynchronizeProperties() override;
	// End of UWidget

	// UVisual
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UVisual

protected:

	// UWidget
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget

private:

	/** Native Slate Widget */
	TSharedPtr<SAudioMaterialEnvelope> EnvelopeCurve;

};

#undef UE_API
