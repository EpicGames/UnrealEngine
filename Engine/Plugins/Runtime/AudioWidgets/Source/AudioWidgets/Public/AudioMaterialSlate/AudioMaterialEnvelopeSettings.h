// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMaterialEnvelopeSettings.generated.h"

/**
*Envelope curve types
*A: Attack
*D: Decay
*S: Sustain
*R: Release
*/
UENUM(BlueprintType)
enum class EAudioMaterialEnvelopeType : uint8
{
	AD,
	ADSR
};

USTRUCT(BlueprintType)
struct FAudioMaterialEnvelopeSettings
{
	GENERATED_BODY();

public:	
	
	/**
	* The Type of the envelope curve.
	*/
	UPROPERTY(EditAnywhere, Category = "Envelope|Type")
	EAudioMaterialEnvelopeType EnvelopeType = EAudioMaterialEnvelopeType::ADSR;

	/**
	* Curve of the envelopes attack stage.
	* Attack is the time taken for the rise of the level from zero to a given value.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Envelope|Attack")
	float AttackCurve = 1.0;

	/**
	* Value of the envelopes attack stage.
	* Attack is the time taken for the rise of the level from zero to a given value.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Envelope|Attack")
	float AttackValue = 1.0;

	/**
	* Time the Value reaches the Attack stage.
	* Attack is the time taken for the rise of the level from zero to a given value.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Envelope|Attack")
	float AttackTime = 1.0;
	
	/**
	* Curve of the envelopes Decay stage.
	* Decay is the time taken for the level to reduce from the attack level to the sustain level.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Envelope|Decay")
	float DecayCurve = 1.0;	
	
	/**
	* Time that takes to reach the level of the Sustain stage.
	* Decay is the time taken for the level to reduce from the attack level to the sustain level.
	*/	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Envelope|Decay")
	float DecayTime = 1.0;	
	
	/**
	* Value of the envelopes Sustain stage.
	* Sustain is the level maintained until release stage.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (EditCondition = "EnvelopeType==EAudioMaterialEnvelopeType::ADSR", EditConditionHides), Category = "Envelope|Sustain")
	float SustainValue = 1.0;

	/**
	* Curve of the envelopes Release stage.
	* Release is the time taken for the level to decay from sustain to zero.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (EditCondition = "EnvelopeType==EAudioMaterialEnvelopeType::ADSR", EditConditionHides), Category = "Envelope|Release")
	float ReleaseCurve = 1.0;	
	
	/**
	* Time that takes to reach zero level
	* Release is the time taken for the level to decay from sustain to zero.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (EditCondition = "EnvelopeType==EAudioMaterialEnvelopeType::ADSR", EditConditionHides), Category = "Envelope|Release")
	float ReleaseTime = 1.0;
	
};
