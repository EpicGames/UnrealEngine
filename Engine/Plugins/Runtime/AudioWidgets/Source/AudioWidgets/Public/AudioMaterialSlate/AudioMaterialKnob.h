// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "Components/Widget.h"
#include "Delegates/Delegate.h"
#include "AudioMaterialKnob.generated.h"

#define UE_API AUDIOWIDGETS_API

class SAudioMaterialKnob;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnKnobMouseCaptureBeginEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnKnobMouseCaptureEndEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnKnobValueChangedEvent, float, Value);


/**
 * A simple widget that shows a turning Knob that allows you to control the value between 0..1.
 * Knob is rendered by using material instead of texture.
 *
 * * No Children
 */
UCLASS(MinimalAPI)
class UAudioMaterialKnob : public UWidget
{
	GENERATED_BODY()

public:

	UE_API UAudioMaterialKnob(const FObjectInitializer& ObjectInitializer);

	/** The button's style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style", meta = (DisplayName = "Style", ShowOnlyInnerProperties))
	FAudioMaterialKnobStyle WidgetStyle;

public:
#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif // WITH_EDITOR

	// UWidget
	UE_API virtual void SynchronizeProperties() override;
	// End of UWidget

	// UVisual
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UVisual

	/** Get the current value of the knob.*/
	UFUNCTION(BlueprintPure, Category = "Audio Widgets| Audio Material Knob")
	UE_API float GetValue();

	/** Set the current value of the knob. InValue is Clamped between 0.f - 1.f */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Material Knob")
	UE_API void SetValue(float InValue);	
	
	/** Set the knobs tune speed. InValue is Clamped between 0.f - 1.f */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Material Knob")
	UE_API void SetTuneSpeed(float InValue);	

	/** Get the Knobs tune speed*/
	UFUNCTION(BlueprintPure, Category = "Audio Widgets| Audio Material Knob")
	UE_API float GetTuneSpeed() const;
	
	/** Set the knobs fine-tune speed. InValue is Clamped between 0.f - 1.f */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Material Knob")
	UE_API void SetFineTuneSpeed(float InValue);

	/** Get the Knobs fine-tune speed*/
	UFUNCTION(BlueprintPure, Category = "Audio Widgets| Audio Material Knob")
	UE_API float GetFineTuneSpeed() const;
	
	/** Set the knob to be interactive or fixed */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Material Knob")
	UE_API void SetLocked(bool InLocked);	

	/** Get whether the knob is interactive or fixed.*/
	UFUNCTION(BlueprintPure, Category = "Audio Widgets| Audio Material Knob")
	UE_API bool GetIsLocked() const;
	
	/** Set the knob to use steps when turning On Mouse move */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Material Knob")
	UE_API void SetMouseUsesStep(bool InUsesStep);

	/** Get whether the knob uses steps when tuning On Mouse move*/
	UFUNCTION(BlueprintPure, Category = "Audio Widgets| Audio Material Knob")
	UE_API bool GetMouseUsesStep() const;

	/** Set the amount to adjust the value when using steps*/
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Material Knob")
	UE_API void SetStepSize(float InValue);

	/** Get Step Size*/
	UFUNCTION(BlueprintPure, Category = "Audio Widgets| Audio Material Knob")
	UE_API float GetStepSize() const;

public:

	/** Called when the value is changed by knob. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnKnobValueChangedEvent OnKnobValueChanged;

	/** Called when mouse capture begins. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnKnobMouseCaptureBeginEvent OnMouseCaptureBegin;

	/** Called when mouse capture ends (mouse release or any other capture loss). */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnKnobMouseCaptureEndEvent OnMouseCaptureEnd;

protected:

	// UWidget
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget

	UE_API void HandleOnKnobValueChanged(float InValue);

private:

	/**Default Value of the Knob*/
	UPROPERTY(EditAnywhere, BlueprintSetter = SetValue, BlueprintGetter = GetValue, Category = "Appearance", meta = (UIMin = "0", UIMax = "1"))
	float Value = 1.f;

	/** The tune speed of the knob */
	UPROPERTY(EditAnywhere, BlueprintSetter = SetTuneSpeed, BlueprintGetter = GetTuneSpeed, Category = Appearance, meta = (ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1"))
	float TuneSpeed;

	/** The tune speed when fine-tuning the knob */
	UPROPERTY(EditAnywhere, BlueprintSetter = SetFineTuneSpeed, BlueprintGetter = GetFineTuneSpeed, Category = Appearance, AdvancedDisplay, meta = (ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1"))
	float FineTuneSpeed;

	/** Whether the knob is interactive or fixed. */
	UPROPERTY(EditAnywhere, BlueprintSetter = SetLocked, BlueprintGetter = GetIsLocked, Category = Appearance, AdvancedDisplay)
	bool bLocked;

	/** Sets new value if mouse position is greater/less than half the step size. */
	UPROPERTY(EditAnywhere, BlueprintSetter = SetMouseUsesStep, BlueprintGetter = GetMouseUsesStep, Category = Appearance, AdvancedDisplay)
	bool bMouseUsesStep;

	/** The amount to adjust the value by, when using steps */
	UPROPERTY(EditAnywhere, BlueprintSetter = SetStepSize, BlueprintGetter = GetStepSize, Category = Appearance, AdvancedDisplay, meta = (UIMin = "0", UIMax = "1"))
	float StepSize;

private:

	void HandleOnKnobCaptureBegin();
	void HandleOnKnobCaptureEnd();

	/** Native Slate Widget */
	TSharedPtr<SAudioMaterialKnob> Knob;

};

#undef UE_API
