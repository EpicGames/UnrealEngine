// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDefines.h"
#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "AudioMeterStyle.h"
#include "AudioMeterTypes.h"
#include "Components/Widget.h"
#include "Delegates/Delegate.h"
#include "AudioMaterialMeter.generated.h"

#define UE_API AUDIOWIDGETS_API

class SAudioMaterialMeter;

/**
 * Meter is rendered by using material instead of texture.
 *
 * * No Children
 */
UCLASS(MinimalAPI)
class UAudioMaterialMeter : public UWidget
{
	GENERATED_BODY()

public:

	UE_API UAudioMaterialMeter();

	/** The meter's style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style", meta = (DisplayName = "Style", ShowOnlyInnerProperties))
	FAudioMaterialMeterStyle WidgetStyle;

	/** The Meter's orientation. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance)
	TEnumAsByte<EOrientation> Orientation;

public:

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif

	// UWidget
	UE_API virtual void SynchronizeProperties() override;
	// End of UWidget

	// UVisual
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UVisual

	/** Gets the current linear values of the meter. */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Material Meter")
	UE_API TArray<FMeterChannelInfo> GetMeterChannelInfo() const;

	/** Sets the current meter values. */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Material Meter")
	UE_API void SetMeterChannelInfo(const TArray<FMeterChannelInfo>& InMeterChannelInfo);

public:

	DECLARE_DYNAMIC_DELEGATE_RetVal(TArray<FMeterChannelInfo>, FGetMeterChannelInfo);

	///** A bindable delegate to allow logic to drive the value of the meter */
	UPROPERTY()
	FGetMeterChannelInfo MeterChannelInfoDelegate;	

protected:

	// UWidget
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget

	PROPERTY_BINDING_IMPLEMENTATION(TArray<FMeterChannelInfo>, MeterChannelInfo);

private:

	/** Native Slate Widget */
	TSharedPtr<SAudioMaterialMeter> Meter;

	/** The current meter value to display. */
	UPROPERTY(EditAnywhere, BlueprintSetter = SetMeterChannelInfo, BlueprintGetter = GetMeterChannelInfo, Category = MeterValues)
	TArray<FMeterChannelInfo> MeterChannelInfo;

};

#undef UE_API
