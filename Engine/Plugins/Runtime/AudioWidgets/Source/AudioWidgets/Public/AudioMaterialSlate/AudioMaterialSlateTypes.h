// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsSlateTypes.h"
#include "Math/Color.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyle.h"
#include "AudioMaterialSlateTypes.generated.h"

#define UE_API AUDIOWIDGETS_API

/**
 *Base for the appearance of an Audio Material Slates 
 */
USTRUCT(BlueprintType)
struct FAudioMaterialWidgetStyle : public FSlateWidgetStyle
{
	GENERATED_BODY()

public:

	UE_API FAudioMaterialWidgetStyle();

	/** Material used to render the Slate */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayPriority = 0), Category = "Style")
	TObjectPtr<UMaterialInterface> Material = nullptr;

	/** Desired Draw size of the rendered material*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayPriority = 1), Category = "Style")
	FVector2f DesiredSize;

public:

	UE_API virtual UMaterialInstanceDynamic* CreateDynamicMaterial(UObject* InOuter) const;

};

/**
 *Represents the appearance of an Audio Material Button 
 */
USTRUCT(BlueprintType)
struct FAudioMaterialButtonStyle : public FAudioMaterialWidgetStyle
{
	GENERATED_BODY()

	UE_API FAudioMaterialButtonStyle();

	// FSlateWidgetStyle
	UE_API virtual void GetResources(TArray<const FSlateBrush*>& OutBrushes) const override;
	static UE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };
	static UE_API const FAudioMaterialButtonStyle& GetDefault();
	UE_API virtual UMaterialInstanceDynamic* CreateDynamicMaterial(UObject* InOuter) const override;

	FAudioMaterialButtonStyle& SetMaterial(UMaterialInterface* InMaterialInterface) { Material = InMaterialInterface; return *this; }

	/** The button's Main color value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Button")
	FLinearColor ButtonMainColor;
	FAudioMaterialButtonStyle& SetButtonMainColor(const FLinearColor& InColor) { ButtonMainColor = InColor; return *this; }

	/** The button color's Tint value covering one half of the gradient. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Button")
	FLinearColor ButtonMainColorTint_1;
	FAudioMaterialButtonStyle& SetButtonMainColorTint_1(const FLinearColor& InColor) { ButtonMainColorTint_1 = InColor; return *this; }

	/** The button color's Tint value covering the other half of the gradient. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Button")
	FLinearColor ButtonMainColorTint_2;
	FAudioMaterialButtonStyle& SetButtonMainColorTint_2(const FLinearColor& InColor) { ButtonMainColorTint_2 = InColor; return *this; }	

	/** The button's Accent color value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Button")
	FLinearColor ButtonAccentColor;	
	FAudioMaterialButtonStyle& SetButtonAccentColor(const FLinearColor& InColor) { ButtonAccentColor = InColor; return *this; }

	/** The button's Shadow color value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Button")
	FLinearColor ButtonShadowColor;	
	FAudioMaterialButtonStyle& SetButtonShadowColor(const FLinearColor& InColor) { ButtonShadowColor = InColor; return *this; }

	/** The button's Outline color value when Unpressed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Button")
	FLinearColor ButtonUnpressedOutlineColor;
	FAudioMaterialButtonStyle& SetButtonUnpressedOutlineColor(const FLinearColor& InColor) { ButtonUnpressedOutlineColor = InColor; return *this; }

	/** The button's Outline color value when Pressed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Button")
	FLinearColor ButtonPressedOutlineColor;
	FAudioMaterialButtonStyle& SetButtonPressedOutlineColor(const FLinearColor& InColor) { ButtonPressedOutlineColor = InColor; return *this; }

};

/**
 *Represents the appearance of an Audio Material Slider 
 */
USTRUCT(BlueprintType)
struct FAudioMaterialSliderStyle : public FAudioMaterialWidgetStyle
{
	GENERATED_BODY()

	UE_API FAudioMaterialSliderStyle();

	// FSlateWidgetStyle
	UE_API virtual void GetResources(TArray<const FSlateBrush*>& OutBrushes) const override;
	static UE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };
	static UE_API const FAudioMaterialSliderStyle& GetDefault();
	UE_API virtual UMaterialInstanceDynamic* CreateDynamicMaterial(UObject* InOuter) const override;

	FAudioMaterialSliderStyle& SetMaterial(UMaterialInterface* InMaterialInterface) { Material = InMaterialInterface; return *this; }

	/** The slider Bar's Background color value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Slider")
	FLinearColor SliderBackgroundColor;
	FAudioMaterialSliderStyle& SetSliderBarBackgroundColor(const FLinearColor& InColor) { SliderBackgroundColor = InColor; return *this; }

	/** The slider Bar's Background Accent color value. Can be thought as the slider's Inner Shadow color value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Slider")
	FLinearColor SliderBackgroundAccentColor;
	FAudioMaterialSliderStyle& SetSliderBarBackgroundAccentColor(const FLinearColor& InColor) { SliderBackgroundAccentColor = InColor; return *this; }

	/** The slider's Color value representing the slider's Output Value amount. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Slider")
	FLinearColor SliderValueMainColor;
	FAudioMaterialSliderStyle& SetSliderBarValueMainColor(const FLinearColor& InColor) { SliderValueMainColor = InColor; return *this; }

	/** The slider Handle's Main color value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Slider")
	FLinearColor SliderHandleMainColor;
	FAudioMaterialSliderStyle& SetSliderHandleMainColor(const FLinearColor& InColor) { SliderHandleMainColor = InColor; return *this; }
	
	/** The slider Handle's Outline color value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Slider")
	FLinearColor SliderHandleOutlineColor;
	FAudioMaterialSliderStyle& SetSliderHandleOutlineColor(const FLinearColor& InColor) { SliderHandleOutlineColor = InColor; return *this; }

	/** The style to use for the audio text box widget. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Label")
	FAudioTextBoxStyle TextBoxStyle;
	FAudioMaterialSliderStyle& SetTextBoxStyle(const FAudioTextBoxStyle& InTextBoxStyle) { TextBoxStyle = InTextBoxStyle; return *this; }

};

/**
 *Represents the appearance of an Audio Material Knob 
 */
USTRUCT(BlueprintType)
struct FAudioMaterialKnobStyle : public FAudioMaterialWidgetStyle
{
	GENERATED_BODY()

	UE_API FAudioMaterialKnobStyle();

	// FSlateWidgetStyle
	UE_API virtual void GetResources(TArray<const FSlateBrush*>& OutBrushes) const override;
	static UE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };
	static UE_API const FAudioMaterialKnobStyle& GetDefault();
	UE_API virtual UMaterialInstanceDynamic* CreateDynamicMaterial(UObject* InOuter) const override;

	FAudioMaterialKnobStyle& SetMaterial(UMaterialInterface* InMaterialInterface) { Material = InMaterialInterface; return *this; }

	/** The knob's Main color value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Knob")
	FLinearColor KnobMainColor;
	FAudioMaterialKnobStyle& SetKnobMainColor(const FLinearColor& InColor) { KnobMainColor = InColor; return *this; }

	/** The knob's Accent color value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Knob")
	FLinearColor KnobAccentColor;
	FAudioMaterialKnobStyle& SetKnobAccentColor(const FLinearColor& InColor) { KnobAccentColor = InColor; return *this; }
	
	/** The knob's Shadow color value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Knob")
	FLinearColor KnobShadowColor;
	FAudioMaterialKnobStyle& SetKnobShadowColor(const FLinearColor& InColor) { KnobShadowColor = InColor; return *this; }
	
	/** The knob's Smooth Bevel color value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Knob")
	FLinearColor KnobSmoothBevelColor;
	FAudioMaterialKnobStyle& SetSmoothBevelColor(const FLinearColor& InColor) { KnobSmoothBevelColor = InColor; return *this; }
	
	/** The knob's Indicator Dot color value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Knob")
	FLinearColor KnobIndicatorDotColor;
	FAudioMaterialKnobStyle& SetKnobIndicatorColor(const FLinearColor& InColor) { KnobIndicatorDotColor = InColor; return *this;}
	
	/* The knob's Edge Fill color value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Knob")
	FLinearColor KnobEdgeFillColor;
	FAudioMaterialKnobStyle& SetKnobEdgeFillColor(const FLinearColor& InColor) { KnobEdgeFillColor = InColor; return *this;}
	
	/** The knob Bar's Color value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Knob")
	FLinearColor KnobBarColor;
	FAudioMaterialKnobStyle& SetKnobBarColor(const FLinearColor& InColor) { KnobBarColor = InColor; return *this; }
	
	/** The knob Bar's Shadow color value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Knob")
	FLinearColor KnobBarShadowColor;
	FAudioMaterialKnobStyle& SetKnobBarShadowColor(const FLinearColor& InColor) { KnobBarShadowColor = InColor; return *this; }

	/** The knob Bar's Fill color value representing the Starting section of the fill. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Knob")
	FLinearColor KnobBarFillMinColor;
	FAudioMaterialKnobStyle& SetKnobBarFillMinColor(const FLinearColor& InColor) { KnobBarFillMinColor = InColor; return *this; }
	
	/** The knob Bar's Fill color value representing the Middle section of the fill. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Knob")
	FLinearColor KnobBarFillMidColor;
	FAudioMaterialKnobStyle& SetKnobFillMidColor(const FLinearColor& InColor) { KnobBarFillMidColor = InColor; return *this; }
	
	/** The knob Bar's Fill color value representing the Ending section of the fill. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Knob")
	FLinearColor KnobBarFillMaxColor;
	FAudioMaterialKnobStyle& SetKnobBarFillMaxColor(const FLinearColor& InColor) { KnobBarFillMaxColor = InColor; return *this; }
	
	/** The knob Bar Fill color's Tint value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Knob")
	FLinearColor KnobBarFillTintColor;
	FAudioMaterialKnobStyle& SetKnobBarFillTintColor(const FLinearColor& InColor) { KnobBarFillTintColor = InColor; return *this; }

	/** The style to use for the audio text box widget. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style | Label")
	FAudioTextBoxStyle TextBoxStyle;
	void SetTextBoxStyle(const FAudioTextBoxStyle& InTextBoxStyle) { TextBoxStyle = InTextBoxStyle; }

};

/**
 *Represents the appearance of an Audio Material Meter
 */
USTRUCT(BlueprintType)
struct FAudioMaterialMeterStyle : public FAudioMaterialWidgetStyle
{
	GENERATED_BODY()

	UE_API FAudioMaterialMeterStyle();

	// FSlateWidgetStyle
	UE_API virtual void GetResources(TArray<const FSlateBrush*>& OutBrushes) const override;
	static UE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };
	static UE_API const FAudioMaterialMeterStyle& GetDefault();
	UE_API virtual UMaterialInstanceDynamic* CreateDynamicMaterial(UObject* InOuter) const override;

	FAudioMaterialMeterStyle& SetMaterial(UMaterialInterface* InMaterialInterface) { Material = InMaterialInterface; return *this; }

	/** The meter's Fill color value representing the Starting section of the fill. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style")
	FLinearColor MeterFillMinColor;
	FAudioMaterialMeterStyle& SetMeterFillMinColor(const FLinearColor& InColor) { MeterFillMinColor = InColor; return *this; }

	/** The meter's Fill color value representing the Middle section of the fill. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style")
	FLinearColor MeterFillMidColor;
	FAudioMaterialMeterStyle& SetMeterFillMidColor(const FLinearColor& InColor) { MeterFillMidColor = InColor; return *this; }

	/** The meter's Fill color value representing the Ending section of the fill. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style")
	FLinearColor MeterFillMaxColor;
	FAudioMaterialMeterStyle& SetMeterFillMaxColor(const FLinearColor& InColor) { MeterFillMaxColor = InColor; return *this; }

	/** The meter's Background Fill color value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style")
	FLinearColor MeterFillBackgroundColor;
	FAudioMaterialMeterStyle& SetMeterOffFillColor(const FLinearColor& InColor) { MeterFillBackgroundColor = InColor; return *this; }

	// How much padding to add around the meter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FVector2D MeterPadding;
	FAudioMaterialMeterStyle& SetMeterpadding(const FVector2D InPadding) { MeterPadding = InPadding; return *this; }

	// The minimum and maximum value to display in dB (values are clamped in this range)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FVector2D ValueRangeDb;
	FAudioMaterialMeterStyle& SetValueRangeDb(const FVector2D& InValueRangeDb) { ValueRangeDb = InValueRangeDb; return *this; }

	// Whether or not to show the decibel scale alongside the meter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	bool bShowScale;
	FAudioMaterialMeterStyle& SetShowScale(bool bInShowScale) { bShowScale = bInShowScale; return *this; }

	// Which side to show the scale. If vertical, true means left side, false means right side. If horizontal, true means above, false means below.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	bool bScaleSide;
	FAudioMaterialMeterStyle& SetScaleSide(bool bInScaleSide) { bScaleSide = bInScaleSide; return *this; }

	// Offset for the hashes
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float ScaleHashOffset;
	FAudioMaterialMeterStyle& SetScaleHashOffset(float InScaleHashOffset) { ScaleHashOffset = InScaleHashOffset; return *this; }

	// The width of each hash mark
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float ScaleHashWidth;
	FAudioMaterialMeterStyle& SetScaleHashWidth(float InScaleHashWidth) { ScaleHashWidth = InScaleHashWidth; return *this; }

	// The height of each hash mark
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float ScaleHashHeight;
	FAudioMaterialMeterStyle& SetScaleHashHeight(float InScaleHashHeight) { ScaleHashHeight = InScaleHashHeight; return *this; }

	// How wide to draw the decibel scale, if it's enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance, meta = (UIMin = "3", ClampMin = "3", UIMax = "10"))
	int32 DecibelsPerHash;
	FAudioMaterialMeterStyle& SetDecibelsPerHash(float InDecibelsPerHash) { DecibelsPerHash = InDecibelsPerHash; return *this; }

	/** Font family and size to be used when displaying the meter scale. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateFontInfo Font;
	FAudioMaterialMeterStyle& SetFont(const FSlateFontInfo& InFont) { Font = InFont; return *this; }
	FAudioMaterialMeterStyle& SetFont(TSharedPtr<const FCompositeFont> InCompositeFont, const int32 InSize, const FName& InTypefaceFontName = NAME_None) { Font = FSlateFontInfo(InCompositeFont, InSize, InTypefaceFontName); return *this; }
	FAudioMaterialMeterStyle& SetFont(const UObject* InFontObject, const int32 InSize, const FName& InTypefaceFontName = NAME_None) { Font = FSlateFontInfo(InFontObject, InSize, InTypefaceFontName); return *this; }

	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMaterialMeterStyle& SetFont(const FName& InFontName, uint16 InSize);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMaterialMeterStyle& SetFont(const FString& InFontName, uint16 InSize);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMaterialMeterStyle& SetFont(const WIDECHAR* InFontName, uint16 InSize);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMaterialMeterStyle& SetFont(const ANSICHAR* InFontName, uint16 InSize);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMaterialMeterStyle& SetFontName(const FName& InFontName);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMaterialMeterStyle& SetFontName(const FString& InFontName);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMaterialMeterStyle& SetFontName(const WIDECHAR* InFontName);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMaterialMeterStyle& SetFontName(const ANSICHAR* InFontName);

	FAudioMaterialMeterStyle& SetFontSize(uint16 InSize) { Font.Size = InSize; return *this; }
	FAudioMaterialMeterStyle& SetTypefaceFontName(const FName& InTypefaceFontName) { Font.TypefaceFontName = InTypefaceFontName; return *this; }

};

/**
 *Represents the appearance of an Audio Material Envelope
 */
USTRUCT(BlueprintType)
struct FAudioMaterialEnvelopeStyle : public FAudioMaterialWidgetStyle
{
	GENERATED_BODY()

	UE_API FAudioMaterialEnvelopeStyle();

	// FSlateWidgetStyle
	UE_API virtual void GetResources(TArray<const FSlateBrush*>& OutBrushes) const override;
	static UE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };
	static UE_API const FAudioMaterialEnvelopeStyle& GetDefault();
	UE_API virtual UMaterialInstanceDynamic* CreateDynamicMaterial(UObject* InOuter) const override;

	FAudioMaterialEnvelopeStyle& SetMaterial(UMaterialInterface* InMaterialInterface) { Material = InMaterialInterface; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style")
	FLinearColor CurveColor;
	FAudioMaterialEnvelopeStyle& SetEnvelopeCurveColor(const FLinearColor& InColor) { CurveColor = InColor; return *this; }
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style")
	FLinearColor BackgroundColor;	
	FAudioMaterialEnvelopeStyle& SetEnvelopeBackgroundColor(const FLinearColor& InColor) { BackgroundColor = InColor; return *this; }
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style")
	FLinearColor OutlineColor;
	FAudioMaterialEnvelopeStyle& SetEnvelopeOutlineColor(const FLinearColor& InColor) { OutlineColor = InColor; return *this; }
	
};

#undef UE_API
