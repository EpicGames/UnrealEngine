// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "Components/Widget.h"
#include "Delegates/Delegate.h"
#include "AudioMaterialSlider.generated.h"

#define UE_API AUDIOWIDGETS_API

class SAudioMaterialSlider;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnSliderMouseCaptureBeginEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnSliderMouseCaptureEndEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnSliderFloatValueChangedEvent, float, Value);

/**
 * A simple widget that shows a sliding bar with a handle that allows you to control the value between 0..1.
 * Slider is rendered by using material instead of texture.
 *
 * * No Children
 */
UCLASS(MinimalAPI, BlueprintType, Blueprintable, CustomFieldNotify)
class UAudioMaterialSlider : public UWidget
{
	GENERATED_BODY()

public:

	UE_API UAudioMaterialSlider(const FObjectInitializer& ObjectInitializer);

	/** The slider's style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style", meta = (DisplayName = "Style", ShowOnlyInnerProperties))
	FAudioMaterialSliderStyle WidgetStyle;

public:

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif

	// UWidget
	UE_API virtual void SynchronizeProperties() override;
	// End of UWidget

	// UVisual
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UVisual

	/** Gets the current value of the slider.*/
	UFUNCTION(BlueprintPure, Category = "Audio Widgets| Audio Material Slider")
	UE_API float GetValue() const;

	/** Sets the current value of the slider. InValue is Clamped between 0.f - 1.f */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Material Slider")
	UE_API void SetValue(float InValue);

	/** Set the tune speed of the slider. InValue is Clamped between 0.f - 1.f */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Material Slider")
	UE_API void SetTuneSpeed(const float InValue);

	/** Get slider tune speed*/
	UFUNCTION(BlueprintPure, Category = "Audio Widgets| Audio Material Slider")
	UE_API float GetTuneSpeed() const;

	/** Set the fine-tune speed of the slider. InValue is Clamped between 0.f - 1.f */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Material Slider")
	UE_API void SetFineTuneSpeed(const float InValue);

	/** Get slider fine-tune speed*/
	UFUNCTION(BlueprintPure, Category = "Audio Widgets| Audio Material Slider")
	UE_API float GetFineTuneSpeed() const;

	/** Set the slider to be interactive or fixed */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Material Slider")
	UE_API void SetLocked(bool bInLocked);

	/** Get whether the slider is interactive or fixed.*/
	UFUNCTION(BlueprintPure, Category = "Audio Widgets| Audio Material Slider")
	UE_API bool GetIsLocked() const;

	/** Sets the slider to use steps when turning On Mouse move */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Material Slider")
	UE_API void SetMouseUsesStep(bool bInUsesStep);

	/** Get whether the slider uses steps when turning On Mouse move*/
	UFUNCTION(BlueprintPure, Category = "Audio Widgets| Audio Material Slider")
	UE_API bool GetMouseUsesStep() const;

	/** Sets the amount to adjust the value when using steps*/
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Material Slider")
	UE_API void SetStepSize(float InValue);

	/** Get Step Size*/
	UFUNCTION(BlueprintPure, Category = "Audio Widgets| Audio Material Slider")
	UE_API float GetStepSize() const;

public:

	/** Called when the value is changed by slider. */
	UPROPERTY(BlueprintAssignable, FieldNotify, Category = "Widget Event")
	FOnSliderFloatValueChangedEvent OnValueChanged;

	/** Called when mouse capture begins. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnSliderMouseCaptureBeginEvent OnMouseCaptureBegin;

	/** Called when mouse capture ends (mouse release or any other capture loss). */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnSliderMouseCaptureEndEvent OnMouseCaptureEnd;

protected:

	// UWidget
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget

	UE_API void HandleOnValueChanged(float InValue);
	UE_API void HandleOnMouseCaptureBegin();
	UE_API void HandleOnMouseCaptureEnd();

private:

	/**Default Value of the slider*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, FieldNotify, Setter = SetValue, Getter = GetValue, Category = "Appearance", meta = (UIMin = "0", UIMax = "1", AllowPrivateAccess))
	float Value = 1.f;

	/**Orientation of the slider*/
	UPROPERTY(EditAnywhere, Category = "Appearance")
	TEnumAsByte<EOrientation> Orientation = EOrientation::Orient_Horizontal;

	/** The tune speed of the slider On Mouse move */
	UPROPERTY(EditAnywhere, BlueprintSetter = SetTuneSpeed, BlueprintGetter = GetTuneSpeed, Category = Appearance, meta = (ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1"))
	float TuneSpeed;

	/** The tune speed of the slider when fine-tuning the slider On Mouse move && Left-Shift pressed */
	UPROPERTY(EditAnywhere, BlueprintSetter = SetFineTuneSpeed, BlueprintGetter = GetFineTuneSpeed, Category = Appearance, AdvancedDisplay, meta = (ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1"))
	float FineTuneSpeed;

	/** Whether the slider is interactive or fixed. */
	UPROPERTY(EditAnywhere, BlueprintSetter = SetLocked, BlueprintGetter = GetIsLocked, Category = Appearance, AdvancedDisplay)
	bool bLocked;

	/** Sets new value if mouse position is greater/less than half the step size. */
	UPROPERTY(EditAnywhere, BlueprintSetter = SetMouseUsesStep, BlueprintGetter = GetMouseUsesStep, Category = Appearance, AdvancedDisplay)
	bool bMouseUsesStep;

	/** The amount to adjust the value by, when using steps */
	UPROPERTY(EditAnywhere, BlueprintSetter = SetStepSize, BlueprintGetter = GetStepSize, Category = Appearance, AdvancedDisplay, meta = (UIMin = "0", UIMax = "1"))
	float StepSize;

private:

	/** Native Slate Widget */
	TSharedPtr<SAudioMaterialSlider> Slider;
};

#undef UE_API
