// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMaterialSlateTypes.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateWidgetStyleContainerBase.h"
#include "AudioMaterialStyleContainers.generated.h"

UCLASS(hidecategories = Object, MinimalAPI)
class UAudioMaterialKnobWidgetStyle : public USlateWidgetStyleContainerBase
{
	GENERATED_BODY()

public:

	/** The actual data describing the AudioMaterialKnob appearance. */
	UPROPERTY(Category = Appearance, EditAnywhere, meta = (ShowOnlyInnerProperties))
	FAudioMaterialKnobStyle KnobStyle;

	virtual const struct FSlateWidgetStyle* const GetStyle() const override
	{
		return static_cast<const struct FSlateWidgetStyle*>(&KnobStyle);
	}
};

UCLASS(hidecategories = Object, MinimalAPI)
class UAudioMaterialMeterWidgetStyle : public USlateWidgetStyleContainerBase
{
	GENERATED_BODY()

public:

	/** The actual data describing the AudioMaterialMeter appearance. */
	UPROPERTY(Category = Appearance, EditAnywhere, meta = (ShowOnlyInnerProperties))
	FAudioMaterialMeterStyle MeterStyle;

	virtual const struct FSlateWidgetStyle* const GetStyle() const override
	{
		return static_cast<const struct FSlateWidgetStyle*>(&MeterStyle);
	}
};

UCLASS(hidecategories = Object, MinimalAPI)
class UAudioMaterialButtonWidgetStyle : public USlateWidgetStyleContainerBase
{
	GENERATED_BODY()

public:

	/** The actual data describing the AudioMaterialButton appearance. */
	UPROPERTY(Category = Appearance, EditAnywhere, meta = (ShowOnlyInnerProperties))
	FAudioMaterialButtonStyle ButtonStyle;

	virtual const struct FSlateWidgetStyle* const GetStyle() const override
	{
		return static_cast<const struct FSlateWidgetStyle*>(&ButtonStyle);
	}
};

UCLASS(hidecategories = Object, MinimalAPI)
class UAudioMaterialSliderWidgetStyle : public USlateWidgetStyleContainerBase
{
	GENERATED_BODY()

public:

	/** The actual data describing the AudioMaterialSlider appearance. */
	UPROPERTY(Category = Appearance, EditAnywhere, meta = (ShowOnlyInnerProperties))
	FAudioMaterialSliderStyle SliderStyle;

	virtual const struct FSlateWidgetStyle* const GetStyle() const override
	{
		return static_cast<const struct FSlateWidgetStyle*>(&SliderStyle);
	}
};
