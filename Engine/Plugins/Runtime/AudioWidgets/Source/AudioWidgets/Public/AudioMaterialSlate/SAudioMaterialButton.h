// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "AudioWidgetsStyle.h"
#include "Framework/SlateDelegates.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SLeafWidget.h"

#define UE_API AUDIOWIDGETS_API

class UObject;
struct FAudioMaterialButtonStyle;

/**
 * A simple slate that renders button in single material and modifies the material on pressed state change.
 *
 */
class SAudioMaterialButton : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SAudioMaterialButton)
	: _AudioMaterialButtonStyle(&FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioMaterialButtonStyle>("AudioMaterialButton.Style"))
	{}

	/** The owner object*/
	SLATE_ARGUMENT(TWeakObjectPtr<UObject>, Owner)

	/**State of the button*/
	SLATE_ATTRIBUTE(bool, bIsPressedAttribute)

	/** The style used to draw the button. */
	SLATE_STYLE_ARGUMENT(FAudioMaterialButtonStyle, AudioMaterialButtonStyle)

	/** Called when the button's state changes. */
	SLATE_EVENT(FOnBooleanValueChanged, OnBooleanValueChanged)

	/** Invoked when the mouse is pressed and a capture begins. */
	SLATE_EVENT(FSimpleDelegate, OnMouseCaptureBegin)

	/** Invoked when mouse capture ends (mouse release or any other capture loss). */
	SLATE_EVENT(FSimpleDelegate, OnMouseCaptureEnd)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs);

	/** Press the button */
	UE_API void SetPressedState(bool InPressedState);

	/** Apply new material to be used to render the Slate.*/
	UE_API UMaterialInstanceDynamic* ApplyNewMaterial();

	/**Set desired size of the Slate*/
	UE_API void SetDesiredSizeOverride(const FVector2D InSize);

public:

	FOnBooleanValueChanged OnBooleanValueChanged;

	// Holds a delegate that is executed when the mouse is pressed and a capture begins.
	FSimpleDelegate OnMouseCaptureBegin;

	// Holds a delegate that is executed when mouse capture ends.
	FSimpleDelegate OnMouseCaptureEnd;

protected:

	// SWidget overrides
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;

private:

	/**Commits new state*/
	void CommitNewState(bool InPressedState);

private:

	// Holds the owner of the Slate
	TWeakObjectPtr<UObject> Owner;

	// Holds the Modifiable Material that represent the Button
	mutable TWeakObjectPtr<UMaterialInstanceDynamic> DynamicMaterial;

	// Holds the style for the Slate
	const FAudioMaterialButtonStyle* AudioMaterialButtonStyle = nullptr;

	//Current pressed state of this button
	TAttribute<bool> bIsPressedAttribute = false;

	// Holds the optional desired size for the Slate
	TAttribute<TOptional<FVector2D>> DesiredSizeOverride;

};

#undef UE_API
