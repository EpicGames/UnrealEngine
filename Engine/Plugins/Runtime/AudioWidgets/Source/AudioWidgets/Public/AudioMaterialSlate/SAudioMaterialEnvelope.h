// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "AudioMaterialEnvelopeSettings.h"
#include "Framework/SlateDelegates.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SLeafWidget.h"

#define UE_API AUDIOWIDGETS_API

/**
 * A simple slate that renders envelope curve in a single material and modifies the material on value change.
 *
 */
class SAudioMaterialEnvelope : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SAudioMaterialEnvelope)
	{}

	/** The owner object*/
	SLATE_ARGUMENT(TWeakObjectPtr<UObject>, Owner)

	/** Drives how the Envelope Curve is rendered*/
	SLATE_ARGUMENT(FAudioMaterialEnvelopeSettings*, EnvelopeSettings)

	/** The style used to draw the Envelope. */
	SLATE_STYLE_ARGUMENT(FAudioMaterialEnvelopeStyle, AudioMaterialEnvelopeStyle)

	SLATE_END_ARGS()

	/**
	* Construct the widget.
	*/
	UE_API void Construct(const FArguments& InArgs);

	// SWidget overrides
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;

	/** Apply new material to be used to render the Slate.*/
	UE_API UMaterialInstanceDynamic* ApplyNewMaterial();

private:

	// Holds the owner of the Slate
	TWeakObjectPtr<UObject> Owner;

	// Holds the style for the Slate
	const FAudioMaterialEnvelopeStyle* AudioMaterialEnvelopeStyle = nullptr;

	// Holds the Modifiable Material that represent the Envelope
	mutable TWeakObjectPtr<UMaterialInstanceDynamic> DynamicMaterial;

	//Holds the current Envelope settings
	const FAudioMaterialEnvelopeSettings* EnvelopeSettings = nullptr;

};

#undef UE_API
