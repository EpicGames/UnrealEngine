// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMaterialSlateTypes.h"
#include "AudioWidgetsStyle.h"
#include "Framework/SlateDelegates.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SLeafWidget.h"

#define UE_API AUDIOWIDGETS_API

class UObject;

/**
 * A simple slate that renders a knob in single material and modifies the material on value change.
 *
 */
class SAudioMaterialKnob : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SAudioMaterialKnob)
	: _TuneSpeed(0.2f)
	, _FineTuneSpeed(0.05f)
	, _IsFocusable(true)
	, _Locked(false)
	, _MouseUsesStep(false)
	, _StepSize(0.01f)
	, _AudioMaterialKnobStyle(&FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioMaterialKnobStyle>("AudioMaterialKnob.Style"))
	{}

	/** The owner object*/
	SLATE_ARGUMENT(TWeakObjectPtr<UObject>, Owner)

	/**Value of the Knob*/
	SLATE_ATTRIBUTE(float, Value)

	/** The tune speed of the knob.*/
	SLATE_ATTRIBUTE(float, TuneSpeed)

	/** The tune speed of the knob when shift is held. */
	SLATE_ATTRIBUTE(float, FineTuneSpeed)

	/** When true knob will be keyboard focusable, else only mouse-clickable and never keyboard focusable. */
	SLATE_ATTRIBUTE(bool, IsFocusable)

	/** Whether the knob is interactive or fixed. */
	SLATE_ATTRIBUTE(bool, Locked)

	/**Rotates knob in given steps. Sets new value if mouse position is greater/less than half the step size. */
	SLATE_ATTRIBUTE(bool, MouseUsesStep)

	/** StepSize */
	SLATE_ATTRIBUTE(float, StepSize)

	/** The style used to draw the knob. */
	SLATE_STYLE_ARGUMENT(FAudioMaterialKnobStyle, AudioMaterialKnobStyle)

	/** Called when the knob's state changes. */
	SLATE_EVENT(FOnFloatValueChanged, OnFloatValueChanged)

	/** Called when the value is committed (mouse capture ends) */
	SLATE_EVENT(FOnFloatValueChanged, OnValueCommitted)

	/** Invoked when the mouse is pressed and a capture begins. */
	SLATE_EVENT(FSimpleDelegate, OnMouseCaptureBegin)

	/** Invoked when mouse capture ends (mouse release or any other capture loss). */
	SLATE_EVENT(FSimpleDelegate, OnMouseCaptureEnd)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs);

	/** Set the Value attribute */
	UE_API void SetValue(const TAttribute<float>& InValueAttribute);

	/** Set the TuneSpeed attribute */
	UE_API void SetTuneSpeed(const float InMouseSpeed);

	/** Set the FineTuneSpeed attribute */
	UE_API void SetFineTuneSpeed(const float InMouseFineTuneSpeed);

	/** Set the bLocked attribute */
	UE_API void SetLocked(const bool InLocked);	
	
	/** See the bMouseUsesStep attribute */
	UE_API void SetMouseUsesStep(const bool InUsesStep);

	/** Set the StepSize attribute */
	UE_API void SetStepSize(const float InStepSize);

	/** @return Is the knob interaction locked or not?*/
	UE_API bool IsLocked() const;

	/** Apply new material to be used to render the Slate.*/
	UE_API UMaterialInstanceDynamic* ApplyNewMaterial();

	UE_API const float GetOutputValue(const float InSliderValue);
	UE_API const float GetSliderValue(const float OutputValue);

	/** Set the output range of the Knob*/
	UE_API void SetOutputRange(const FVector2D Range);

	/**Set desired size of the Slate*/
	UE_API void SetDesiredSizeOverride(const FVector2D Size);

public:

	// Holds a delegate that is executed when the knob's value changes.
	FOnFloatValueChanged OnValueChanged;

	// Holds a delegate that is executed when the knob's value is committed (mouse capture ends).
	FOnFloatValueChanged OnValueCommitted;

	// Holds a delegate that is executed when the mouse is pressed and a capture begins.
	FSimpleDelegate OnMouseCaptureBegin;

	// Holds a delegate that is executed when mouse capture ends.
	FSimpleDelegate OnMouseCaptureEnd;

protected:

	//SWidget
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UE_API virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UE_API virtual bool SupportsKeyboardFocus() const override;
	UE_API virtual bool IsInteractable() const override;
	//~SWidget

protected:

	// Range for output 
	FVector2D OutputRange = FVector2D(0.0f, 1.0f);
	const FVector2D NormalizedLinearSliderRange = FVector2D(0.0f, 1.0f);

private:

	// Holds the optional style for the Slate
	TAttribute<TOptional<FVector2D>> DesiredSizeOverride;

	/**Commits new value*/
	void CommitValue(float NewValue);

private:

	// Holds the owner of the Slate
	TWeakObjectPtr<UObject> Owner;

	// Holds the style for the Slate
	const FAudioMaterialKnobStyle* AudioMaterialKnobStyle = nullptr;

	// Holds the Modifiable Material that represent the Knob
	mutable TWeakObjectPtr<UMaterialInstanceDynamic> DynamicMaterial;

	//Holds the knobs current Value
	TAttribute<float> ValueAttribute = 1.0;

	/** Holds the amount to adjust the knob On Mouse move*/
	TAttribute<float> TuneSpeed;

	/** Holds the amount to adjust the knob On Mouse move & FineTuning */
	TAttribute<float> FineTuneSpeed;

	/** Holds a flag indicating whether knob will be keyboard focusable. */
	TAttribute<bool> bIsFocusable;

	// Holds a flag indicating whether the knob is locked.
	TAttribute<bool> bLocked;	
	
	// Holds a flag indicating whether the knob uses steps when roating on Mouse move.
	TAttribute<bool> bMouseUsesStep;

	/** Holds the amount to adjust the value when steps are used */
	TAttribute<float> StepSize;

	// The position of the mouse when it pushed down and started rotating the knob
	FVector2D MouseDownPosition;

	// the value when the mouse was pushed down
	float MouseDownValue = 0.f;

	// Holds the initial cursor in case a custom cursor has been specified, so we can restore it after dragging the slider
	EMouseCursor::Type CachedCursor;

	// the max pixels to go to min or max value (clamped to 0 or 1) in one drag period
	int32 PixelDelta = 50;

	// Whether or not we're in fine-tune mode
	bool bIsFineTune;

};

#undef UE_API
