// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "Framework/SlateDelegates.h"
#include "AudioWidgetsEnums.h"
#include "SAudioInputWidget.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SLeafWidget.h"

#define UE_API AUDIOWIDGETS_API

class SAudioTextBox;
class SAudioMaterialKnob;
class SVerticalBox;
class UObject;

/**
 * Wraps SAudioMaterialKnob and adds Label text that will show a value text.
 */
class SAudioMaterialLabeledKnob : public SAudioInputWidget
{
public:
	SLATE_BEGIN_ARGS(SAudioMaterialLabeledKnob)
	{}

	/** A value representing the normalized linear (0 - 1) knobs value position. */
	SLATE_ATTRIBUTE(float, Value)

	/** The owner object*/
	SLATE_ARGUMENT(TWeakObjectPtr<UObject>, Owner)

	/** The knob's ValueType. */
	SLATE_ARGUMENT(EAudioUnitsValueType, AudioUnitsValueType)

	/** Will the knob use Linear Output. This is used when ValueType is Volume */
	SLATE_ARGUMENT(bool, bUseLinearOutput)

	/** The style used to draw the knob. */
	SLATE_STYLE_ARGUMENT(FAudioMaterialKnobStyle, Style)

	/** Called when the knob's value is changed by tuning or typing. */
	SLATE_EVENT(FOnFloatValueChanged, OnValueChanged)

	/** Called when the value is committed from label's text field */
	SLATE_EVENT(FOnFloatValueChanged, OnValueTextCommitted)

	/** Invoked when the mouse is pressed and a capture begins. */
	SLATE_EVENT(FSimpleDelegate, OnMouseCaptureBegin)

	/** Invoked when the mouse is released and a capture ends. */
	SLATE_EVENT(FSimpleDelegate, OnMouseCaptureEnd)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs);

	/** Set the Value attribute */
	UE_API void SetValue(float InValueAttribute);

	//SAudioInputWidget
	UE_API virtual const float GetOutputValue(const float InSliderValue) override;
	UE_API virtual const float GetSliderValue(const float OutputValue) override;
	UE_API virtual const float GetOutputValueForText(const float InSliderValue);
	UE_API virtual const float GetSliderValueForText(const float OutputValue);

	/**
	 * Set the knob's linear (0-1 normalized) value.
	 */
	UE_API virtual void SetSliderValue(float InSliderValue) override;
	UE_API virtual void SetOutputRange(const FVector2D InRange) override;
	UE_API virtual void SetDesiredSizeOverride(const FVector2D Size) override;
	UE_API virtual void SetLabelBackgroundColor(FSlateColor InColor) override;
	UE_API virtual void SetUnitsText(const FText Units) override;
	UE_API virtual void SetUnitsTextReadOnly(const bool bIsReadOnly) override;
	UE_API virtual void SetShowUnitsText(const bool bShowUnitsText) override;
	//~SAudioInputWidget

public:

	// Holds a delegate that is executed when the knob's value changes.
	FOnFloatValueChanged OnValueChanged;

	// Holds a delegate that is executed when the value is committed from label's text field
	FOnFloatValueChanged OnValueTextCommitted;

	// Holds a delegate that is executed when the mouse is pressed and a capture begins.
	FSimpleDelegate OnMouseCaptureBegin;

	// Holds a delegate that is executed when the mouse is let up and a capture ends.
	FSimpleDelegate OnMouseCaptureEnd;

protected:

	//SWidget
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	//~SWidget

private:

	TAttribute<TOptional<FVector2D>> DesiredSizeOverride;

	// Holds the owner of the Slate
	TWeakObjectPtr<UObject> Owner;

	// Holds the style for the Slate
	const FAudioMaterialKnobStyle* Style = nullptr;

	//Holds the knobs current Value
	TAttribute<float> ValueAttribute = 1.0f;

	// Holds the knob's unit value type
	TAttribute<EAudioUnitsValueType> AudioUnitsValueType;

	// Widget components
	TSharedPtr<SAudioMaterialKnob> Knob;
	TSharedPtr<SAudioTextBox> Label;

	/** verticalBox that holds the widgets*/
	TSharedPtr<SVerticalBox> VerticalLayotWidget;

	// Range for output 
	FVector2D OutputRange = FVector2D(0.0f, 1.0f);
	const FVector2D NormalizedLinearSliderRange = FVector2D(0.0f, 1.0f);

	/**Hold the ref to the current Unit processor */
	TSharedPtr<FAudioUnitProcessor> AudioUnitProcessor;

};

#undef UE_API
