// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "AudioWidgetsEnums.h"
#include "AudioWidgetsStyle.h"
#include "Framework/SlateDelegates.h"
#include "SAudioInputWidget.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateWidgetStyleAsset.h"

#define UE_API AUDIOWIDGETS_API

class SAudioTextBox;
class SAudioMaterialSlider;
class SWidgetSwitcher;

/**
 * Wraps SAudioMaterialSlider and adds Label text that will show a value text.
 */
class SAudioMaterialLabeledSlider
	: public SAudioInputWidget
{
public:
	SLATE_BEGIN_ARGS(SAudioMaterialLabeledSlider)
	: _Style(&FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioMaterialSliderStyle>("AudioMaterialSlider.Style"))
	{}

	/** The style used to draw the slider. */
	SLATE_STYLE_ARGUMENT(FAudioMaterialSliderStyle, Style)

	/** The owner object*/
	SLATE_ARGUMENT(TWeakObjectPtr<UObject>, Owner)

	/** A value representing the normalized linear (0 - 1) slider value position. */
	SLATE_ATTRIBUTE(float, SliderValue)
		
	/** The slider's orientation. */
	SLATE_ARGUMENT(EOrientation, Orientation)	
	
	/** The slider's ValueType. */
	SLATE_ARGUMENT(EAudioUnitsValueType, AudioUnitsValueType)
	
	/** Will the slider use Linear Output. This is used when ValueType is Volume */
	SLATE_ARGUMENT(bool , bUseLinearOutput)
	
	/** When specified, use this as the slider's desired size */
	SLATE_ATTRIBUTE(TOptional<FVector2D>, DesiredSizeOverride)

	/** Called when the value is changed by slider or typing */
	SLATE_EVENT(FOnFloatValueChanged, OnValueChanged)

	/** Called when the value is committed from label's text field */
	SLATE_EVENT(FOnFloatValueChanged, OnValueCommitted)

	SLATE_END_ARGS()

	// Holds a delegate that is executed when the slider's value changed.
	FOnFloatValueChanged OnValueChanged;

	// Holds a delegate that is executed when the slider's value is committed (mouse capture ends).
	FOnFloatValueChanged OnValueCommitted;

	/**
	 * Construct the widget.
	 */
	UE_API virtual void Construct(const SAudioMaterialLabeledSlider::FArguments& InArgs);
	UE_API virtual const float GetOutputValue(const float InSliderValue) override;
	UE_API virtual const float GetSliderValue(const float OutputValue) override;
	UE_API virtual const float GetOutputValueForText(const float InSliderValue);
	UE_API virtual const float GetSliderValueForText(const float OutputValue);
	
	//SAudioInputWidget
	/**
	 * Set the slider's linear (0-1 normalized) value. 
	 */
	UE_API virtual void SetSliderValue(float InSliderValue) override;
	UE_API FVector2D ComputeDesiredSize(float) const;
	UE_API virtual void SetDesiredSizeOverride(const FVector2D DesiredSize) override;

	UE_API void SetOrientation(EOrientation InOrientation);
	UE_API virtual void SetOutputRange(const FVector2D InRange) override;

	// Text label functions 
	UE_API void SetLabelBackgroundColor(FSlateColor InColor) override;
	UE_API void SetUnitsText(const FText Units) override;
	UE_API void SetUnitsTextReadOnly(const bool bIsReadOnly) override;
	UE_API void SetValueTextReadOnly(const bool bIsReadOnly);
	UE_API void SetShowLabelOnlyOnHover(const bool bShowLabelOnlyOnHover);
	UE_API void SetShowUnitsText(const bool bShowUnitsText) override;
	//~SAudioInputWidget

private:

	// Holds the style for the Slider
	const FAudioMaterialSliderStyle* Style;
	
	// Holds the slider's current linear value
	TAttribute<float> SliderValueAttribute;

	// Holds the slider's orientation
	TAttribute<EOrientation> Orientation;
	
	// Holds the slider's unit value type
	TAttribute<EAudioUnitsValueType> AudioUnitsValueType;

	// Optional override for desired size 
	TAttribute<TOptional<FVector2D>> DesiredSizeOverride;

	// Label text bg color 
	TAttribute<FSlateColor> LabelBackgroundColor;

	// Widget components
	TSharedPtr<SAudioMaterialSlider> Slider;
	TSharedPtr<SAudioTextBox> Label;

	// Range for output
	FVector2D OutputRange = FVector2D(0.0f, 1.0f);

	/** Switches between the vertical and horizontal views */
	TSharedPtr<SWidgetSwitcher> LayoutWidgetSwitcher;

	/**Hold the ref to the current Unit processor */
	TSharedPtr<FAudioUnitProcessor> AudioUnitProcessor;

private:

	TSharedRef<SWidgetSwitcher> CreateWidgetLayout();
};

#undef UE_API
