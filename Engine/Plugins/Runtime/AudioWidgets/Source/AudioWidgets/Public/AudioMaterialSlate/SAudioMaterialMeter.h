// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMaterialSlateTypes.h"
#include "AudioMeterTypes.h"
#include "AudioWidgetsStyle.h"
#include "Framework/SlateDelegates.h"
#include "SAudioMeter.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"

#define UE_API AUDIOWIDGETS_API

class UWidget;
struct FMeterChannelInfo;

/**
 * A simple slate that renders the meter in single material and modifies the material on value change.
 *
 */
class SAudioMaterialMeter : public SAudioMeterBase
{
public:
	SLATE_BEGIN_ARGS(SAudioMaterialMeter)
	: _Orientation(EOrientation::Orient_Vertical)
	, _AudioMaterialMeterStyle(&FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioMaterialMeterStyle>("AudioMaterialMeter.Style"))
	{}

	/** The meter's orientation. */
	SLATE_ARGUMENT(EOrientation, Orientation)

	/** The owner object*/
	SLATE_ARGUMENT(TWeakObjectPtr<UObject>, Owner)

	/** The style used to draw the meter. */
	SLATE_STYLE_ARGUMENT(FAudioMaterialMeterStyle, AudioMaterialMeterStyle)

	/** Attribute representing the meter values */
	SLATE_ATTRIBUTE(TArray<FMeterChannelInfo>, MeterChannelInfo)

	SLATE_END_ARGS()

	/**
	* Construct the widget.
	*/
	UE_API void Construct(const FArguments& InArgs);

	//SWidget
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	//~SWidget

	/**Set the Orientation attribute*/
	UE_API void SetOrientation(EOrientation InOrientation);

	/** Apply new material to be used to render the Meter.*/
	UE_API TArray<TWeakObjectPtr<UMaterialInstanceDynamic>> ApplyNewMaterial();

	/**Set the MeterChannelInfo attribute*/
	UE_API void SetMeterChannelInfo(const TAttribute<TArray<FMeterChannelInfo>>& InMeterChannelInfo) override;

	/**Get the MeterChannelInfo attribute*/
	UE_API TArray<FMeterChannelInfo> GetMeterChannelInfo() const override;

private:

	// Returns the scale width based off font size and hash width
	float GetScaleWidth() const;

private:

	// Holds the owner of the Slate
	TWeakObjectPtr<UObject> Owner;

	// Holds the style for the Slate
	const FAudioMaterialMeterStyle* Style = nullptr;

	// Holds the Modifiable Materials that represent the Meters
	mutable TArray<TWeakObjectPtr<UMaterialInstanceDynamic>> DynamicMaterials;

	// Holds the Meter's orientation.
	EOrientation Orientation;

	//Holds the MeterChannelInfoAttributes
	TAttribute<TArray<FMeterChannelInfo>> MeterChannelInfoAttribute;

};

#undef UE_API
