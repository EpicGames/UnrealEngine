// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMaterialSlate/AudioMaterialSlateTypes.h"
#include "Framework/SlateDelegates.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SLeafWidget.h"

#define UE_API AUDIOWIDGETS_API

class UWidget;

/**
 * A simple slate that renders slider in single material and modifies the material on value change.
 *
 */
class SAudioMaterialSlider : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SAudioMaterialSlider)
	: _TuneSpeed(0.2f)
	, _FineTuneSpeed(0.05f)
	, _IsFocusable(true)
	, _Locked(false)
	, _MouseUsesStep(false)
	, _StepSize(0.01f)
	{}

	/** The owner object*/
	SLATE_ARGUMENT(TWeakObjectPtr<UObject>, Owner)

	/** The slider's orientation. */
	SLATE_ARGUMENT(EOrientation, Orientation)

	/** The tune speed of the slider handle */
	SLATE_ATTRIBUTE(float, TuneSpeed)

	/** The fine-tune speed of the slider handle */
	SLATE_ATTRIBUTE(float, FineTuneSpeed)

	/** When true slider will be keyboard focusable, else only mouse-clickable and never keyboard focusable. */
	SLATE_ATTRIBUTE(bool, IsFocusable)

	/** Whether the slider is interactive or fixed. */
	SLATE_ATTRIBUTE(bool, Locked)

	/**Moves Slider handle in given steps. Sets new value if mouse position is greater/less than half the step size. */
	SLATE_ATTRIBUTE(bool, MouseUsesStep)

	/** StepSize */
	SLATE_ATTRIBUTE(float, StepSize)

	/** The style used to draw the slider. */
	SLATE_STYLE_ARGUMENT(FAudioMaterialSliderStyle, AudioMaterialSliderStyle)

	/** A value that drives where the slider handle appears. Value is clamped between 0 and 1. */
	SLATE_ATTRIBUTE(float, ValueAttribute)

	/** Called when the value is changed by the slider. */
	SLATE_EVENT(FOnFloatValueChanged, OnValueChanged)

	/** Called when the value is committed (mouse capture ends) */
	SLATE_EVENT(FOnFloatValueChanged, OnValueCommitted)

	/** Invoked when the mouse is pressed and a capture begins. */
	SLATE_EVENT(FSimpleDelegate, OnMouseCaptureBegin)

	/** Invoked when mouse capture ends (mouse release or any other capture loss). */
	SLATE_EVENT(FSimpleDelegate, OnMouseCaptureEnd)

	SLATE_END_ARGS()

	/**
	* Construct the widget.
	*/
	UE_API void Construct(const FArguments& InArgs);

	// SWidget overrides
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;

	/** Set the Value attribute */
	UE_API void SetValue(TAttribute<float> InValueAttribute);

	/** Set the TuneSpeed attribute */
	UE_API void SetTuneSpeed(const float InMouseTuneSpeed);

	/** Set the FineTuneSpeed attribute */
	UE_API void SetFineTuneSpeed(const float InMouseFineTuneSpeed);

	/** See the bMouseUsesStep attribute */
	UE_API void SetMouseUsesStep(const bool InUsesStep);

	/** Set the StepSize attribute */
	UE_API void SetStepSize(const float InStepSize);

	/** Set the bLocked attribute */
	UE_API void SetLocked(const bool bInLocked);

	/** @return Is the knob interaction locked or not?*/
	UE_API bool IsLocked() const;

	/** Apply new material to be used to render the Slate.*/
	UE_API void ApplyNewMaterial();

	/** Set the orientation of the slider*/
	UE_API void SetOrientation(EOrientation InOrientation);

	//SWidget
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;
	UE_API virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UE_API virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	UE_API virtual bool SupportsKeyboardFocus() const override;
	UE_API virtual bool IsInteractable() const override;
	//~SWidget

public:

	// Holds a delegate that is executed when the slider's value changed.
	FOnFloatValueChanged OnValueChanged;

	// Holds a delegate that is executed when the slider's value is committed (mouse capture ends).
	FOnFloatValueChanged OnValueCommitted;

	// Holds a delegate that is executed when the mouse is pressed and a capture begins.
	FSimpleDelegate OnMouseCaptureBegin;

	// Holds a delegate that is executed when mouse capture ends.
	FSimpleDelegate OnMouseCaptureEnd;

private:

	/**
	* Commits the specified slider value.
	*
	* @param NewValue The value to commit.
	*/
	void CommitValue(float NewValue);

private:

	// Optional override for desired size 
	TAttribute<TOptional<FVector2D>> DesiredSizeOverride;

	// Holds the slider's orientation
	EOrientation Orientation;

	// Holds the owner of the Slate
	TWeakObjectPtr<UObject> Owner;

	// Holds the style for the Slate
	const FAudioMaterialSliderStyle* AudioMaterialSliderStyle = nullptr;

	// Holds the Modifiable Material that represent the slider
	mutable TWeakObjectPtr<UMaterialInstanceDynamic> DynamicMaterial;

	//Holds the current value
	TAttribute<float> ValueAttribute = 0.f;

	/** Holds the amount to adjust the slider On Mouse move*/
	TAttribute<float> TuneSpeed;

	/** Holds the amount to adjust the slider On Mouse move & FineTuning */
	TAttribute<float> FineTuneSpeed;

	/** Holds a flag indicating whether slider will be keyboard focusable. */
	TAttribute<bool> bIsFocusable;

	// Holds a flag indicating whether the slider is locked.
	TAttribute<bool> bLocked;

	// Holds a flag indicating whether the slider uses steps when roating on Mouse move.
	TAttribute<bool> bMouseUsesStep;

	/** Holds the amount to adjust the value when steps are used */
	TAttribute<float> StepSize;

	// the max pixels to go to min or max value (clamped to 0 or 1) in one drag period
	int32 PixelDelta = 50;

	// Whether or not we're in fine-tune mode	
	bool bIsFineTune = false;

	// The position of the mouse when it pushed down and started moving the slider
	FVector2D MouseDownStartPosition;

	// the value when the mouse was pushed down
	float MouseDownValue = 0.f;

	// Holds the initial cursor in case a custom cursor has been specified, so we can restore it after dragging the slider
	EMouseCursor::Type CachedCursor = EMouseCursor::None;
};

#undef UE_API
