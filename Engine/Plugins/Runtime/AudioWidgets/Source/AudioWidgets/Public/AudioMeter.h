// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioAnalyzer.h"
#include "AudioAnalyzerRack.h"
#include "AudioDefines.h"
#include "AudioMeterStyle.h"
#include "AudioMeterTypes.h"
#include "AudioMeterWidgetStyle.h"
#include "Components/Widget.h"
#include "Delegates/IDelegateInstance.h"
#include "Engine/World.h"
#include "Meter.h"
#include "SAudioMeter.h"
#include "Sound/AudioBus.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "UObject/NoExportTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/SWidget.h"

#include "AudioMeter.generated.h"

#define UE_API AUDIOWIDGETS_API

// Forward Declarations
class SAudioMaterialMeter;
class SAudioMeter;
class UWorld;

struct FAudioMaterialMeterStyle;


USTRUCT(BlueprintType)
struct FAudioMeterDefaultColorStyle : public FAudioMeterDefaultColorWidgetStyle
{
	GENERATED_BODY()

	UE_API static const FAudioMeterDefaultColorStyle& GetDefault();

	virtual const FName GetTypeName() const override { return TypeName; };

	inline static const FName TypeName = "FAudioMeterDefaultColorStyle";
};

/**
 * An audio meter widget.
 *
 * Supports displaying a slower moving peak-hold value as well as the current meter value.
 *
 * A clipping value is also displayed which shows a customizable color to indicate clipping.
 *
 * Internal values are stored and interacted with as linear volume values.
 * 
 */
UCLASS(MinimalAPI)
class UAudioMeter: public UWidget
{
	GENERATED_UCLASS_BODY()

	public:

	DECLARE_DYNAMIC_DELEGATE_RetVal(TArray<FMeterChannelInfo>, FGetMeterChannelInfo);

	/** The current meter value to display. */
	UPROPERTY(EditAnywhere, Category = MeterValues)
	TArray<FMeterChannelInfo> MeterChannelInfo;

	/** A bindable delegate to allow logic to drive the value of the meter */
	UPROPERTY()
	FGetMeterChannelInfo MeterChannelInfoDelegate;

public:
	
	/** The audio meter style */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Style", meta=( DisplayName="Style" ))
	FAudioMeterStyle WidgetStyle;

	/** The slider's orientation. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Appearance)
	TEnumAsByte<EOrientation> Orientation;

	/** The color to draw the background. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance)
	FLinearColor BackgroundColor;

	/** The color to draw the meter background. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Appearance)
	FLinearColor MeterBackgroundColor;

	/** The color to draw the meter value. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance)
	FLinearColor MeterValueColor;

	/** The color to draw the meter peak value. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance)
	FLinearColor MeterPeakColor;

	/** The color to draw the meter clipping value. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance)
	FLinearColor MeterClippingColor;

	/** The color to draw the meter scale hashes. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance)
	FLinearColor MeterScaleColor;

	/** The color to draw the meter scale label. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance)
	FLinearColor MeterScaleLabelColor;

public:

 	/** Gets the current linear value of the meter. */
 	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Meter")
	UE_API TArray<FMeterChannelInfo> GetMeterChannelInfo() const;
 
 	/** Sets the current meter values. */
 	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Meter")
 	UE_API void SetMeterChannelInfo(const TArray<FMeterChannelInfo>& InMeterChannelInfo);

	/** Sets the background color */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Meter")
	UE_API void SetBackgroundColor(FLinearColor InValue);

	/** Sets the meter background color */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Meter")
	UE_API void SetMeterBackgroundColor(FLinearColor InValue);

	/** Sets the meter value color */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Meter")
	UE_API void SetMeterValueColor(FLinearColor InValue);

	/** Sets the meter peak color */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Meter")
	UE_API void SetMeterPeakColor(FLinearColor InValue);

	/** Sets the meter clipping color */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Meter")
	UE_API void SetMeterClippingColor(FLinearColor InValue);

	/** Sets the meter scale color */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Meter")
	UE_API void SetMeterScaleColor(FLinearColor InValue);

	/** Sets the meter scale color */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Meter")
	UE_API void SetMeterScaleLabelColor(FLinearColor InValue);

	// UWidget interface
	UE_API virtual void SynchronizeProperties() override;
	// End of UWidget interface

	// UVisual interface
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UVisual interface

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif

protected:
	/** Native Slate Widget */
	TSharedPtr<SAudioMeter> MyAudioMeter;

	// UWidget interface
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	// End of UWidget interface

	PROPERTY_BINDING_IMPLEMENTATION(TArray<FMeterChannelInfo>, MeterChannelInfo);
};

namespace AudioWidgets
{
	class FAudioMeter : public IAudioAnalyzerRackUnit
	{
	public:
		static UE_API const FAudioAnalyzerRackUnitTypeInfo RackUnitTypeInfo;

		UE_DEPRECATED(5.4, "Use the FAudioMeter constructor that uses Audio::FDeviceId.")
		UE_API FAudioMeter(int32 InNumChannels, UWorld& InWorld, TObjectPtr<UAudioBus> InExternalAudioBus = nullptr); 
		
		//** OPTIONAL PARAM InExternalAudioBus: An audio meter can be constructed from this audio bus.*/
		UE_API FAudioMeter(const int32 InNumChannels, const Audio::FDeviceId InAudioDeviceId, TObjectPtr<UAudioBus> InExternalAudioBus = nullptr, const FAudioMeterDefaultColorStyle* AudioMeterColorStyle = nullptr);

		//** Constructs the Meter using AudioMaterialMeter with the given style. OPTIONAL PARAM InExternalAudioBus: An audio meter can be constructed from this audio bus.
		UE_API FAudioMeter(const int32 InNumChannels, const Audio::FDeviceId InAudioDeviceId, const FAudioMaterialMeterStyle& AudioMaterialMeterStyle, TObjectPtr<UAudioBus> InExternalAudioBus = nullptr);

		UE_API ~FAudioMeter();

		UE_API UAudioBus* GetAudioBus() const;

		UE_API TSharedRef<SAudioMeter> GetWidget() const;

		template<class T>
		TSharedRef<T> GetWidget() const
		{
			return StaticCastSharedRef<T>(Widget->AsShared());
		};

		UE_DEPRECATED(5.4, "Use the Init method that uses Audio::FDeviceId.")
		UE_API void Init(int32 InNumChannels, UWorld& InWorld, TObjectPtr<UAudioBus> InExternalAudioBus = nullptr);

		UE_API void Init(const int32 InNumChannels, const Audio::FDeviceId InAudioDeviceId, TObjectPtr<UAudioBus> InExternalAudioBus);

		// Begin IAudioAnalyzerRackUnit overrides.
		virtual void SetAudioBusInfo(const FAudioBusInfo& AudioBusInfo) override
		{
			Init(AudioBusInfo.AudioBus->GetNumChannels(), AudioBusInfo.AudioDeviceId, AudioBusInfo.AudioBus);
		}

		UE_API virtual TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args) const override;
		// End IAudioAnalyzerRackUnit overrides.

	protected:
		UE_API void OnMeterOutput(UMeterAnalyzer* InMeterAnalyzer, int32 ChannelIndex, const FMeterResults& InMeterResults);

	private:
		static UE_API TSharedRef<IAudioAnalyzerRackUnit> MakeRackUnit(const FAudioAnalyzerRackUnitConstructParams& Params);

		UE_API void Teardown();

		/** Metasound analyzer object. */
		TStrongObjectPtr<UMeterAnalyzer> Analyzer;

		/** The audio bus used for analysis. */
		TStrongObjectPtr<UAudioBus> AudioBus;

		/** Cached channel info for the meter. */
		TArray<FMeterChannelInfo> ChannelInfo;

		/** Handle for results delegate for MetaSound meter analyzer. */
		FDelegateHandle ResultsDelegateHandle;

		/** Meter settings. */
		TStrongObjectPtr<UMeterSettings> Settings;

		/** MetaSound Output Meter widget */
		TSharedPtr<SAudioMeterBase> Widget;

		bool bUseExternalAudioBus = false;
	};
} // namespace AudioWidgets

#undef UE_API
