// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMeterWidgetStyle.h"

#include "AudioMeterStyle.generated.h"

#define UE_API AUDIOWIDGETS_API

/**
 * Represents the appearance of an SAudioMeter
 */
USTRUCT(BlueprintType)
struct FAudioMeterStyle : public FAudioMeterWidgetStyle
{
	GENERATED_BODY()

	static UE_API const FAudioMeterStyle& GetDefault();

	UE_API virtual void GetResources(TArray<const FSlateBrush*>& OutBrushes) const override;

	virtual const FName GetTypeName() const override { return TypeName; };

	inline static const FName TypeName = "FAudioMeterStyle";

	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMeterStyle& SetFont(const FName& InFontName, uint16 InSize);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMeterStyle& SetFont(const FString& InFontName, uint16 InSize);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMeterStyle& SetFont(const WIDECHAR* InFontName, uint16 InSize);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMeterStyle& SetFont(const ANSICHAR* InFontName, uint16 InSize);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMeterStyle& SetFontName(const FName& InFontName);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMeterStyle& SetFontName(const FString& InFontName);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMeterStyle& SetFontName(const WIDECHAR* InFontName);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMeterStyle& SetFontName(const ANSICHAR* InFontName);
};

#undef UE_API
