// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMeterChannelInfo.h"

#include "AudioMeterTypes.generated.h"

USTRUCT(BlueprintType)
struct FMeterChannelInfo : public FAudioMeterChannelInfo
{
	GENERATED_BODY()
};
