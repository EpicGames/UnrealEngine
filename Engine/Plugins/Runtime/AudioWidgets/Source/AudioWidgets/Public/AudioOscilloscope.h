// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioAnalyzerRack.h"
#include "AudioDefines.h"
#include "AudioOscilloscopePanelStyle.h"
#include "AudioWidgetsEnums.h"
#include "SAudioOscilloscopePanelWidget.h"
#include "Sound/AudioBus.h"
#include "UObject/StrongObjectPtr.h"
#include "WaveformAudioSamplesDataProvider.h"

#define UE_API AUDIOWIDGETS_API

class SAudioOscilloscopePanelWidget;

namespace AudioWidgets
{
	class FWaveformAudioSamplesDataProvider;

	class FAudioOscilloscope : public IAudioAnalyzerRackUnit
	{
	public:
		static UE_API const FAudioAnalyzerRackUnitTypeInfo RackUnitTypeInfo;

		UE_API FAudioOscilloscope(Audio::FDeviceId InAudioDeviceId,
			const uint32 InNumChannels,
			const float InTimeWindowMs,
			const float InMaxTimeWindowMs,
			const float InAnalysisPeriodMs,
			const EAudioPanelLayoutType InPanelLayoutType,
			const FAudioOscilloscopePanelStyle* InOscilloscopePanelStyle = nullptr,
			TObjectPtr<UAudioBus> InExternalAudioBus = nullptr);

		UE_API void CreateAudioBus(const uint32 InNumChannels);

		UE_API void CreateDataProvider(Audio::FDeviceId InAudioDeviceId,
			const float InTimeWindowMs,
			const float InMaxTimeWindowMs,
			const float InAnalysisPeriodMs,
			const EAudioPanelLayoutType InPanelLayoutType);

		UE_API void CreateOscilloscopeWidget(const uint32 InNumChannels, const EAudioPanelLayoutType InPanelLayoutType, const FAudioOscilloscopePanelStyle* InOscilloscopePanelStyle = nullptr);

		UE_API virtual void StartProcessing() override;
		UE_API virtual void StopProcessing() override;

		UE_API UAudioBus* GetAudioBus() const;
		UE_API TSharedRef<SWidget> GetPanelWidget() const;

		// Begin IAudioAnalyzerRackUnit overrides.
		UE_API virtual void SetAudioBusInfo(const FAudioBusInfo& AudioBusInfo) override;
		UE_API virtual TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args) const override;
		// End IAudioAnalyzerRackUnit overrides.

	private:
		static TSharedRef<IAudioAnalyzerRackUnit> MakeRackUnit(const FAudioAnalyzerRackUnitConstructParams& Params);
		static constexpr float RackUnitTimeWindowMs = 10.0f;
		static constexpr float RackUnitMaxTimeWindowMs = 10.0f;
		static constexpr float RackUnitAnalysisPeriodMs = 10.0f;
		static constexpr EAudioPanelLayoutType RackUnitPanelLayoutType = EAudioPanelLayoutType::Basic;

		FAudioOscilloscopePanelStyle OscilloscopePanelStyle;

		TSharedPtr<FWaveformAudioSamplesDataProvider> AudioSamplesDataProvider = nullptr;
		TSharedPtr<SAudioOscilloscopePanelWidget> OscilloscopePanelWidget      = nullptr;

		TStrongObjectPtr<UAudioBus> AudioBus = nullptr;
	};
} // namespace AudioWidgets

#undef UE_API
