// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioOscilloscopeEnums.generated.h"

UENUM(BlueprintType)
enum class EAudioOscilloscopeTriggerMode : uint8
{
	None    UMETA(DisplayName = "None"),
	Rising  UMETA(DisplayName = "Rising"),
	Falling UMETA(DisplayName = "Falling")
};
