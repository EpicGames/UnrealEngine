// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsSlateTypes.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/StyleDefaults.h"
#include "TriggerThresholdLineStyle.h"

#include "AudioOscilloscopePanelStyle.generated.h"

#define UE_API AUDIOWIDGETS_API

DECLARE_MULTICAST_DELEGATE_OneParam(FOnNewTimeRulerStyle,        FFixedSampleSequenceRulerStyle)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnNewValueGridOverlayStyle, FSampledSequenceValueGridOverlayStyle)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnNewWaveformViewerStyle,   FSampledSequenceViewerStyle)

/**
* Represents the appearance of an SAudioOscilloscopePanelWidget
*/
USTRUCT(BlueprintType)
struct FAudioOscilloscopePanelStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	FAudioOscilloscopePanelStyle()
	{
		TimeRulerStyle = FFixedSampleSequenceRulerStyle()
		.SetHandleColor(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f))
		.SetTicksColor(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f))
		.SetTicksTextColor(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f))
		.SetHandleColor(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f))
		.SetFontSize(10);

		ValueGridStyle = FSampledSequenceValueGridOverlayStyle()
		.SetGridColor(FLinearColor(0.05f, 0.05f, 0.05f, 0.5f))
		.SetGridThickness(1.0f)
		.SetLabelTextColor(FLinearColor(0.442708f, 0.442708f, 0.442708f, 1.0f))
		.SetLabelTextFontSize(10.0f);

		WaveViewerStyle = FSampledSequenceViewerStyle()
		.SetSequenceColor(FLinearColor(0.1f, 1.0f, 0.55f, 1.0f))
		.SetBackgroundColor(FLinearColor(0.01f, 0.01f, 0.01f, 1.0f))
		.SetSequenceLineThickness(2.0f)
		.SetSampleMarkersSize(0.0f)
		.SetMajorGridLineColor(FLinearColor(0.1f, 0.1f, 0.1f, 1.0f))
		.SetMinorGridLineColor(FLinearColor(0.05f, 0.05f, 0.05f, 1.0f))
		.SetZeroCrossingLineColor(FLinearColor(0.3f, 0.3f, 0.3f, 1.0f))
		.SetZeroCrossingLineThickness(1.0f);

		TriggerThresholdLineStyle = FTriggerThresholdLineStyle()
		.SetLineColor(FLinearColor(1.0f, 0.91f, 0.34f));
	}

	virtual ~FAudioOscilloscopePanelStyle() = default;

	virtual const FName GetTypeName() const override { return TypeName; }
	static const FAudioOscilloscopePanelStyle& GetDefault() { static FAudioOscilloscopePanelStyle Default; return Default; }

	/** The time rule style. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FFixedSampleSequenceRulerStyle TimeRulerStyle;
	FAudioOscilloscopePanelStyle& SetTimeRulerStyle(const FFixedSampleSequenceRulerStyle& InTimeRulerStyle) { TimeRulerStyle = InTimeRulerStyle; return *this; }

	/** The value grid style. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSampledSequenceValueGridOverlayStyle ValueGridStyle;
	FAudioOscilloscopePanelStyle& SetValueGridStyle(const FSampledSequenceValueGridOverlayStyle& InValueGridStyle) { ValueGridStyle = InValueGridStyle; return *this; }

	/** The waveform view style. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSampledSequenceViewerStyle WaveViewerStyle;
	FAudioOscilloscopePanelStyle& SetWaveViewerStyle(const FSampledSequenceViewerStyle& InWaveViewerStyle) { WaveViewerStyle = InWaveViewerStyle; return *this; }

	/** The triggerthreshold line style. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FTriggerThresholdLineStyle TriggerThresholdLineStyle;
	FAudioOscilloscopePanelStyle& SetTriggerThresholdLineStyle(const FTriggerThresholdLineStyle& InTriggerThresholdLineStyle) { TriggerThresholdLineStyle = InTriggerThresholdLineStyle; return *this; }

	inline static FName TypeName = FName("FAudioOscilloscopePanelStyle");

	UE_API static FOnNewTimeRulerStyle        OnNewTimeRulerStyle;
	UE_API static FOnNewValueGridOverlayStyle OnNewValueGridOverlayStyle;
	UE_API static FOnNewWaveformViewerStyle   OnNewWaveformViewerStyle;
};

#undef UE_API
