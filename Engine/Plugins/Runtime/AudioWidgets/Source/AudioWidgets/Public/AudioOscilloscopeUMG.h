// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioOscilloscopeEnums.h"
#include "AudioOscilloscopePanelStyle.h"
#include "AudioWidgetsEnums.h"
#include "Components/Widget.h"
#include "FixedSampledSequenceView.h"
#include "SampledSequenceDisplayUnit.h"
#include "SAudioOscilloscopePanelWidget.h"
#include "Sound/AudioBus.h"

#include "AudioOscilloscopeUMG.generated.h"

#define UE_API AUDIOWIDGETS_API

namespace AudioWidgets { class FWaveformAudioSamplesDataProvider; }
class UAudioBus;
class UWorld;

/**
 * An oscilloscope UMG widget.
 *
 * Supports displaying waveforms from incoming audio samples.
 * 
 */
UCLASS(MinimalAPI)
class UAudioOscilloscope: public UWidget
{
	GENERATED_UCLASS_BODY()

public:
	DECLARE_DYNAMIC_DELEGATE_RetVal(TArray<float>, FGetOscilloscopeAudioSamples);

	// UWidget overrides
	UE_API virtual void SynchronizeProperties() override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif

	/** Starts the oscilloscope processing. */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Oscilloscope")
	UE_API void StartProcessing();

	/** Stops the oscilloscope processing. */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Oscilloscope")
	UE_API void StopProcessing();

	/** The oscilloscope panel style */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Style", meta=(DisplayName="Style"))
	FAudioOscilloscopePanelStyle OscilloscopeStyle;

	/** The audio bus used to obtain audio samples for the oscilloscope */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Oscilloscope Values", meta = (DesignerRebuild = "True"))
	TObjectPtr<UAudioBus> AudioBus = nullptr;

	/** The max time window in milliseconds. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Oscilloscope Values", meta = (UIMin = 10.0, UIMax = 5000.0, ClampMin = 10.0, ClampMax = 5000.0))
	float MaxTimeWindowMs = 5000.0f;

	/** The time window in milliseconds. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Oscilloscope Values", meta = (UIMin = 10.0, ClampMin = 10.0))
	float TimeWindowMs = 10.0f;

	/** The analysis period in milliseconds. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Oscilloscope Values", meta = (UIMin = 10, UIMax = 1000, ClampMin = 10, ClampMax = 1000))
	float AnalysisPeriodMs = 10.0f;

	/** Show/Hide the time grid. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Oscilloscope Values")
	bool bShowTimeGrid = true;

	/** Define the time grid labels unit. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Oscilloscope Values")
	EXAxisLabelsUnit TimeGridLabelsUnit = EXAxisLabelsUnit::Samples;

	/** Show/Hide the amplitude grid. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Oscilloscope Values")
	bool bShowAmplitudeGrid = true;

	/** Show/Hide the amplitude labels. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Oscilloscope Values")
	bool bShowAmplitudeLabels = true;

	/** Define the amplitude grid labels unit. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Oscilloscope Values")
	EYAxisLabelsUnit AmplitudeGridLabelsUnit = EYAxisLabelsUnit::Linear;

	/** The trigger detection behavior. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Oscilloscope Values", meta = (EditCondition = "CanTriggeringBeSet()", EditConditionHides))
	EAudioOscilloscopeTriggerMode TriggerMode = EAudioOscilloscopeTriggerMode::None;

	/** The trigger threshold position in the Y axis. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Oscilloscope Values", meta = (UIMin = -1, UIMax = 1, ClampMin = -1, ClampMax = 1, EditCondition = "CanTriggeringBeSet() && TriggerMode != EAudioOscilloscopeTriggerMode::None", EditConditionHides))
	float TriggerThreshold = 0.0f;

	/** Show/Hide advanced panel layout. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Oscilloscope Values", meta = (DesignerRebuild = "True"))
	EAudioPanelLayoutType PanelLayoutType = EAudioPanelLayoutType::Basic;

	/** The channel to analyze with the oscilloscope (only available if PanelLayoutType is set to "Advanced"). */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Oscilloscope Values", meta = (EditCondition = "PanelLayoutType == EAudioPanelLayoutType::Advanced", EditConditionHides))
	int32 ChannelToAnalyze = 1;

private:
	void CreateDummyOscilloscopeWidget();
	void CreateDataProvider();
	void CreateOscilloscopeWidget();

	// UWidget overrides
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
	
	UFUNCTION()
	bool CanTriggeringBeSet();

	// The underlying audio samples data provider
	TSharedPtr<AudioWidgets::FWaveformAudioSamplesDataProvider> AudioSamplesDataProvider;

	// Native Slate Widget
	TSharedPtr<SAudioOscilloscopePanelWidget> OscilloscopePanelWidget;

	// Dummy waveform data to display if audio bus is not set
	static constexpr uint32 DummySampleRate   = 48000;
	static constexpr int32 DummyMaxNumSamples = DummySampleRate * 5;
	static constexpr int32 DummyNumChannels   = 1;

	int32 NumChannels = DummyNumChannels;

	TArray<float> DummyAudioSamples;
	FFixedSampledSequenceView DummyDataView;
};

#undef UE_API
