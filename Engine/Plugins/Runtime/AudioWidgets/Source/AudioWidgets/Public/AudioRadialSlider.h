// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsSlateTypes.h"
#include "Components/Widget.h"
#include "AudioRadialSlider.generated.h"

#define UE_API AUDIOWIDGETS_API

class SAudioRadialSlider;
enum EAudioRadialSliderLayout : int;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnAudioRadialSliderMouseCaptureBeginEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnAudioRadialSliderMouseCaptureEndEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnAudioRadialSliderValueChangedEvent, float, Value);

/**
 * An audio radial slider widget. 
 */
UCLASS(MinimalAPI)
class UAudioRadialSlider : public UWidget
{
	GENERATED_UCLASS_BODY()

public:
	/** The normalized linear (0 - 1) slider value position. */
	UPROPERTY(EditAnywhere, Category = Appearance, meta = (UIMin = "0", UIMax = "1"))
	float Value;

	/** A bindable delegate to allow logic to drive the value of the widget */
	UPROPERTY()
	FGetFloat ValueDelegate;

	/** The layout of the widget (position of text label). */
	UPROPERTY(EditAnywhere, Category = Appearance)
	TEnumAsByte<EAudioRadialSliderLayout> WidgetLayout;

	/** The color to draw the widget background. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor CenterBackgroundColor;
	
	/** The color to draw the slider progress bar in. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor SliderProgressColor;

	/** The color to draw the slider bar in. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor SliderBarColor;

	/** Start and end of the hand as a ratio to the slider radius (so 0.0 to 1.0 is from the slider center to the handle). */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FVector2D HandStartEndRatio;

	/** The text label units */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FText UnitsText;

	/** The color to draw the text label background. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor TextLabelBackgroundColor;

	/** If true, show text label only on hover; if false always show label. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	bool ShowLabelOnlyOnHover;

	/** Whether to show the units part of the text label. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	bool ShowUnitsText;

	/** Whether to set the units part of the text label read only. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	bool IsUnitsTextReadOnly;

	/** Whether to set the value part of the text label read only. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	bool IsValueTextReadOnly;

	/** The slider thickness. */
	UPROPERTY(EditAnywhere, Category = Appearance, meta = (ClampMin = "0.0"))
	float SliderThickness;

	/** Output range */
	UPROPERTY(EditAnywhere, Category = Behavior)
	FVector2D OutputRange;

	/** Called when the value is changed by slider or typing. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnAudioRadialSliderValueChangedEvent OnValueChanged;

	/** Called when a mouse drag begins. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnAudioRadialSliderMouseCaptureBeginEvent OnMouseCaptureBegin;

	/** Called when a mouse drag ends. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnAudioRadialSliderMouseCaptureEndEvent OnMouseCaptureEnd;

public: 
	/** Get output value from normalized linear (0 - 1) based on internal lin to output mapping. */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Radial Slider")
	UE_API float GetOutputValue(const float InSliderValue);

	/** Get normalized linear (0 - 1) slider value from output based on internal lin to output mapping. */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Radial Slider")
	UE_API float GetSliderValue(const float OutputValue);

	/** Sets the widget layout */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Radial Slider")
	UE_API void SetWidgetLayout(EAudioRadialSliderLayout InLayout);

	/** Sets the label background color */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Radial Slider")
	UE_API void SetCenterBackgroundColor(FLinearColor InValue);

	/** Sets the slider progress color */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Radial Slider")
	UE_API void SetSliderProgressColor(FLinearColor InValue);

	/** Sets the slider bar color */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Radial Slider")
	UE_API void SetSliderBarColor(FLinearColor InValue);

	/** Sets the start and end of the hand as a ratio to the slider radius (so 0.0 to 1.0 is from the slider center to the handle). */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Radial Slider")
	UE_API void SetHandStartEndRatio(const FVector2D InHandStartEndRatio);

	/** Sets the label background color */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Radial Slider")
	UE_API void SetTextLabelBackgroundColor(FSlateColor InColor);

	/** Sets the units text */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Radial Slider")
	UE_API void SetUnitsText(const FText Units);
	
	/** Sets whether the units text is read only*/
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Radial Slider")
	UE_API void SetUnitsTextReadOnly(const bool bIsReadOnly);

	/** Sets whether the value text is read only*/
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Radial Slider")
	UE_API void SetValueTextReadOnly(const bool bIsReadOnly);
	
	/** If true, show text label only on hover; if false always show label. */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Radial Slider")
	UE_API void SetShowLabelOnlyOnHover(const bool bShowLabelOnlyOnHover);
	
	/** Sets whether to show the units text */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Radial Slider")
	UE_API void SetShowUnitsText(const bool bShowUnitsText);

	/** Sets the slider thickness */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Radial Slider")
	UE_API void SetSliderThickness(const float InThickness);
	
	/** Sets the output range */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Radial Slider")
	UE_API void SetOutputRange(const FVector2D InOutputRange);

	// UWidget interface
	UE_API virtual void SynchronizeProperties() override;
	// End of UWidget interface

	// UVisual interface
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UVisual interface

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif

	protected:
	FAudioRadialSliderStyle WidgetStyle;

	/** Native Slate Widget */
	TSharedPtr<SAudioRadialSlider> MyAudioRadialSlider;

	UE_API void HandleOnValueChanged(float InValue);
	UE_API void HandleOnMouseCaptureBegin();
	UE_API void HandleOnMouseCaptureEnd();

	// UWidget interface
	UE_API virtual TSharedRef<SWidget> RebuildWidget();
	// End of UWidget interface
	PROPERTY_BINDING_IMPLEMENTATION(float, Value);

};

/**
 * An audio slider widget, for use with volume.
 */
UCLASS(MinimalAPI)
class UAudioVolumeRadialSlider : public UAudioRadialSlider
{
	GENERATED_UCLASS_BODY()

protected:
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
};

/**
 * An audio slider widget, for use with frequency.
 */
UCLASS(MinimalAPI)
class UAudioFrequencyRadialSlider : public UAudioRadialSlider
{
	GENERATED_UCLASS_BODY()

protected:
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
};

#undef UE_API
