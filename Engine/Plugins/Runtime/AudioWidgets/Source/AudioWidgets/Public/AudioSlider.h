// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDefines.h"
#include "AudioWidgetsSlateTypes.h"
#include "Components/Slider.h"
#include "AudioSlider.generated.h"

#define UE_API AUDIOWIDGETS_API

class UCurveFloat;

class SAudioSliderBase;

/**
 * An audio slider widget. 
 */
UCLASS(MinimalAPI, Abstract)
class UAudioSliderBase: public UWidget
{
	GENERATED_UCLASS_BODY()

public:
	/** The normalized linear (0 - 1) slider value. */
	UPROPERTY(EditAnywhere, Category = Appearance, meta = (UIMin = "0", UIMax = "1"))
	float Value;

	/** The text label units */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FText UnitsText;

	/** The color to draw the text label background. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor TextLabelBackgroundColor;

	/** A bindable delegate for the TextLabelBackgroundColor. */
	UPROPERTY()
	FGetLinearColor TextLabelBackgroundColorDelegate;

	/** If true, show text label only on hover; if false always show label. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	bool ShowLabelOnlyOnHover;

	/** Whether to show the units part of the text label. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	bool ShowUnitsText;

	/** Whether to set the units part of the text label read only. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	bool IsUnitsTextReadOnly;

	/** Whether to set the value part of the text label read only. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	bool IsValueTextReadOnly;

	/** A bindable delegate to allow logic to drive the value of the widget */
	UPROPERTY()
	FGetFloat ValueDelegate;

	/** The color to draw the slider background. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor SliderBackgroundColor;

	/** A bindable delegate for the SliderBackgroundColor. */
	UPROPERTY()
	FGetLinearColor SliderBackgroundColorDelegate;

	/** The color to draw the slider bar. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor SliderBarColor;

	/** A bindable delegate for the SliderBarColor. */
	UPROPERTY()
	FGetLinearColor SliderBarColorDelegate;

	/** The color to draw the slider thumb. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor SliderThumbColor;

	/** A bindable delegate for the SliderThumbColor. */
	UPROPERTY()
	FGetLinearColor SliderThumbColorDelegate;

	/** The color to draw the widget background. */
	UPROPERTY(EditAnywhere, Category = Appearance)
	FLinearColor WidgetBackgroundColor;

	/** A bindable delegate for the WidgetBackgroundColor. */
	UPROPERTY()
	FGetLinearColor WidgetBackgroundColorDelegate;

public:
	/** Get output value from normalized linear (0 - 1) based on internal lin to output mapping. */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Slider")
	UE_API float GetOutputValue(const float InSliderValue);

	/** Get normalized linear (0 - 1) value from output based on internal lin to output mapping. */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Slider", meta=(DeprecatedFunction, DeprecationMessage="5.1 - GetLinValue is deprecated, please use GetSliderValue instead."))
	UE_API float GetLinValue(const float OutputValue);

	/** Get normalized linear (0 - 1) slider value from output based on internal lin to output mapping. */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Slider")
	UE_API float GetSliderValue(const float OutputValue);

	/** Sets the label background color */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Slider")
	UE_API void SetTextLabelBackgroundColor(FSlateColor InColor);

	/** Sets the units text */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Slider")
	UE_API void SetUnitsText(const FText Units);
	
	/** Sets whether the units text is read only*/
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Slider")
	UE_API void SetUnitsTextReadOnly(const bool bIsReadOnly);

	/** Sets whether the value text is read only*/
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Slider")
	UE_API void SetValueTextReadOnly(const bool bIsReadOnly);
	
	/** If true, show text label only on hover; if false always show label. */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Slider")
	UE_API void SetShowLabelOnlyOnHover(const bool bShowLabelOnlyOnHover);
	
	/** Sets whether to show the units text */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Slider")
	UE_API void SetShowUnitsText(const bool bShowUnitsText);

	/** Sets the slider background color */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Slider")
	UE_API void SetSliderBackgroundColor(FLinearColor InValue);

	/** Sets the slider bar color */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Slider")
	UE_API void SetSliderBarColor(FLinearColor InValue);

	/** Sets the slider thumb color */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Slider")
	UE_API void SetSliderThumbColor(FLinearColor InValue);

	/** Sets the widget background color */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Audio Slider")
	UE_API void SetWidgetBackgroundColor(FLinearColor InValue);

	/** The slider's orientation. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Appearance)
	TEnumAsByte<EOrientation> Orientation;

	/** Called when the value is changed by slider or typing. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnFloatValueChangedEvent OnValueChanged;

	/** Called when a mouse drag begins. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnMouseCaptureBeginEvent OnMouseCaptureBegin;

	/** Called when a mouse drag ends. */
	UPROPERTY(BlueprintAssignable, Category = "Widget Event")
	FOnMouseCaptureEndEvent OnMouseCaptureEnd;

	// UWidget interface
	UE_API virtual void SynchronizeProperties() override;
	// End of UWidget interface

	// UVisual interface
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;
	// End of UVisual interface

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif

protected:
	FAudioSliderStyle WidgetStyle;
	
	/** Native Slate Widget */
	TSharedPtr<SAudioSliderBase> MyAudioSlider;

	UE_API void HandleOnValueChanged(float InValue);
	UE_API void HandleOnMouseCaptureBegin();
	UE_API void HandleOnMouseCaptureEnd();

	// UWidget interface
	UE_API virtual TSharedRef<SWidget> RebuildWidget();
	// End of UWidget interface

	PROPERTY_BINDING_IMPLEMENTATION(float, Value);
	PROPERTY_BINDING_IMPLEMENTATION(FLinearColor, TextLabelBackgroundColor);
	PROPERTY_BINDING_IMPLEMENTATION(FLinearColor, SliderBackgroundColor);
	PROPERTY_BINDING_IMPLEMENTATION(FLinearColor, SliderBarColor);
	PROPERTY_BINDING_IMPLEMENTATION(FLinearColor, SliderThumbColor);
	PROPERTY_BINDING_IMPLEMENTATION(FLinearColor, WidgetBackgroundColor);
};

/**
 * An audio slider widget with customizable curves.
 */
UCLASS(MinimalAPI)
class UAudioSlider : public UAudioSliderBase
{
	GENERATED_UCLASS_BODY()

	/** Curves for mapping linear to output values. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Behavior")
	TWeakObjectPtr<const UCurveFloat> LinToOutputCurve;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Behavior")
	TWeakObjectPtr <const UCurveFloat> OutputToLinCurve;

protected:
	UE_API virtual void SynchronizeProperties() override;
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
};

/**
 * An audio slider widget with default customizable curves for volume (dB).
 */
UCLASS(MinimalAPI)
class UAudioVolumeSlider : public UAudioSlider
{
	GENERATED_UCLASS_BODY()
protected:
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
};

/**
 * An audio slider widget, for use with frequency. 
 */
UCLASS(MinimalAPI)
class UAudioFrequencySlider : public UAudioSliderBase
{
	GENERATED_UCLASS_BODY()
	
	/** Frequency output range */
	UPROPERTY(EditAnywhere, Category = Behavior)
	FVector2D OutputRange = FVector2D(MIN_FILTER_FREQUENCY, MAX_FILTER_FREQUENCY);
protected:
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;
};

#undef UE_API
