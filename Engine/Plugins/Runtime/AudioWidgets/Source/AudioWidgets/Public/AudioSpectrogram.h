// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioAnalyzerRack.h"
#include "AudioSpectrumAnalyzer.h"
#include "SAudioSpectrogram.h"

#include "AudioSpectrogram.generated.h"

#define UE_API AUDIOWIDGETS_API

namespace AudioWidgets
{
	/**
	 * Constructor parameters for the analyzer.
	 */
	struct FAudioSpectrogramParams
	{
		int32 NumChannels = 1;
		Audio::FDeviceId AudioDeviceId = INDEX_NONE;
		TObjectPtr<UAudioBus> ExternalAudioBus = nullptr;

		TAttribute<EAudioSpectrumAnalyzerType> AnalyzerType = EAudioSpectrumAnalyzerType::FFT;
		TAttribute<EFFTSize> FFTAnalyzerFFTSize = EFFTSize::Max;
		TAttribute<EConstantQFFTSizeEnum> CQTAnalyzerFFTSize = EConstantQFFTSizeEnum::XXLarge;
		TAttribute<EAudioSpectrogramFrequencyAxisPixelBucketMode> FrequencyAxisPixelBucketMode = EAudioSpectrogramFrequencyAxisPixelBucketMode::Average;
		TAttribute<EAudioSpectrogramFrequencyAxisScale> FrequencyAxisScale = EAudioSpectrogramFrequencyAxisScale::Logarithmic;
		TAttribute<EAudioColorGradient> ColorMap = EAudioColorGradient::BlackToWhite;
		TAttribute<EOrientation> Orientation = EOrientation::Orient_Horizontal;

		FOnAnalyzerTypeMenuEntryClicked OnAnalyzerTypeMenuEntryClicked;
		FOnFFTAnalyzerFFTSizeMenuEntryClicked OnFFTAnalyzerFFTSizeMenuEntryClicked;
		FOnCQTAnalyzerFFTSizeMenuEntryClicked OnCQTAnalyzerFFTSizeMenuEntryClicked;
		FOnSpectrogramFrequencyAxisPixelBucketModeMenuEntryClicked OnFrequencyAxisPixelBucketModeMenuEntryClicked;
		FOnSpectrogramFrequencyAxisScaleMenuEntryClicked OnFrequencyAxisScaleMenuEntryClicked;
		FOnSpectrogramColorMapMenuEntryClicked OnColorMapMenuEntryClicked;
		FOnSpectrogramOrientationMenuEntryClicked OnOrientationMenuEntryClicked;
	};

	/**
	 * Owns an analyzer and a corresponding Slate widget for displaying the resulting spectra.
	 * Can either create an Audio Bus to analyze, or analyze the given Bus.
	 */
	class FAudioSpectrogram : public IAudioAnalyzerRackUnit
	{
	public:
		static UE_API const FAudioAnalyzerRackUnitTypeInfo RackUnitTypeInfo;
		
		UE_API FAudioSpectrogram(const FAudioSpectrogramParams& Params);
		UE_API ~FAudioSpectrogram();

		UE_API UAudioBus* GetAudioBus() const;

		UE_API TSharedRef<SWidget> GetWidget() const;

		UE_API void Init(int32 InNumChannels, Audio::FDeviceId InAudioDeviceId, TObjectPtr<UAudioBus> InExternalAudioBus = nullptr);

		// Begin IAudioAnalyzerRackUnit overrides.
		virtual void SetAudioBusInfo(const FAudioBusInfo& AudioBusInfo) override
		{
			Init(AudioBusInfo.AudioBus->GetNumChannels(), AudioBusInfo.AudioDeviceId, AudioBusInfo.AudioBus);
		}

		UE_API virtual TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args) const override;
		// End IAudioAnalyzerRackUnit overrides.

	protected:
		UE_API void StartAnalyzing(const EAudioSpectrumAnalyzerType InAnalyzerType);
		UE_API void StopAnalyzing();

		UE_API void OnSpectrumResults(USynesthesiaSpectrumAnalyzer* InSpectrumAnalyzer, int32 ChannelIndex, const TArray<FSynesthesiaSpectrumResults>& InSpectrumResultsArray);
		UE_API void OnConstantQResults(UConstantQAnalyzer* InSpectrumAnalyzer, int32 ChannelIndex, const TArray<FConstantQResults>& InSpectrumResultsArray);

		UE_API void ExtendSpectrumPlotContextMenu(FMenuBuilder& MenuBuilder);
		UE_API void BuildAnalyzerTypeSubMenu(FMenuBuilder& SubMenu);
		UE_API void BuildFFTSizeSubMenu(FMenuBuilder& SubMenu);

		UE_API EActiveTimerReturnType Update(double InCurrentTime, float InDeltaTime);

	private:
		static UE_API TSharedRef<IAudioAnalyzerRackUnit> MakeRackUnit(const FAudioAnalyzerRackUnitConstructParams& Params);

		UE_API void CreateSynesthesiaSpectrumAnalyzer();
		UE_API void ReleaseSynesthesiaSpectrumAnalyzer();

		UE_API void CreateConstantQAnalyzer();
		UE_API void ReleaseConstantQAnalyzer();

		UE_API void Teardown();

		/** Audio analyzer objects. */
		TStrongObjectPtr<USynesthesiaSpectrumAnalyzer> SpectrumAnalyzer;
		TStrongObjectPtr<UConstantQAnalyzer> ConstantQAnalyzer;

		/** The audio bus used for analysis. */
		TStrongObjectPtr<UAudioBus> AudioBus;

		/** Handles for results delegate for analyzers. */
		FDelegateHandle SpectrumResultsDelegateHandle;
		FDelegateHandle ConstantQResultsDelegateHandle;

		/** Analyzer settings. */
		TStrongObjectPtr<USynesthesiaSpectrumAnalysisSettings> SpectrumAnalysisSettings;
		TStrongObjectPtr<UConstantQSettings> ConstantQSettings;

		/** Slate widget for spectrum display */
		TSharedPtr<SAudioSpectrogram> Widget;
		TSharedPtr<const FExtensionBase> ContextMenuExtension;
		TSharedPtr<FActiveTimerHandle> ActiveTimer;

		Audio::FDeviceId AudioDeviceId = INDEX_NONE;
		bool bUseExternalAudioBus = false;

		TOptional<EAudioSpectrumAnalyzerType> ActiveAnalyzerType;
		TAttribute<EAudioSpectrumAnalyzerType> AnalyzerType;
		TAttribute<EFFTSize> FFTAnalyzerFFTSize;
		TAttribute<EConstantQFFTSizeEnum> CQTAnalyzerFFTSize;

		FOnAnalyzerTypeMenuEntryClicked OnAnalyzerTypeMenuEntryClicked;
		FOnFFTAnalyzerFFTSizeMenuEntryClicked OnFFTAnalyzerFFTSizeMenuEntryClicked;
		FOnCQTAnalyzerFFTSizeMenuEntryClicked OnCQTAnalyzerFFTSizeMenuEntryClicked;
	};
} // namespace AudioWidgets

USTRUCT()
struct FSpectrogramRackUnitSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, config, Category = Spectrogram)
	EAudioSpectrumAnalyzerType AnalyzerType = EAudioSpectrumAnalyzerType::FFT;

	UPROPERTY(EditAnywhere, config, Category = Spectrogram, meta = (DisplayName = "FFT Size (FFT Analyzer)"))
	EFFTSize FFTAnalyzerFFTSize = EFFTSize::Max;

	UPROPERTY(EditAnywhere, config, Category = Spectrogram, meta = (DisplayName = "FFT Size (CQT Analyzer)"))
	EConstantQFFTSizeEnum CQTAnalyzerFFTSize = EConstantQFFTSizeEnum::XXLarge;

	UPROPERTY(EditAnywhere, config, Category = Spectrogram)
	EAudioSpectrogramFrequencyAxisPixelBucketMode PixelPlotMode = EAudioSpectrogramFrequencyAxisPixelBucketMode::Average;

	UPROPERTY(EditAnywhere, config, Category = Spectrogram)
	EAudioSpectrogramFrequencyAxisScale FrequencyScale = EAudioSpectrogramFrequencyAxisScale::Logarithmic;

	UPROPERTY(EditAnywhere, config, Category = Spectrogram)
	EAudioColorGradient ColorMap = EAudioColorGradient::BlackToWhite;

	UPROPERTY(EditAnywhere, config, Category = Spectrogram)
	TEnumAsByte<EOrientation> Orientation = EOrientation::Orient_Horizontal;
};

#undef UE_API
