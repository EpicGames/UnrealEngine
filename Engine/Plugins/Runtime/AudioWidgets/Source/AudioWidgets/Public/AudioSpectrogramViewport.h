// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioColorMapper.h"
#include "Containers/RingBuffer.h"
#include "PixelFormat.h"
#include "Rendering/RenderingCommon.h"
#include "Sound/SoundSubmix.h"
#include "Textures/SlateUpdatableTexture.h"
#include "AudioSpectrogramViewport.generated.h"

UENUM(BlueprintType)
enum class EAudioSpectrogramFrequencyAxisScale : uint8
{
	Linear,
	Logarithmic,
};

UENUM(BlueprintType)
enum class EAudioSpectrogramFrequencyAxisPixelBucketMode : uint8
{
	Sample UMETA(ToolTip = "Plot one data point per frequency axis pixel bucket only, choosing the data point nearest the pixel center."),
	Peak UMETA(ToolTip = "Plot one data point per frequency axis pixel bucket only, choosing the data point with the highest sound level."),
	Average UMETA(ToolTip = "Plot the average of the data points in each frequency axis pixel bucket."),
};

/**
 * Parameters for the spectrogram viewport.
 */
struct FAudioSpectrogramViewportRenderParams
{
	int NumRows = 0;
	int NumPixelsPerRow = 0;
	float ViewMinFrequency = 0.0f;
	float ViewMaxFrequency = 0.0f;
	float ColorMapMinSoundLevel = 0.0f;
	float ColorMapMaxSoundLevel = 0.0f;
	EAudioColorGradient ColorMap = EAudioColorGradient::BlackToWhite;
	EAudioSpectrogramFrequencyAxisScale FrequencyAxisScale = EAudioSpectrogramFrequencyAxisScale::Logarithmic;
	EAudioSpectrogramFrequencyAxisPixelBucketMode FrequencyAxisPixelBucketMode = EAudioSpectrogramFrequencyAxisPixelBucketMode::Average;
};

/**
 * Struct for passing spectrum data for one frame of the spectrogram
 */
struct FAudioSpectrogramFrameData
{
	TConstArrayView<float> SpectrumValues;
	EAudioSpectrumType SpectrumType = EAudioSpectrumType::PowerSpectrum;
	float MinFrequency = 0.0f;
	float MaxFrequency = 0.0f;
	bool bLogSpacedFreqencies = false;
};

/**
 * Maintains a history of spectrogram frames, and presents their visualization by implementing ISlateViewport.
 */
class FAudioSpectrogramViewport : public ISlateViewport, public TSharedFromThis<FAudioSpectrogramViewport>
{
public:
	FAudioSpectrogramViewport();
	~FAudioSpectrogramViewport();

	/* ISlateViewport interface */
	virtual FIntPoint GetSize() const override;
	virtual class FSlateShaderResource* GetViewportRenderTargetTexture() const override { return (UpdatableTexture) ? UpdatableTexture->GetSlateResource() : nullptr; }
	virtual bool RequiresVsync() const override { return false; }
	virtual bool AllowScaling() const override { return true; }

	// The render params must be set before the FAudioSpectrogramViewport will do anything.
	void SetRenderParams(const FAudioSpectrogramViewportRenderParams& InRenderParams);

	// Add the given spectrum frame data to the history buffer.
	void AddFrame(const FAudioSpectrogramFrameData& SpectrogramFrameData);

private:
	// If row data or render params have changed, we must update the texture data. If render params change, we must also invalidate any previously cached pixel data for all rows.
	void UpdateTextureData(const bool bInvalidateCachedPixelData = false);

	/**
	 * Stores one frame of spectrogram data. Also generates the visual representation of this data as colored pixels, according to the given render params.
	 */
	class FHistoryFrameData : public FNoncopyable
	{
	public:
		// Construct a history row to store the given spectrum data. Optionally also specify the pixel bucket mode to pre-allocate space for mips if required.
		FHistoryFrameData(const FAudioSpectrogramFrameData& SpectrogramFrameData, const EAudioSpectrogramFrequencyAxisPixelBucketMode InFrequencyAxisPixelBucketMode = EAudioSpectrogramFrequencyAxisPixelBucketMode::Sample);

		// Include a move constructor due to being non-copyable.
		FHistoryFrameData(FHistoryFrameData&& Other);

		// Generate pixel data for this frame according to the given render params.
		void UpdateCachedPixels(const FAudioSpectrogramViewportRenderParams& InRenderParams);

		// Read the last generated pixel data.
		TConstArrayView<FColor> GetCachedPixels() const { return CachedPixels; }

	private:
		// Allocate the necessary space for the mip chain if required.
		void SetMipChainLengthUninitialized(const EAudioSpectrogramFrequencyAxisPixelBucketMode InFrequencyAxisPixelBucketMode);

		// Generates mip maps if required.
		void UpdateMipChain(const EAudioSpectrogramFrequencyAxisPixelBucketMode InFrequencyAxisPixelBucketMode);

		// We have mips if the mip chain is larger than mip zero.
		bool HasMips() const { return (SoundLevelsMipChain.Num() > NumSoundLevels); }

		// Get a read-only view of the mip data for the given mip size.
		TConstArrayView<float> GetMipFromMipSize(const int MipSize) const;

		// Sample a fully interpolated value from possibly between two mips, at the given normalized read position.
		float GetMipInterpolatedSoundLevel(const float DesiredMipSize, const float NormalizedDataPos) const;

		// Helper function for sampling from the given mip data at the given normalized read position. Data is linearly interpolated.
		static float GetInterpolatedSoundLevel(const TConstArrayView<float> MipData, const float NormalizedDataPos);

		// Convert magnitude values to decibel values. db = 20 * log10(val)
		static void ArrayMagnitudeToDecibel(TConstArrayView<float> InValues, TArrayView<float> OutValues, float InMinimumDb);

		// Convert power values to decibel values. db = 10 * log10(val)
		static void ArrayPowerToDecibel(TConstArrayView<float> InValues, TArrayView<float> OutValues, float InMinimumDb);

		// Immutable data describing this spectrogram frame:
		const int32 NumSoundLevels;
		const float SpectrumDataMinFrequency;
		const float SpectrumDataMaxFrequency;
		const bool bSpectrumDataHasLogSpacedFreqencies;

		// The first NumSoundLevels values are the full resolution spectrogram data for this frame, and considered immutable. Following that may also be lower resolution mips generated for the given current pixel bucket mode.
		TArray<float> SoundLevelsMipChain;
		EAudioSpectrogramFrequencyAxisPixelBucketMode SoundLevelsMipChainPixelBucketMode;

		// The most recently generated pixel data.
		TArray<FColor> CachedPixels;
	};

	FAudioSpectrogramViewportRenderParams RenderParams;
	TRingBuffer<FHistoryFrameData> History;
	FSlateUpdatableTexture* UpdatableTexture = nullptr;
	EPixelFormat UpdatableTexturePixelFormat = PF_Unknown;
};

