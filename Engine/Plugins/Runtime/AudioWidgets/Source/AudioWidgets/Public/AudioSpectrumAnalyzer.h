// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioAnalyzerRack.h"
#include "ConstantQ.h"
#include "SAudioSpectrumPlot.h"
#include "Sound/AudioBus.h"
#include "SynesthesiaSpectrumAnalysis.h"
#include "UObject/StrongObjectPtr.h"
#include "AudioSpectrumAnalyzer.generated.h"

#define UE_API AUDIOWIDGETS_API

class UWorld;

UENUM(BlueprintType)
enum class EAudioSpectrumAnalyzerBallistics : uint8
{
	Analog,
	Digital,
};

UENUM(BlueprintType)
enum class EAudioSpectrumAnalyzerType : uint8
{
	FFT UMETA(ToolTip = "Fast Fourier Transform"),
	CQT UMETA(ToolTip = "Constant-Q Transform"),
};

DECLARE_DELEGATE_OneParam(FOnBallisticsMenuEntryClicked, EAudioSpectrumAnalyzerBallistics);
DECLARE_DELEGATE_OneParam(FOnAnalyzerTypeMenuEntryClicked, EAudioSpectrumAnalyzerType);
DECLARE_DELEGATE_OneParam(FOnFFTAnalyzerFFTSizeMenuEntryClicked, EFFTSize);
DECLARE_DELEGATE_OneParam(FOnCQTAnalyzerFFTSizeMenuEntryClicked, EConstantQFFTSizeEnum);

namespace AudioWidgets
{
	/**
	 * Constructor parameters for the analyzer.
	 */
	struct FAudioSpectrumAnalyzerParams
	{
		int32 NumChannels = 1;
		Audio::FDeviceId AudioDeviceId = INDEX_NONE;
		TObjectPtr<UAudioBus> ExternalAudioBus = nullptr;

		TAttribute<EAudioSpectrumAnalyzerBallistics> Ballistics = EAudioSpectrumAnalyzerBallistics::Digital;
		TAttribute<EAudioSpectrumAnalyzerType> AnalyzerType = EAudioSpectrumAnalyzerType::CQT;
		TAttribute<EFFTSize> FFTAnalyzerFFTSize = EFFTSize::Max;
		TAttribute<EConstantQFFTSizeEnum> CQTAnalyzerFFTSize = EConstantQFFTSizeEnum::XXLarge;
		TAttribute<float> TiltExponent = 0.0f;
		TAttribute<EAudioSpectrumPlotFrequencyAxisPixelBucketMode> FrequencyAxisPixelBucketMode = EAudioSpectrumPlotFrequencyAxisPixelBucketMode::Average;
		TAttribute<EAudioSpectrumPlotFrequencyAxisScale> FrequencyAxisScale = EAudioSpectrumPlotFrequencyAxisScale::Logarithmic;
		TAttribute<bool> bDisplayFrequencyAxisLabels = false;
		TAttribute<bool> bDisplaySoundLevelAxisLabels = false;
		
		FOnBallisticsMenuEntryClicked OnBallisticsMenuEntryClicked;
		FOnAnalyzerTypeMenuEntryClicked OnAnalyzerTypeMenuEntryClicked;
		FOnFFTAnalyzerFFTSizeMenuEntryClicked OnFFTAnalyzerFFTSizeMenuEntryClicked;
		FOnCQTAnalyzerFFTSizeMenuEntryClicked OnCQTAnalyzerFFTSizeMenuEntryClicked;
		FOnTiltSpectrumMenuEntryClicked OnTiltSpectrumMenuEntryClicked;
		FOnFrequencyAxisPixelBucketModeMenuEntryClicked OnFrequencyAxisPixelBucketModeMenuEntryClicked;
		FOnFrequencyAxisScaleMenuEntryClicked OnFrequencyAxisScaleMenuEntryClicked;
		FOnDisplayAxisLabelsButtonToggled OnDisplayFrequencyAxisLabelsButtonToggled;
		FOnDisplayAxisLabelsButtonToggled OnDisplaySoundLevelAxisLabelsButtonToggled;

		const FAudioSpectrumPlotStyle* PlotStyle = nullptr;
	};

	/**
	 * Owns an analyzer and a corresponding Slate widget for displaying the resulting spectrum.
	 * Exponential time-smoothing is applied to the spectrum.
	 * Can either create an Audio Bus to analyze, or analyze the given Bus.
	 */
	class FAudioSpectrumAnalyzer : public IAudioAnalyzerRackUnit
	{
	public:
		static UE_API const FAudioAnalyzerRackUnitTypeInfo RackUnitTypeInfo;

		UE_API FAudioSpectrumAnalyzer(const FAudioSpectrumAnalyzerParams& Params);
		UE_API FAudioSpectrumAnalyzer(int32 InNumChannels, Audio::FDeviceId InAudioDeviceId, TObjectPtr<UAudioBus> InExternalAudioBus = nullptr);
		UE_API ~FAudioSpectrumAnalyzer();

		UE_API UAudioBus* GetAudioBus() const;

		UE_API TSharedRef<SWidget> GetWidget() const;

		UE_API void Init(int32 InNumChannels, Audio::FDeviceId InAudioDeviceId, TObjectPtr<UAudioBus> InExternalAudioBus = nullptr);

		// Begin IAudioAnalyzerRackUnit overrides.
		virtual void SetAudioBusInfo(const FAudioBusInfo& AudioBusInfo) override
		{
			Init(AudioBusInfo.AudioBus->GetNumChannels(), AudioBusInfo.AudioDeviceId, AudioBusInfo.AudioBus);
		}

		UE_API virtual TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args) const override;
		// End IAudioAnalyzerRackUnit overrides.

	protected:
		UE_API void StartAnalyzing(const EAudioSpectrumAnalyzerType InAnalyzerType);
		UE_API void StopAnalyzing();

		UE_API void OnSpectrumResults(USynesthesiaSpectrumAnalyzer* InSpectrumAnalyzer, int32 ChannelIndex, const TArray<FSynesthesiaSpectrumResults>& InSpectrumResultsArray);
		UE_API void OnConstantQResults(UConstantQAnalyzer* InSpectrumAnalyzer, int32 ChannelIndex, const TArray<FConstantQResults>& InSpectrumResultsArray);
		UE_API void UpdateARSmoothing(const float TimeStamp, TConstArrayView<float> SquaredMagnitudes);

		UE_API FAudioPowerSpectrumData GetAudioSpectrumData();

		UE_API void ExtendSpectrumPlotContextMenu(FMenuBuilder& MenuBuilder);
		UE_API void BuildBallisticsSubMenu(FMenuBuilder& SubMenu);
		UE_API void BuildAnalyzerTypeSubMenu(FMenuBuilder& SubMenu);
		UE_API void BuildFFTSizeSubMenu(FMenuBuilder& SubMenu);

		UE_API void UpdateAnalyzerSettings();

	private:
		static UE_API TSharedRef<IAudioAnalyzerRackUnit> MakeRackUnit(const FAudioAnalyzerRackUnitConstructParams& Params);

		UE_API void CreateSynesthesiaSpectrumAnalyzer();
		UE_API void ReleaseSynesthesiaSpectrumAnalyzer();
			
		UE_API void CreateConstantQAnalyzer();
		UE_API void ReleaseConstantQAnalyzer();

		UE_API void Teardown();

		/** Audio analyzer objects. */
		TStrongObjectPtr<USynesthesiaSpectrumAnalyzer> SpectrumAnalyzer;
		TStrongObjectPtr<UConstantQAnalyzer> ConstantQAnalyzer;

		/** The audio bus used for analysis. */
		TStrongObjectPtr<UAudioBus> AudioBus;

		/** Meaning of spectrum data. */
		TArray<float> CenterFrequencies;

		/** Cached spectrum data, with AR smoothing applied. */
		TArray<float> ARSmoothedSquaredMagnitudes;

		/** Handles for results delegate for analyzers. */
		FDelegateHandle SpectrumResultsDelegateHandle;
		FDelegateHandle ConstantQResultsDelegateHandle;

		/** Analyzer settings. */
		TStrongObjectPtr<USynesthesiaSpectrumAnalysisSettings> SpectrumAnalysisSettings;
		TStrongObjectPtr<UConstantQSettings> ConstantQSettings;

		/** Slate widget for spectrum display */
		TSharedPtr<SAudioSpectrumPlot> Widget;
		TSharedPtr<const FExtensionBase> ContextMenuExtension;

		Audio::FDeviceId AudioDeviceId = INDEX_NONE;
		bool bUseExternalAudioBus = false;

		TOptional<EAudioSpectrumAnalyzerType> ActiveAnalyzerType;
		TOptional<float> PrevTimeStamp;
		float WindowCompensationPowerGain = 1.0f;
		float AttackTimeMsec = 300.0f;
		float ReleaseTimeMsec = 300.0f;
		TAttribute<EAudioSpectrumAnalyzerBallistics> Ballistics;
		TAttribute<EAudioSpectrumAnalyzerType> AnalyzerType;
		TAttribute<EFFTSize> FFTAnalyzerFFTSize;
		TAttribute<EConstantQFFTSizeEnum> CQTAnalyzerFFTSize;

		FOnBallisticsMenuEntryClicked OnBallisticsMenuEntryClicked;
		FOnAnalyzerTypeMenuEntryClicked OnAnalyzerTypeMenuEntryClicked;
		FOnFFTAnalyzerFFTSizeMenuEntryClicked OnFFTAnalyzerFFTSizeMenuEntryClicked;
		FOnCQTAnalyzerFFTSizeMenuEntryClicked OnCQTAnalyzerFFTSizeMenuEntryClicked;
	};
} // namespace AudioWidgets

USTRUCT()
struct FSpectrumAnalyzerRackUnitSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, config, Category = SpectrumAnalyzer)
	EAudioSpectrumAnalyzerBallistics Ballistics = EAudioSpectrumAnalyzerBallistics::Digital;

	UPROPERTY(EditAnywhere, config, Category = SpectrumAnalyzer)
	EAudioSpectrumAnalyzerType AnalyzerType = EAudioSpectrumAnalyzerType::CQT;

	UPROPERTY(EditAnywhere, config, Category = SpectrumAnalyzer, meta = (DisplayName = "FFT Size (FFT Analyzer)"))
	EFFTSize FFTAnalyzerFFTSize = EFFTSize::Max;

	UPROPERTY(EditAnywhere, config, Category = SpectrumAnalyzer, meta = (DisplayName = "FFT Size (CQT Analyzer)"))
	EConstantQFFTSizeEnum CQTAnalyzerFFTSize = EConstantQFFTSizeEnum::XXLarge;

	UPROPERTY(EditAnywhere, config, Category = SpectrumAnalyzer)
	EAudioSpectrumPlotTilt TiltSpectrum = EAudioSpectrumPlotTilt::NoTilt;

	UPROPERTY(EditAnywhere, config, Category = SpectrumAnalyzer)
	EAudioSpectrumPlotFrequencyAxisPixelBucketMode PixelPlotMode = EAudioSpectrumPlotFrequencyAxisPixelBucketMode::Average;

	UPROPERTY(EditAnywhere, config, Category = SpectrumAnalyzer)
	EAudioSpectrumPlotFrequencyAxisScale FrequencyScale = EAudioSpectrumPlotFrequencyAxisScale::Logarithmic;

	UPROPERTY(EditAnywhere, config, Category = SpectrumAnalyzer)
	bool bDisplayFrequencyAxisLabels = false;

	UPROPERTY(EditAnywhere, config, Category = SpectrumAnalyzer)
	bool bDisplaySoundLevelAxisLabels = false;
};

#undef UE_API
