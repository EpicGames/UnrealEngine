// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Fonts/SlateFontInfo.h"
#include "Styling/StyleColors.h"
#include "Styling/StyleDefaults.h"
#include "Styling/SlateWidgetStyle.h"
#include "AudioSpectrumPlotStyle.generated.h"

#define UE_API AUDIOWIDGETS_API

/**
 * Represents the appearance of an SAudioSpectrumPlot
 */
USTRUCT(BlueprintType)
struct FAudioSpectrumPlotStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	UE_API FAudioSpectrumPlotStyle();

	virtual ~FAudioSpectrumPlotStyle() = default;

	static UE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static UE_API const FAudioSpectrumPlotStyle& GetDefault();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor BackgroundColor = FStyleColors::Background;
	FAudioSpectrumPlotStyle& SetBackgroundColor(const FSlateColor& InBackgroundColor) { BackgroundColor = InBackgroundColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor GridColor = FMath::Lerp(FStyleColors::Background.GetSpecifiedColor(), FStyleColors::Foreground.GetSpecifiedColor(), 0.1f);
	FAudioSpectrumPlotStyle& SetGridColor(const FSlateColor& InGridColor) { GridColor = InGridColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor AxisLabelColor = FStyleColors::Foreground;
	FAudioSpectrumPlotStyle& SetAxisLabelColor(const FSlateColor& InAxisLabelColor) { AxisLabelColor = InAxisLabelColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateFontInfo AxisLabelFont = FStyleDefaults::GetFontInfo(5);
	FAudioSpectrumPlotStyle& SetAxisLabelFont(const FSlateFontInfo& InAxisLabelFont) { AxisLabelFont = InAxisLabelFont; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor SpectrumColor = FStyleColors::Foreground;
	FAudioSpectrumPlotStyle& SetSpectrumColor(const FSlateColor& InSpectrumColor) { SpectrumColor = InSpectrumColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor CrosshairColor = FStyleColors::Foreground;
	FAudioSpectrumPlotStyle& SetCrosshairColor(const FSlateColor& InCrosshairColor) { CrosshairColor = InCrosshairColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateFontInfo CrosshairLabelFont = FStyleDefaults::GetFontInfo(8);
	FAudioSpectrumPlotStyle& SetCrosshairLabelFont(const FSlateFontInfo& InCrosshairLabelFont) { CrosshairLabelFont = InCrosshairLabelFont; return *this; }
};

#undef UE_API
