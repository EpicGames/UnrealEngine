// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsSlateTypes.h"
#include "SampledSequenceVectorViewerStyle.h"
#include "Styling/SlateWidgetStyle.h"

#include "AudioVectorscopePanelStyle.generated.h"

#define UE_API AUDIOWIDGETS_API

DECLARE_MULTICAST_DELEGATE_OneParam(FOnNewValueGridOverlayStyle, FSampledSequenceValueGridOverlayStyle)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnNewVectorViewerStyle,     FSampledSequenceVectorViewerStyle)

/**
* Represents the appearance of an SAudioVectorscopePanelWidget
*/
USTRUCT(BlueprintType)
struct FAudioVectorscopePanelStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	FAudioVectorscopePanelStyle()
	{
		ValueGridStyle = FSampledSequenceValueGridOverlayStyle()
		.SetGridColor(FLinearColor(0.05f, 0.05f, 0.05f, 0.5f))
		.SetGridThickness(1.0f);

		VectorViewerStyle = FSampledSequenceVectorViewerStyle()
		.SetBackgroundColor(FLinearColor(0.00273f, 0.00478f, 0.00368f, 1.0f))
		.SetLineColor(FLinearColor(0.0075f, 0.57758f, 0.05448f, 1.0f))
		.SetLineThickness(2.5f);
	}

	virtual ~FAudioVectorscopePanelStyle() = default;

	virtual const FName GetTypeName() const override { return TypeName; }
	static const FAudioVectorscopePanelStyle& GetDefault() { static FAudioVectorscopePanelStyle Default; return Default; }

	/** The value grid style. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSampledSequenceValueGridOverlayStyle ValueGridStyle;
	FAudioVectorscopePanelStyle& SetValueGridStyle(const FSampledSequenceValueGridOverlayStyle& InValueGridStyle) { ValueGridStyle = InValueGridStyle; return *this; }

	/** The vector view style. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSampledSequenceVectorViewerStyle VectorViewerStyle;
	FAudioVectorscopePanelStyle& SetVectorViewerStyle(const FSampledSequenceVectorViewerStyle& InVectorViewerStyle) { VectorViewerStyle = InVectorViewerStyle; return *this; }

	inline static FName TypeName = FName("FAudioVectorscopePanelStyle");

	UE_API static FOnNewValueGridOverlayStyle OnNewValueGridOverlayStyle;
	UE_API static FOnNewVectorViewerStyle     OnNewVectorViewerStyle;
};

#undef UE_API
