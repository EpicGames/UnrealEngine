// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioVectorscopePanelStyle.h"
#include "AudioWidgetsEnums.h"
#include "Components/Widget.h"
#include "FixedSampledSequenceView.h"
#include "Sound/AudioBus.h"

#include "AudioVectorscopeUMG.generated.h"

#define UE_API AUDIOWIDGETS_API

namespace AudioWidgets { class FWaveformAudioSamplesDataProvider; }
class SAudioVectorscopePanelWidget;

/**
 * A vectorscope UMG widget.
 *
 * Supports displaying waveforms in 2D (Left channel X axis, Right channel Y axis) from incoming audio samples.
 * 
 */
UCLASS(MinimalAPI)
class UAudioVectorscope: public UWidget
{
	GENERATED_UCLASS_BODY()

public:
	DECLARE_DYNAMIC_DELEGATE_RetVal(TArray<float>, FGetVectorscopeAudioSamples);

	// UWidget overrides
	UE_API virtual void SynchronizeProperties() override;
	UE_API virtual void ReleaseSlateResources(bool bReleaseChildren) override;

#if WITH_EDITOR
	UE_API virtual const FText GetPaletteCategory() override;
#endif

	/** Starts the vectorscope processing. */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Vectorscope")
	UE_API void StartProcessing();

	/** Stops the vectorscope processing. */
	UFUNCTION(BlueprintCallable, Category = "Audio Widgets| Vectorscope")
	UE_API void StopProcessing();

	/** The vectorscope panel style */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category="Style", meta=(DisplayName="Style"))
	FAudioVectorscopePanelStyle VectorscopeStyle;

	/** The audio bus used to obtain audio samples for the vectorscope */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Vectorscope Values", meta = (DesignerRebuild = "True"))
	TObjectPtr<UAudioBus> AudioBus = nullptr;

	/** Show/Hide the vectorscope grid. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Vectorscope Values")
	bool bShowGrid = true;

	/** The number of grid divisions. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Vectorscope Values", meta = (UIMin = 1, UIMax = 6, ClampMin = 1, ClampMax = 6))
	int32 GridDivisions = 2;

	/** The max where the audio samples should persist in the screen (in milliseconds). */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Vectorscope Values", meta = (UIMin = 10.0, UIMax = 500.0, ClampMin = 10.0, ClampMax = 500.0))
	float MaxDisplayPersistenceMs = 500.0f;

	/** For how long the audio samples should persist in the screen (in milliseconds). */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Vectorscope Values", meta = (UIMin = 10.0, ClampMin = 10.0))
	float DisplayPersistenceMs = 60.0f;

	/** The scale for the displayed audio samples. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Vectorscope Values", meta = (UIMin = 0.0, UIMax = 1.0, ClampMin = 0.0, ClampMax = 1.0))
	float Scale = 1.0f;

	/** Show/Hide advanced panel layout. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Vectorscope Values", meta = (DesignerRebuild = "True"))
	EAudioPanelLayoutType PanelLayoutType = EAudioPanelLayoutType::Basic;

private:
	void CreateDummyVectorscopeWidget();
	void CreateDataProvider();
	void CreateVectorscopeWidget();

	// UWidget overrides
	UE_API virtual TSharedRef<SWidget> RebuildWidget() override;

	static constexpr float AnalysisPeriodMs = 10.0f;

	// The underlying audio samples data provider
	TSharedPtr<AudioWidgets::FWaveformAudioSamplesDataProvider> AudioSamplesDataProvider;

	// Native Slate Widget
	TSharedPtr<SAudioVectorscopePanelWidget> VectorscopePanelWidget;

	// Dummy waveform data to display if audio bus is not set
	TArray<float> DummyAudioSamples;
	FFixedSampledSequenceView DummyDataView;
};

#undef UE_API
