// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

namespace AudioWidgets
{
	struct FAudioAnalyzerRackUnitTypeInfo;
}

class FAudioWidgetsModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	// Register a custom audio analyzer rack unit type
	AUDIOWIDGETS_API void RegisterAudioAnalyzerRackUnitType(const AudioWidgets::FAudioAnalyzerRackUnitTypeInfo* RackUnitTypeInfo);
};
