// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsEnums.generated.h"

UENUM(BlueprintType)
enum class EAudioPanelLayoutType : uint8
{
	Basic    UMETA(DisplayName = "Basic"),
	Advanced UMETA(DisplayName = "Advanced")
};

UENUM()
enum class EAudioUnitsValueType : uint8
{
	Linear,
	Frequency UMETA(DisplayName = "Frequency (Log)"),
	Volume
};
