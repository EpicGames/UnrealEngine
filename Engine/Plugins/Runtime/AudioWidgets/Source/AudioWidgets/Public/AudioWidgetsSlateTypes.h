// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Styling/SlateTypes.h"

#include "AudioWidgetsSlateTypes.generated.h"

#define UE_API AUDIOWIDGETS_API

/**
 * Represents the appearance of an Audio Text Box 
 */
USTRUCT(BlueprintType)
struct FAudioTextBoxStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	UE_API FAudioTextBoxStyle();

	virtual ~FAudioTextBoxStyle() = default;

	UE_API virtual void GetResources(TArray< const FSlateBrush* >& OutBrushes) const override;

	static UE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static UE_API const FAudioTextBoxStyle& GetDefault();

	/** Image for the label border */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush BackgroundImage;
	FAudioTextBoxStyle& SetBackgroundImage(const FSlateBrush& InBackgroundImage) { BackgroundImage = InBackgroundImage; return *this; }

	/** Color used to draw the label background */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor BackgroundColor;
	FAudioTextBoxStyle& SetBackgroundColor(const FSlateColor& InBackgroundColor) { BackgroundColor = InBackgroundColor; return *this; }

	/**
	* Unlinks all colors in this style.
	* @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		BackgroundImage.UnlinkColors();
	}
};

/**
 * Represents the appearance of an Audio Slider
 */
USTRUCT(BlueprintType)
struct FAudioSliderStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	UE_API FAudioSliderStyle();

	virtual ~FAudioSliderStyle() = default;

	UE_API virtual void GetResources(TArray< const FSlateBrush* >& OutBrushes) const override;

	static UE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static UE_API const FAudioSliderStyle& GetDefault();

	/** The style to use for the underlying SSlider. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSliderStyle SliderStyle;
	FAudioSliderStyle& SetSliderStyle(const FSliderStyle& InSliderStyle) { SliderStyle = InSliderStyle; return *this; }

	/** The style to use for the audio text box widget. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FAudioTextBoxStyle TextBoxStyle;
	FAudioSliderStyle& SetTextBoxStyle(const FAudioTextBoxStyle& InTextBoxStyle) { TextBoxStyle = InTextBoxStyle; return *this; }

	/** Image for the widget background */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush WidgetBackgroundImage;
	FAudioSliderStyle& SetWidgetBackgroundImage(const FSlateBrush& InWidgetBackgroundImage) { WidgetBackgroundImage = InWidgetBackgroundImage; return *this; }

	/** Color used to draw the slider background */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor SliderBackgroundColor;
	FAudioSliderStyle& SetSliderBackgroundColor(const FSlateColor& InSliderBackgroundColor) { SliderBackgroundColor = InSliderBackgroundColor; return *this; }

	/** Size of the slider background (slider default is vertical)*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FVector2D SliderBackgroundSize;
	FAudioSliderStyle& SetSliderBackgroundSize(const FVector2D& InSliderBackgroundSize) { SliderBackgroundSize = InSliderBackgroundSize; return *this; }

	/** Size of the padding between the label and the slider */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float LabelPadding;
	FAudioSliderStyle& SetLabelPadding(const float& InLabelPadding) { LabelPadding = InLabelPadding; return *this; }

	/** Color used to draw the slider bar */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor SliderBarColor;
	FAudioSliderStyle& SetSliderBarColor(const FSlateColor& InSliderBarColor) { SliderBarColor = InSliderBarColor; return *this; }

	/** Color used to draw the slider thumb (handle) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor SliderThumbColor;
	FAudioSliderStyle& SetSliderThumbColor(const FSlateColor& InSliderThumbColor) { SliderThumbColor = InSliderThumbColor; return *this; }

	/** Color used to draw the widget background */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor WidgetBackgroundColor;
	FAudioSliderStyle& SetWidgetBackgroundColor(const FSlateColor& InWidgetBackgroundColor) { WidgetBackgroundColor = InWidgetBackgroundColor; return *this; }

	/**
	* Unlinks all colors in this style.
	* @see FSlateColor::Unlink
	 */
	void UnlinkColors()
	{
		WidgetBackgroundImage.UnlinkColors();
	}
};


/**
 * Represents the appearance of an Audio Radial Slider
 */
USTRUCT(BlueprintType)
struct FAudioRadialSliderStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	UE_API FAudioRadialSliderStyle();

	virtual ~FAudioRadialSliderStyle() = default;

	virtual void GetResources(TArray< const FSlateBrush* >& OutBrushes) const override {}

	static UE_API const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };

	static UE_API const FAudioRadialSliderStyle& GetDefault();
	
	/** The style to use for the audio text box widget. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FAudioTextBoxStyle TextBoxStyle;
	FAudioRadialSliderStyle& SetTextBoxStyle(const FAudioTextBoxStyle& InTextBoxStyle) { TextBoxStyle = InTextBoxStyle; return *this; }

	/** Color used to draw the slider center background */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor CenterBackgroundColor;
	FAudioRadialSliderStyle& SetCenterBackgroundColor(const FSlateColor& InCenterBackgroundColor) { CenterBackgroundColor = InCenterBackgroundColor; return *this; }

	/** Color used to draw the unprogressed slider bar */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor SliderBarColor;
	FAudioRadialSliderStyle& SetSliderBarColor(const FSlateColor& InSliderBarColor) { SliderBarColor = InSliderBarColor; return *this; }

	/** Color used to draw the progress bar */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor SliderProgressColor;
	FAudioRadialSliderStyle& SetSliderProgressColor(const FSlateColor& InSliderProgressColor) { SliderProgressColor = InSliderProgressColor; return *this; }
	
	/** Size of the padding between the label and the slider */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float LabelPadding;
	FAudioRadialSliderStyle& SetLabelPadding(const float& InLabelPadding) { LabelPadding = InLabelPadding; return *this; }

	/** Default size of the slider itself (not including label) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float DefaultSliderRadius;
	FAudioRadialSliderStyle& SetDefaultSliderRadius(const float& InDefaultSliderRadius) { DefaultSliderRadius = InDefaultSliderRadius; return *this; }
};

/**
 * Represents the appearance of a Sampled Sequence Viewer
 */
USTRUCT(BlueprintType)
struct FSampledSequenceViewerStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	UE_API FSampledSequenceViewerStyle();

	static UE_API const FSampledSequenceViewerStyle& GetDefault();
	virtual const FName GetTypeName() const override { return TypeName; };
	UE_API virtual void GetResources(TArray< const FSlateBrush* >& OutBrushes) const override;

	static UE_API const FName TypeName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor SequenceColor;
	FSampledSequenceViewerStyle& SetSequenceColor(const FSlateColor InSequenceColor) { SequenceColor = InSequenceColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float SequenceLineThickness;
	FSampledSequenceViewerStyle& SetSequenceLineThickness(const float InSequenceLineThickness) { SequenceLineThickness = InSequenceLineThickness; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor MajorGridLineColor;
	FSampledSequenceViewerStyle& SetMajorGridLineColor(const FSlateColor InMajorGridLineColor) { MajorGridLineColor = InMajorGridLineColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor MinorGridLineColor;
	FSampledSequenceViewerStyle& SetMinorGridLineColor(const FSlateColor InMinorGridLineColor) { MinorGridLineColor = InMinorGridLineColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor ZeroCrossingLineColor;
	FSampledSequenceViewerStyle& SetZeroCrossingLineColor(const FSlateColor InZeroCrossingLineColor) { ZeroCrossingLineColor = InZeroCrossingLineColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float ZeroCrossingLineThickness;
	FSampledSequenceViewerStyle& SetZeroCrossingLineThickness(const float InZeroCrossingLineThickness) { ZeroCrossingLineThickness = InZeroCrossingLineThickness; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float SampleMarkersSize;
	FSampledSequenceViewerStyle& SetSampleMarkersSize(const float InSampleMarkersSize) { SampleMarkersSize = InSampleMarkersSize; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor SequenceBackgroundColor;
	FSampledSequenceViewerStyle& SetBackgroundColor(const FSlateColor InBackgroundColor) { SequenceBackgroundColor = InBackgroundColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush BackgroundBrush;
	FSampledSequenceViewerStyle& SetBackgroundBrush(const FSlateBrush InBackgroundBrush) { BackgroundBrush = InBackgroundBrush; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float DesiredWidth;
	FSampledSequenceViewerStyle& SetDesiredWidth(const float InDesiredWidth) { DesiredWidth = InDesiredWidth; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float DesiredHeight;
	FSampledSequenceViewerStyle& SetDesiredHeight(const float InDesiredHeight) { DesiredHeight = InDesiredHeight; return *this; }
};

/**
 * Represents the appearance of a Waveform Viewer Overlay style
 */
USTRUCT(BlueprintType)
struct FPlayheadOverlayStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	UE_API FPlayheadOverlayStyle();

	static UE_API const FPlayheadOverlayStyle& GetDefault();
	virtual const FName GetTypeName() const override { return TypeName; };

	static UE_API const FName TypeName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor PlayheadColor;
	FPlayheadOverlayStyle& SetPlayheadColor(const FSlateColor InPlayheadColor) { PlayheadColor = InPlayheadColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float PlayheadWidth;
	FPlayheadOverlayStyle& SetPlayheadWidth(const float InPlayheadWidth) { PlayheadWidth = InPlayheadWidth; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float DesiredWidth;
	FPlayheadOverlayStyle& SetDesiredWidth(const float InDesiredWidth) { DesiredWidth = InDesiredWidth; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float DesiredHeight;
	FPlayheadOverlayStyle& SetDesiredHeight(const float InDesiredHeight) { DesiredHeight = InDesiredHeight; return *this; }
};

/**
 * Represents the appearance of a Sampled Sequence Time Ruler
 */
USTRUCT(BlueprintType)
struct FFixedSampleSequenceRulerStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	UE_API FFixedSampleSequenceRulerStyle();

	static UE_API const FFixedSampleSequenceRulerStyle& GetDefault();
	virtual const FName GetTypeName() const override { return TypeName; };
	UE_API virtual void GetResources(TArray< const FSlateBrush* >& OutBrushes) const override;

	static UE_API const FName TypeName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float HandleWidth;
	FFixedSampleSequenceRulerStyle& SetHandleWidth(const float InHandleWidth) { HandleWidth = InHandleWidth; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor HandleColor;
	FFixedSampleSequenceRulerStyle& SetHandleColor(const FSlateColor& InHandleColor) { HandleColor = InHandleColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush HandleBrush;
	FFixedSampleSequenceRulerStyle& SetHandleBrush(const FSlateBrush& InHandleBrush) { HandleBrush = InHandleBrush; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor TicksColor;
	FFixedSampleSequenceRulerStyle& SetTicksColor(const FSlateColor& InTicksColor) { TicksColor = InTicksColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor TicksTextColor;
	FFixedSampleSequenceRulerStyle& SetTicksTextColor(const FSlateColor& InTicksTextColor) { TicksTextColor = InTicksTextColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateFontInfo TicksTextFont;
	FFixedSampleSequenceRulerStyle& SetTicksTextFont(const FSlateFontInfo& InTicksTextFont) { TicksTextFont = InTicksTextFont; return *this; }
	FFixedSampleSequenceRulerStyle& SetFontSize(const float InFontSize) { TicksTextFont.Size = InFontSize; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float TicksTextOffset;
	FFixedSampleSequenceRulerStyle& SetTicksTextOffset(const float InTicksTextOffset) { TicksTextOffset = InTicksTextOffset; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor BackgroundColor;
	FFixedSampleSequenceRulerStyle& SetBackgroundColor(const FSlateColor& InBackgroundColor) { BackgroundColor = InBackgroundColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush BackgroundBrush;
	FFixedSampleSequenceRulerStyle& SetBackgroundBrush(const FSlateBrush& InBackgroundBrush) { BackgroundBrush = InBackgroundBrush; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float DesiredWidth;
	FFixedSampleSequenceRulerStyle& SetDesiredWidth(const float InDesiredWidth) { DesiredWidth = InDesiredWidth; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float DesiredHeight;
	FFixedSampleSequenceRulerStyle& SetDesiredHeight(const float InDesiredHeight) { DesiredHeight = InDesiredHeight; return *this; }
};

/**
 * Represents the appearance of a Sampled Sequence Value Grid Overlay
 */
USTRUCT(BlueprintType)
struct FSampledSequenceValueGridOverlayStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	UE_API FSampledSequenceValueGridOverlayStyle();

	static UE_API const FSampledSequenceValueGridOverlayStyle& GetDefault();
	virtual const FName GetTypeName() const override { return TypeName; };

	static UE_API const FName TypeName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor GridColor;
	FSampledSequenceValueGridOverlayStyle& SetGridColor(const FSlateColor InGridColor) { GridColor = InGridColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float GridThickness;
	FSampledSequenceValueGridOverlayStyle& SetGridThickness(const float InGridThickness) { GridThickness = InGridThickness; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor LabelTextColor;
	FSampledSequenceValueGridOverlayStyle& SetLabelTextColor(const FSlateColor& InLabelTextColor) { LabelTextColor = InLabelTextColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateFontInfo LabelTextFont;
	FSampledSequenceValueGridOverlayStyle& SetLabelTextFont(const FSlateFontInfo& InLabelTextFont) { LabelTextFont = InLabelTextFont; return *this; }
	FSampledSequenceValueGridOverlayStyle& SetLabelTextFontSize(const float InFontSize) { LabelTextFont.Size = InFontSize; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float DesiredWidth;
	FSampledSequenceValueGridOverlayStyle& SetDesiredWidth(const float InDesiredWidth) { DesiredWidth = InDesiredWidth; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float DesiredHeight;
	FSampledSequenceValueGridOverlayStyle& SetDesiredHeight(const float InDesiredHeight) { DesiredHeight = InDesiredHeight; return *this; }

};

#undef UE_API
