// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

#define UE_API AUDIOWIDGETS_API

/**
 * Slate style set that defines all the styles for audio widgets
 */
class FAudioWidgetsStyle
	: public FSlateStyleSet

{
public:
	static UE_API FName StyleName;

	/** Access the singleton instance for this style set */
	static UE_API FAudioWidgetsStyle& Get();

private:
	void SetResources();
	void SetupStyles();

	FAudioWidgetsStyle();
	~FAudioWidgetsStyle();
};

#undef UE_API
