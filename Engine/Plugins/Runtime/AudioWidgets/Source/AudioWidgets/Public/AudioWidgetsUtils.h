// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SWidget.h"

namespace AudioWidgetsUtils
{
	typedef FReply(SWidget::* WidgetMouseInputFunction)(const FGeometry&, const FPointerEvent&);

	/**
	*	Propagates mouse input through an array of widget until that input is handled 
	*	or all the elements have been iterated
	* 
	*/
	FReply AUDIOWIDGETS_API RouteMouseInput(WidgetMouseInputFunction InputFunction, const FPointerEvent& MouseEvent, TArrayView<TSharedPtr<SWidget>> WidgetsStack, const bool bReverse = false);

	/**
	*	Propagates a cursor query through an array of widget until that input is handled
	*	or all the elements have been iterated
	*/
	FCursorReply AUDIOWIDGETS_API RouteCursorQuery(const FPointerEvent& CursorEvent, TArrayView<const TSharedPtr<SWidget>> WidgetsStack, const bool bReverse = false);
}