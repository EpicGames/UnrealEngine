// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "IFixedSampledSequenceGridService.h"
#include "Fonts/SlateFontInfo.h"
#include "Math/Range.h"
#include "Misc/FrameRate.h"

#define UE_API AUDIOWIDGETS_API

DECLARE_MULTICAST_DELEGATE(FOnWaveformEditorGridUpdated)

class FFixedSampledSequenceGridData : public IFixedSampledSequenceGridService
{
public:
	UE_API explicit FFixedSampledSequenceGridData(const uint32 InTotalFrames, const uint32 InSampleRateHz, const FSlateFontInfo& InTicksTimeFont, const float InGridSizePixels = 1.f);

	UE_DEPRECATED(5.4, "Constructor taking a const FSlateFontInfo* is deprecated, use the one taking a refrence instead")
	UE_API explicit FFixedSampledSequenceGridData(const uint32 InTotalFrames, const uint32 InSampleRateHz, const float InGridSizePixels = 1.f, const FSlateFontInfo* InTicksTimeFont = nullptr);


	UE_API void UpdateDisplayRange(const TRange<uint32> InDisplayRange);
	UE_API bool UpdateGridMetrics(const float InGridSizePixels);
	UE_API virtual const FFixedSampledSequenceGridMetrics GetGridMetrics() const override;
	UE_API void SetTicksTimeFont(const FSlateFontInfo& InNewFont);

	UE_DEPRECATED(5.4, "SetTicksTimeFont taking a const FSlateFontInfo* is deprecated, use the one taking a refrence instead")
	UE_API void SetTicksTimeFont(const FSlateFontInfo* InNewFont);

	
	UE_API const float SnapPositionToClosestFrame(const float InPixelPosition) const;

	FOnWaveformEditorGridUpdated OnGridMetricsUpdated;

private:
	FFixedSampledSequenceGridMetrics GridMetrics;
	uint32 TotalFrames = 0;
	TRange<uint32> DisplayRange;

	float GridSizePixels = 0.f;
	FSlateFontInfo TicksTimeFont;
	FFrameRate GridFrameRate;
};

#undef UE_API
