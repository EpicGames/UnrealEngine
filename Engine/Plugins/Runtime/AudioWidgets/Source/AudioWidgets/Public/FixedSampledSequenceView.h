// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"

struct FFixedSampledSequenceView
{
	TArrayView<const float> SampleData;
	uint32 NumDimensions = 0;
	uint32 SampleRate = 0;
};