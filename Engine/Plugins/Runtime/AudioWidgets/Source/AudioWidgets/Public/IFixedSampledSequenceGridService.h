// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

struct FFixedSampledSequenceGridMetrics
{
	int32 NumMinorGridDivisions = 0;
	uint32 SampleRate = 0;
	uint32 StartFrame = 0;
	double PixelsPerFrame = 0;
	double FirstMajorTickX = 0;
	double MajorGridXStep = 0;
};

class IFixedSampledSequenceGridService
{
public:
	virtual ~IFixedSampledSequenceGridService() = default;
	virtual const FFixedSampledSequenceGridMetrics GetGridMetrics() const = 0;
	virtual const float SnapPositionToClosestFrame(const float InPixelPosition) const = 0;
};