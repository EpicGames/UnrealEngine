// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FixedSampledSequenceView.h"

class IFixedSampledSequenceViewReceiver
{
public:
	virtual ~IFixedSampledSequenceViewReceiver() = default;

	
	// Receives and handles a sequence view. 
	// 
	// @param InView Received view for a Fixed Sampled Sequence
	// @param FirstSampleIndex the index of the first sample shown by the view	
	virtual void ReceiveSequenceView(const FFixedSampledSequenceView InView, const uint32 FirstSampleIndex = 0) = 0;
};