// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Range.h"

class ISparseSampledSequenceTransportCoordinator
{
public:
	virtual ~ISparseSampledSequenceTransportCoordinator() = default;

	virtual const TRange<double> GetDisplayRange() const = 0;
	virtual const double GetFocusPoint() const = 0;
	virtual void ScrubFocusPoint(const float InTargetFocusPoint, const bool bIsMoving) = 0;
	virtual const bool IsScrubbing() const = 0;

	virtual void SetProgressRatio(const float NewRatio) = 0;
	virtual void SetZoomRatio(const float NewRatio) = 0;

	virtual float ConvertAbsoluteRatioToZoomed(const float InAbsoluteRatio) const = 0;
	virtual float ConvertZoomedRatioToAbsolute(const float InZoomedRatio) const = 0;

};
