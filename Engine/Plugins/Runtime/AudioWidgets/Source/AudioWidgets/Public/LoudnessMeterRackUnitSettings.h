// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "LoudnessMeterSettings.h"
#include "Styling/StyleColors.h"

#include "LoudnessMeterRackUnitSettings.generated.h"

USTRUCT()
struct FLoudnessMetricDisplayOptions : public FLoudnessMeterDisplayOptions
{
	GENERATED_BODY()
};

USTRUCT()
struct FLoudnessMeterRackUnitSettings : public FLoudnessMeterSettings
{
	GENERATED_BODY()
};
