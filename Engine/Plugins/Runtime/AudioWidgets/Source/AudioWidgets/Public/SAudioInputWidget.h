// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

/**
  * Abstract class for use by audio sliders and knobs 
  * that consists of a visual representation of a float value
  * and a text label.  
 *//*todo make this an audio value display widget that inherits from swidget 
	include textbox and widget ref */
 class SAudioInputWidget
	: public SCompoundWidget
{
public:
	virtual const float GetOutputValue(const float InSliderValue) = 0;
	virtual const float GetSliderValue(const float OutputValue) = 0;
	
	/**
	 * Set the slider's linear (0-1 normalized) value. 
	 */
	virtual void SetSliderValue(float InSliderValue) = 0;
	virtual void SetOutputRange(const FVector2D Range) = 0;
	
	virtual void SetLabelBackgroundColor(FSlateColor InColor) = 0;
	virtual void SetUnitsText(const FText Units) = 0;
	virtual void SetUnitsTextReadOnly(const bool bIsReadOnly) = 0;
	virtual void SetShowUnitsText(const bool bShowUnitsText) = 0;
	virtual void SetDesiredSizeOverride(const FVector2D DesiredSize) = 0;
};

/**
*Processor to allow output and display of different Audio Units. This is because what is shown for the user and what the output is might not be the same. 
*For example, when using the linear output option for volume, Volume is displayed as dB, but the output value will still be 0.0-1.0f.
*/
 struct FAudioUnitProcessor
 {
public:
	
	/**Get the units that the processor will output*/
	virtual const FText GetUnitsText() { return FText(); };

	/**Get the OutputValue. This is where to calculate what the processor will actually output*/
	virtual const float GetOutputValue(const FVector2D OutputRange, const float InSliderValue);

	/**Get the OutputValue that will be shown as text. This might be different then OutputValue depending on the type*/
	virtual const float GetOutputValueForText(const FVector2D OutputRange, const float InSliderValue);

	/**Get the Value for the slider. This should be clamped to be inside the given slider range*/
	virtual const float GetSliderValue(const FVector2D OutputRange, const float OutputValue);

	/**Get the Slider value for the text. option to override what would be returned when value is set directly to the text field. The value in the text field might differ from the slider value because the slider is usally just between 0-1*/
	virtual const float GetSliderValueForText(const FVector2D OutputRange, const float OutputValue);
	
	/**Get the default Output range for the processor. This will be used when Slate is created, and default values are set.*/
	virtual const FVector2D GetDefaultOutputRange() { return FVector2D(0.0f , 1.0f);};

	/**Get the OutputRange. Possibility to modify the output range. Currently only used for Volume*/
	virtual const FVector2D GetOutputRange(FVector2D InRange) { return InRange; };

protected:

	static const FVector2D NormalizedLinearSliderRange;

 };

 struct FVolumeProcessor : FAudioUnitProcessor
 {

	 FVolumeProcessor(bool bInUseLinearOutput){ bUseLinearOutput  = bInUseLinearOutput;}

	 virtual const FText GetUnitsText() override;
	 virtual const FVector2D GetDefaultOutputRange() override;

	 virtual const float GetOutputValue(const FVector2D OutputRange, const float InSliderValue) override;
	 virtual const float GetOutputValueForText(const FVector2D OutputRange, const float InSliderValue) override;
	 virtual const float GetSliderValue(const FVector2D OutputRange, const float OutputValue) override;
	 virtual const float GetSliderValueForText(const FVector2D OutputRange, const float OutputValue) override;
	 virtual const FVector2D GetOutputRange(const FVector2D InRange) override;

private:

	/**
	*Convert the given SliderValue to dB within the given range
	*/
	const float GetDbValueFromSliderValue(const FVector2D OutputRange, const float InSliderValue);

	/**Convert decibels to linear 0-1 space*/
	const float GetSliderValueFromDb(const FVector2D OutputRange, const float DbValue);

private:

	// Min/max possible values for output range, derived to avoid Audio::ConvertToLinear/dB functions returning NaN
	static const float MinDbValue;
	static const float MaxDbValue;

	// Use linear (converted from dB, not normalized) output value. Only applies to the output value reported by GetOutputValue(); the text displayed will still be in decibels. 
	bool bUseLinearOutput;
 }; 
 
 struct FFrequencyProcessor : FAudioUnitProcessor
 {
	 virtual const FText GetUnitsText() override;
	 virtual const FVector2D GetDefaultOutputRange() override;

	 virtual const float GetOutputValue(const FVector2D OutputRange, const float InSliderValue) override;
	 virtual const float GetSliderValue(const FVector2D OutputRange, const float OutputValue) override;
 };
