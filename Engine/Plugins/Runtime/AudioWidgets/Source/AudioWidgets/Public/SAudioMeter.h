// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMeterStyle.h"
#include "AudioMeterTypes.h"
#include "AudioWidgetsStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"

#define UE_API AUDIOWIDGETS_API

class FPaintArgs;
class FSlateWindowElementList;

class SAudioMeterBase
	: public SLeafWidget
{
public:

	SLATE_BEGIN_ARGS(SAudioMeterBase){}

	SLATE_END_ARGS()

	/** Is the active timer registered to refresh the meter channel info. */
	bool bIsActiveTimerRegistered = false;

public:

	virtual void SetMeterChannelInfo(const TAttribute<TArray<FMeterChannelInfo>>& InMeterChannelInfo) = 0;
	virtual TArray<FMeterChannelInfo> GetMeterChannelInfo() const = 0;

};

/**
 * A widget that displays audio channel's levels.
 */
class SAudioMeter
	: public SAudioMeterBase
{
public:

	SLATE_BEGIN_ARGS(SAudioMeter)
		: _Orientation(EOrientation::Orient_Horizontal)
		, _BackgroundColor(FLinearColor::Black)
		, _MeterBackgroundColor(FLinearColor::Gray)
		, _MeterValueColor(FLinearColor::Green)
		, _MeterPeakColor(FLinearColor::Blue)
		, _MeterClippingColor(FLinearColor::Transparent)
		, _MeterScaleColor(FLinearColor::White)
		, _MeterScaleLabelColor(FLinearColor::Gray)
		, _Style(&FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioMeterStyle>("AudioMeter.Style"))
	{
	}

	/** Whether the slidable area should be indented to fit the handle. */
	SLATE_ATTRIBUTE(bool, IndentHandle)

		/** The audio meter's orientation. */
		SLATE_ARGUMENT(EOrientation, Orientation)

		/** The color to draw the background in. */
		SLATE_ATTRIBUTE(FSlateColor, BackgroundColor)

		/** The color to draw the meter background in. */
		SLATE_ATTRIBUTE(FSlateColor, MeterBackgroundColor)

		/** The color to draw the meter value in. */
		SLATE_ATTRIBUTE(FSlateColor, MeterValueColor)

		/** The color to draw the meter peak. */
		SLATE_ATTRIBUTE(FSlateColor, MeterPeakColor)

		/** The color to draw the clipping value in. */
		SLATE_ATTRIBUTE(FSlateColor, MeterClippingColor)

		/** The color to draw the scale in. */
		SLATE_ATTRIBUTE(FSlateColor, MeterScaleColor)

		/** The color to draw the scale in. */
		SLATE_ATTRIBUTE(FSlateColor, MeterScaleLabelColor)

		/** The style used to draw the slider. */
		SLATE_STYLE_ARGUMENT(FAudioMeterStyle, Style)

		/** A value representing the audio meter value. */
		SLATE_ATTRIBUTE(TArray<FMeterChannelInfo>, MeterChannelInfo)

		SLATE_END_ARGS()

	/**
	 * Construct the widget.
	 *
	 * @param InDeclaration A declaration from which to construct the widget.
	 */
	UE_API void Construct(const SAudioMeter::FArguments& InDeclaration);

	UE_API void SetMeterChannelInfo(const TAttribute<TArray<FMeterChannelInfo>>& InMeterChannelInfo) override;
	UE_API TArray<FMeterChannelInfo> GetMeterChannelInfo() const override;

	/** Set the Orientation attribute */
	UE_API void SetOrientation(EOrientation InOrientation);

	UE_API void SetBackgroundColor(FSlateColor InBackgroundColor);
	UE_API void SetMeterBackgroundColor(FSlateColor InMeterBackgroundColor);
	UE_API void SetMeterValueColor(FSlateColor InMeterValueColor);
	UE_API void SetMeterPeakColor(FSlateColor InMeterPeakColor);
	UE_API void SetMeterClippingColor(FSlateColor InMeterPeakColor);
	UE_API void SetMeterScaleColor(FSlateColor InMeterScaleColor);
	UE_API void SetMeterScaleLabelColor(FSlateColor InMeterScaleLabelColor);

public:

	// SWidget overrides

	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual FVector2D ComputeDesiredSize(float LayoutScaleMultiplier) const override;
	UE_API virtual bool ComputeVolatility() const override;

protected:
	
	// Returns the scale height based off font size and hash height
	UE_API float GetScaleHeight() const;

	TSharedPtr<class SAudioMeterWidget> AudioMeterWidget;
};

#undef UE_API
