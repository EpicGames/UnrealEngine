// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioOscilloscopeEnums.h"
#include "AudioOscilloscopePanelStyle.h"
#include "AudioWidgetsEnums.h"
#include "AudioWidgetsSlateTypes.h"
#include "AudioWidgetsStyle.h"
#include "IFixedSampledSequenceViewReceiver.h"
#include "SampledSequenceDisplayUnit.h"
#include "SampledSequenceDrawingUtils.h"
#include "SSampledSequenceValueGridOverlay.h"
#include "STriggerThresholdLineWidget.h"
#include "TriggerThresholdLineStyle.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"
#include "SAudioOscilloscopePanelWidget.generated.h"

#define UE_API AUDIOWIDGETS_API

class FFixedSampledSequenceGridData;
class SAudioRadialSlider;
class SBorder;
class SFixedSampledSequenceRuler;
class SFixedSampledSequenceViewer;
class STriggerThresholdLineWidget;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnSelectedChannelChanged,  const int32 /*InChannel*/)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTriggerModeChanged,      const EAudioOscilloscopeTriggerMode /*InTriggerMode*/)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTriggerThresholdChanged, const float /*InTriggerThreshold*/)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnTimeWindowValueChanged,  const float /*InTimeWindowMs*/)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnAnalysisPeriodChanged,   const float /*InAnalysisPeriodMs*/)

UENUM(BlueprintType)
enum class EXAxisLabelsUnit : uint8
{
	Samples UMETA(DisplayName = "Samples"),
	Seconds UMETA(DisplayName = "Seconds")
};

UENUM(BlueprintType)
enum class EYAxisLabelsUnit : uint8
{
	Linear UMETA(DisplayName = "Linear"),
	Db     UMETA(DisplayName = "dB")
};

class SAudioOscilloscopePanelWidget : public SCompoundWidget, public IFixedSampledSequenceViewReceiver
{
public:
	SLATE_BEGIN_ARGS(SAudioOscilloscopePanelWidget)
		: _HideSequenceGrid(false)
		, _HideSequenceRuler(false)
		, _HideValueGrid(false)
		, _HideTriggerThresholdLine(true)
		, _YAxisLabelsUnit(EYAxisLabelsUnit::Linear)
		, _ValueGridMaxDivisionParameter(2)
		, _SequenceRulerDisplayUnit(EXAxisLabelsUnit::Samples)
		, _PanelLayoutType(EAudioPanelLayoutType::Basic)
		, _PanelStyle(&FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioOscilloscopePanelStyle>("AudioOscilloscope.PanelStyle"))
	{}
		
		SLATE_DEFAULT_SLOT(FArguments, InArgs)

		/** Whether the sequenbce grid should be drawn or not */
		SLATE_ATTRIBUTE(bool, HideSequenceGrid)

		/** Whether the sequence ruler should be drawn or not */
		SLATE_ATTRIBUTE(bool, HideSequenceRuler)

		/** Whether the value grid should be drawn or not */
		SLATE_ATTRIBUTE(bool, HideValueGrid)

		/** Whether the trigger threshold line should be drawn or not */
		SLATE_ATTRIBUTE(bool, HideTriggerThresholdLine)

		/** Whether the value grid values are linear or in dB */
		SLATE_ATTRIBUTE(EYAxisLabelsUnit, YAxisLabelsUnit)

		/** Maximum number of divisions in the value grid */
		SLATE_ARGUMENT(uint32, ValueGridMaxDivisionParameter)

		/** Sequence ruler display unit */
		SLATE_ARGUMENT(EXAxisLabelsUnit, SequenceRulerDisplayUnit)

		/** If we want to set the basic or advanced layout */
		SLATE_ARGUMENT(EAudioPanelLayoutType, PanelLayoutType)

		/** Oscilloscope Panel widget style */
		SLATE_STYLE_ARGUMENT(FAudioOscilloscopePanelStyle, PanelStyle)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const FFixedSampledSequenceView& InData, const int32 InNumChannels);

	UE_API void BuildWidget(const FFixedSampledSequenceView& InData, const int32 InNumChannels, const EAudioPanelLayoutType InPanelLayoutType);

	// SWidget methods overrides
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	// IFixedSampledSequenceViewReceiver Interface
	UE_API virtual void ReceiveSequenceView(const FFixedSampledSequenceView InData, const uint32 FirstSampleIndex) override;

	UE_API void SetSequenceRulerDisplayUnit(const EXAxisLabelsUnit InDisplayUnit);
	UE_API void SetXAxisGridVisibility(const bool InbIsVisible);

	UE_API void SetValueGridOverlayDisplayUnit(const EYAxisLabelsUnit InDisplayUnit);
	UE_API void SetYAxisGridVisibility(const bool InbIsVisible);
	UE_API void SetYAxisLabelsVisibility(const bool InbIsVisible);

	UE_API void SetMaxTimeWindowMs(const float InMaxTimeWindowMs);

	UE_API void SetChannelToAnalyze(const int32 InChannelToAnalyze);
	UE_API void SetTriggerMode(const EAudioOscilloscopeTriggerMode InTriggerMode);
	UE_API void SetTriggerThreshold(const float InTriggerThreshold);
	UE_API void SetTimeWindow(const float InTimeWindow);
	UE_API void SetAnalysisPeriod(const float InAnalysisPeriod);

	UE_API void UpdateSequenceRulerStyle(const FFixedSampleSequenceRulerStyle UpdatedRulerStyle);
	UE_API void UpdateValueGridOverlayStyle(const FSampledSequenceValueGridOverlayStyle UpdatedValueGridOverlayStyle);
	UE_API void UpdateSequenceViewerStyle(const FSampledSequenceViewerStyle UpdatedSequenceViewerStyle);
	UE_API void UpdateTriggerThresholdStyle(const FTriggerThresholdLineStyle UpdatedStyle);

	EAudioPanelLayoutType GetPanelLayoutType() { return PanelLayoutType; }

	FOnSelectedChannelChanged  OnSelectedChannelChanged;
	FOnTriggerModeChanged      OnTriggerModeChanged;
	FOnTriggerThresholdChanged OnTriggerThresholdChanged;
	FOnTimeWindowValueChanged  OnTimeWindowValueChanged;
	FOnAnalysisPeriodChanged   OnAnalysisPeriodChanged;

private:
	UE_API void CreateLayout();

	UE_API void CreateGridData(const FFixedSampleSequenceRulerStyle& RulerStyle);

	// Basic panel methods
	UE_API void CreateSequenceRuler(TSharedRef<FFixedSampledSequenceGridData> InGridData, const FFixedSampleSequenceRulerStyle& RulerStyle);
	UE_API void CreateBackground(const FSampledSequenceViewerStyle& ViewerStyle);
	UE_API void CreateValueGridOverlay(const uint32 MaxDivisionParameter, const SampledSequenceValueGridOverlay::EGridDivideMode DivideMode, const EYAxisLabelsUnit InValueGridOverlayDisplayUnit, const FSampledSequenceValueGridOverlayStyle& ValueGridStyle);
	UE_API void CreateSequenceViewer(TSharedRef<FFixedSampledSequenceGridData> InGridData, const FFixedSampledSequenceView& InData, const FSampledSequenceViewerStyle& ViewerStyle);
	UE_API void CreateTriggerThresholdLine(const FTriggerThresholdLineStyle& TriggerThresholdLineStyle);

	// Advanced panel methods
	UE_API void CreateChannelCombobox();
	UE_API void CreateTriggerModeCombobox();
	UE_API void CreateTriggerThresholdKnob();
	UE_API void CreateTimeWindowKnob();
	UE_API void CreateAnalysisPeriodKnob();
	UE_API void CreateOscilloscopeControls();

	const FAudioOscilloscopePanelStyle* PanelStyle = nullptr;

	// Basic panel widgets
	TSharedPtr<FFixedSampledSequenceGridData> SequenceGridData;
	TSharedPtr<SBorder> BackgroundBorder;
	TSharedPtr<SFixedSampledSequenceRuler> SequenceRuler;
	TSharedPtr<SFixedSampledSequenceViewer> SequenceViewer;
	TSharedPtr<SSampledSequenceValueGridOverlay> ValueGridOverlay;
	TSharedPtr<STriggerThresholdLineWidget> TriggerThresholdLineWidget;

	// Advanced panel widgets
	TSharedPtr<SComboBox<TSharedPtr<FString>>> ChannelCombobox;
	TArray<TSharedPtr<FString>> ChannelComboboxOptionsSource;
	TSharedPtr<FString> SelectedChannelPtr;

	TSharedPtr<SComboBox<TSharedPtr<EAudioOscilloscopeTriggerMode>>> TriggerModeCombobox;
	TArray<TSharedPtr<EAudioOscilloscopeTriggerMode>> TriggerModeComboboxOptionsSource;
	TSharedPtr<EAudioOscilloscopeTriggerMode> SelectedTriggerModePtr;

	TSharedPtr<SAudioRadialSlider> TriggerThresholdKnob;
	TSharedPtr<SAudioRadialSlider> TimeWindowKnob;
	TSharedPtr<SAudioRadialSlider> AnalysisPeriodKnob;

	inline static const FVector2D TriggerThresholdKnobOutputRange = { -1.0, 1.0 };
	inline static const FVector2D AnalysisPeriodKnobOutputRange   = { 10.0, 1000.0 };

	FVector2D TimeWindowKnobOutputRange = { 10.0, 5000.0 };

	EAudioPanelLayoutType PanelLayoutType = EAudioPanelLayoutType::Basic;

	uint32 NumChannels = 0;

	bool bIsInputWidgetTransacting = false;

	float TriggerThresholdValue = 0.0f;
	float TimeWindowValue       = 0.0f;
	float AnalysisPeriodValue   = 0.0f;

	float CachedPixelWidth           = 0.0f;
	float OscilloscopeViewProportion = 1.0f;

	EXAxisLabelsUnit SequenceRulerDisplayUnit;
	EYAxisLabelsUnit ValueGridOverlayDisplayUnit;

	uint32 ValueGridMaxDivisionParameter = 2;

	FFixedSampledSequenceView DataView;
	SampledSequenceDrawingUtils::FSampledSequenceDrawingParams DrawingParams;

	bool bHideSequenceGrid  = false;
	bool bHideSequenceRuler = false;
	bool bHideValueGrid     = false;

	bool bHideTriggerThresholdLine = true;
};

#undef UE_API
