// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsSlateTypes.h"
#include "Curves/CurveFloat.h"
#include "Framework/SlateDelegates.h"
#include "SAudioInputWidget.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "SAudioRadialSlider.generated.h"

#define UE_API AUDIOWIDGETS_API

class SAudioTextBox;
class SImage;
class SWidgetSwitcher;
class SRadialSlider;

UENUM()
enum EAudioRadialSliderLayout : int
{
	/** Label above radial slider. */
	Layout_LabelTop UMETA(DisplayName = "Label Top"),

	/** Label in the center of the radial slider. */
	Layout_LabelCenter UMETA(DisplayName = "Label Center"),

	/** Label below radial slider. */
	Layout_LabelBottom UMETA(DisplayName = "Label Bottom"),
};

/**
 * Slate audio radial sliders that wrap SRadialSlider and provides additional audio specific functionality.
 * This is a nativized version of the previous Audio Knob Small/Large widgets. 
 */
class SAudioRadialSlider
	: public SAudioInputWidget
{
public:
	SLATE_BEGIN_ARGS(SAudioRadialSlider)
	{
		_SliderValue = 0.0f;
		_WidgetLayout = EAudioRadialSliderLayout::Layout_LabelBottom;
		_HandStartEndRatio = FVector2D(0.0f, 1.0f);

		const ISlateStyle* AudioWidgetsStyle = FSlateStyleRegistry::FindSlateStyle("AudioWidgetsStyle");
		if (ensure(AudioWidgetsStyle))
		{
			_Style = &AudioWidgetsStyle->GetWidgetStyle<FAudioRadialSliderStyle>("AudioRadialSlider.Style");
			_CenterBackgroundColor = _Style->CenterBackgroundColor;
			_SliderProgressColor = _Style->SliderProgressColor;
			_SliderBarColor = _Style->SliderBarColor;
		}
	}

	/** The style used to draw the audio radial slider. */
	SLATE_STYLE_ARGUMENT(FAudioRadialSliderStyle, Style)

	/** A value representing the normalized linear (0 - 1) audio slider value position. */
	SLATE_ATTRIBUTE(float, SliderValue)

	/** The widget layout. */
	SLATE_ATTRIBUTE(EAudioRadialSliderLayout, WidgetLayout)

	/** The color to draw the progress bar in. */
	SLATE_ATTRIBUTE(FSlateColor, SliderProgressColor)

	/** The color to draw the slider bar in. */
	SLATE_ATTRIBUTE(FSlateColor, SliderBarColor)

	/** The color to draw the center background in. */
	SLATE_ATTRIBUTE(FSlateColor, CenterBackgroundColor)

	/** Start and end of the hand as a ratio to the slider radius (so 0.0 to 1.0 is from the slider center to the handle). */
	SLATE_ATTRIBUTE(FVector2D, HandStartEndRatio)

	/** A curve that defines how the slider should be sampled. Default is linear.*/
	SLATE_ARGUMENT(FRuntimeFloatCurve, SliderCurve)

	/** When specified, use this as the slider's desired size */
	SLATE_ATTRIBUTE(TOptional<FVector2D>, DesiredSizeOverride)

	/** Called when the value is changed by slider or typing */
	SLATE_EVENT(FOnFloatValueChanged, OnValueChanged)

	/** Invoked when the mouse is pressed and a capture begins. */
	SLATE_EVENT(FSimpleDelegate, OnMouseCaptureBegin)

	/** Invoked when the mouse is released and a capture ends. */
	SLATE_EVENT(FSimpleDelegate, OnMouseCaptureEnd)

	SLATE_END_ARGS()

	UE_API SAudioRadialSlider();
	virtual ~SAudioRadialSlider() = default;

	// Holds a delegate that is executed when the slider's value changed.
	FOnFloatValueChanged OnValueChanged;

	// Holds a delegate that is executed when the mouse is pressed and a capture begins.
	FSimpleDelegate OnMouseCaptureBegin;

	// Holds a delegate that is executed when the mouse is let up and a capture ends.
	FSimpleDelegate OnMouseCaptureEnd;
	
	UE_API void SetCenterBackgroundColor(FSlateColor InColor);
	UE_API void SetSliderProgressColor(FSlateColor InColor);
	UE_API void SetSliderBarColor(FSlateColor InColor);
	UE_API void SetHandStartEndRatio(const FVector2D InHandStartEndRatio);
	UE_API void SetSliderThickness(const float Thickness);
	UE_API void SetWidgetLayout(EAudioRadialSliderLayout InLayout);
	UE_API FVector2D ComputeDesiredSize(float) const;
	UE_API void SetDesiredSizeOverride(const FVector2D DesiredSize);

	UE_API virtual void Construct(const SAudioRadialSlider::FArguments& InArgs);
	UE_API void SetSliderValue(float InSliderValue);
	UE_API virtual const float GetOutputValue(const float InSliderValue);
	UE_API virtual const float GetSliderValue(const float OutputValue);
	UE_API virtual const float GetOutputValueForText(const float InSliderValue);
	UE_API virtual const float GetSliderValueForText(const float OutputValue);
	UE_API virtual void SetOutputRange(const FVector2D Range);

	// Text label functions 
	UE_API void SetLabelBackgroundColor(FSlateColor InColor);
	UE_API void SetUnitsText(const FText Units);
	UE_API void SetUnitsTextReadOnly(const bool bIsReadOnly);
	UE_API void SetValueTextReadOnly(const bool bIsReadOnly);
	UE_API void SetShowLabelOnlyOnHover(const bool bShowLabelOnlyOnHover);
	UE_API void SetShowUnitsText(const bool bShowUnitsText);

protected:
	const FAudioRadialSliderStyle* Style;
	TAttribute<float> SliderValue;
	FRuntimeFloatCurve SliderCurve;
	
	TAttribute<FSlateColor> CenterBackgroundColor;
	TAttribute<FSlateColor> SliderProgressColor;
	TAttribute<FSlateColor> SliderBarColor;
	TAttribute<FSlateColor> LabelBackgroundColor;
	TAttribute<FVector2D> HandStartEndRatio;
	TAttribute<EAudioRadialSliderLayout> WidgetLayout;
	TAttribute<TOptional<FVector2D>> DesiredSizeOverride;

	TSharedPtr<SRadialSlider> RadialSlider;
	TSharedPtr<SImage> CenterBackgroundImage;
	TSharedPtr<SImage> OuterBackgroundImage;
	TSharedPtr<SAudioTextBox> Label;
	// Overall widget layout 
	TSharedPtr<SWidgetSwitcher> LayoutWidgetSwitcher;

	// Range for output 
	FVector2D OutputRange = FVector2D(0.0f, 1.0f);
	static UE_API const FVector2D NormalizedLinearSliderRange;

	UE_API TSharedRef<SWidgetSwitcher> CreateLayoutWidgetSwitcher();
};

/*
* An Audio Radial Slider widget with default conversion for volume (dB).
*/
class SAudioVolumeRadialSlider
	: public SAudioRadialSlider
{
public:
	UE_API SAudioVolumeRadialSlider();
	UE_API void Construct(const SAudioRadialSlider::FArguments& InArgs);
	UE_API const float GetOutputValue(const float InSliderValue) override;
	UE_API const float GetSliderValue(const float OutputValue) override;
	UE_API const float GetOutputValueForText(const float InSliderValue) override;
	UE_API const float GetSliderValueForText(const float OutputValue) override;
	UE_API void SetUseLinearOutput(bool InUseLinearOutput);
	UE_API void SetOutputRange(const FVector2D Range) override;

	// Min/max possible values for output range, derived to avoid Audio::ConvertToLinear/dB functions returning NaN
	static UE_API const float MinDbValue;
	static UE_API const float MaxDbValue;
private:
	// Use linear (converted from dB, not normalized) output value. Only applies to the output value reported by GetOutputValue(); the text displayed will still be in decibels. 
	bool bUseLinearOutput = true;

	const float GetDbValueFromSliderValue(const float InSliderValue);
	const float GetSliderValueFromDb(const float DbValue);
};

/*
* An Audio Radial Slider widget with default logarithmic conversion intended to be used for frequency (Hz).
*/
class SAudioFrequencyRadialSlider
	: public SAudioRadialSlider
{
public:
	UE_API SAudioFrequencyRadialSlider();
	UE_API void Construct(const SAudioRadialSlider::FArguments& InArgs);
	UE_API const float GetOutputValue(const float InSliderValue);
	UE_API const float GetSliderValue(const float OutputValue);
};

#undef UE_API
