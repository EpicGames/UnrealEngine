// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsSlateTypes.h"
#include "Framework/SlateDelegates.h"
#include "SAudioInputWidget.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateWidgetStyleAsset.h"

#define UE_API AUDIOWIDGETS_API

class SAudioTextBox;
class SImage;
class SSlider;
class SWidgetSwitcher;
class UCurveFloat;

/**
 * Slate audio sliders that wrap SSlider and provides additional audio specific functionality.
 * This is a nativized version of the previous Audio Fader widget. 
 */
class SAudioSliderBase
	: public SAudioInputWidget
{
public:
	SLATE_BEGIN_ARGS(SAudioSliderBase)
	{
		_SliderValue = 0.0f;
		_Orientation = Orient_Vertical;

		const ISlateStyle* AudioWidgetsStyle = FSlateStyleRegistry::FindSlateStyle("AudioWidgetsStyle");
		if (ensure(AudioWidgetsStyle))
		{
			_Style = &AudioWidgetsStyle->GetWidgetStyle<FAudioSliderStyle>("AudioSlider.Style");
			_SliderBackgroundColor = _Style->SliderBackgroundColor;
			_SliderBarColor = _Style->SliderBarColor;
			_SliderThumbColor = _Style->SliderThumbColor;
			_WidgetBackgroundColor = _Style->WidgetBackgroundColor;
		}
	}
		/** The style used to draw the audio slider. */
		SLATE_STYLE_ARGUMENT(FAudioSliderStyle, Style)

		/** A value representing the normalized linear (0 - 1) audio slider value position. */
		SLATE_ATTRIBUTE(float, SliderValue)

		/** Whether the text label is always shown or only on hover. */
		SLATE_ATTRIBUTE(bool, AlwaysShowLabel)
			
		/** The orientation of the slider. */
		SLATE_ARGUMENT(EOrientation, Orientation)

		/** The color to draw the slider background in. */
		SLATE_ATTRIBUTE(FSlateColor, SliderBackgroundColor)

		/** The color to draw the slider bar in. */
		SLATE_ATTRIBUTE(FSlateColor, SliderBarColor)

		/** The color to draw the slider thumb in. */
		SLATE_ATTRIBUTE(FSlateColor, SliderThumbColor)
		
		/** The color to draw the widget background in. */
		SLATE_ATTRIBUTE(FSlateColor, WidgetBackgroundColor)
		
		/** When specified, use this as the slider's desired size */
		SLATE_ATTRIBUTE(TOptional<FVector2D>, DesiredSizeOverride)

		/** Called when the value is changed by slider or typing */
		SLATE_EVENT(FOnFloatValueChanged, OnValueChanged)

		/** Called when the value is committed by mouse capture ending */
		SLATE_EVENT(FOnFloatValueChanged, OnValueCommitted)

		/** Invoked when the mouse is pressed and a capture begins. */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureBegin)

		/** Invoked when mouse capture ends (mouse release or any other capture loss). */
		SLATE_EVENT(FSimpleDelegate, OnMouseCaptureEnd)

	SLATE_END_ARGS()

	UE_API SAudioSliderBase();
	virtual ~SAudioSliderBase() = default;

	// Holds a delegate that is executed when the slider's value changed.
	FOnFloatValueChanged OnValueChanged;

	// Holds a delegate that is executed when the slider's value is committed (mouse capture ends).
	FOnFloatValueChanged OnValueCommitted;

	// Holds a delegate that is executed when the mouse is pressed and a capture begins.
	FSimpleDelegate OnMouseCaptureBegin;

	// Holds a delegate that is executed when mouse capture ends.
	FSimpleDelegate OnMouseCaptureEnd;

	/**
	 * Construct the widget.
	 *
	 * @param InDeclaration A declaration from which to construct the widget.
	 */
	UE_API virtual void Construct(const SAudioSliderBase::FArguments& InDeclaration);
	UE_API virtual const float GetOutputValue(const float InSliderValue);
	UE_API virtual const float GetSliderValue(const float OutputValue);
	UE_API virtual const float GetOutputValueForText(const float InSliderValue);
	UE_API virtual const float GetSliderValueForText(const float OutputValue);
	
	/**
	 * Set the slider's linear (0-1 normalized) value. 
	 */
	UE_API void SetSliderValue(float InSliderValue);
	UE_API void SetSliderValue(TAttribute<float> InSliderValue);
	UE_API FVector2D ComputeDesiredSize(float) const;
	UE_API void SetDesiredSizeOverride(const FVector2D DesiredSize);

	UE_API void SetOrientation(EOrientation InOrientation);
	UE_API void SetSliderBackgroundColor(FSlateColor InSliderBackgroundColor);
	UE_API void SetSliderBarColor(FSlateColor InSliderBarColor);
	UE_API void SetSliderThumbColor(FSlateColor InSliderThumbColor);
	UE_API void SetWidgetBackgroundColor(FSlateColor InWidgetBackgroundColor);
	UE_API virtual void SetOutputRange(const FVector2D Range);

	// Text label functions 
	UE_API void SetLabelBackgroundColor(FSlateColor InColor);
	UE_API void SetUnitsText(const FText Units);
	UE_API void SetUnitsTextReadOnly(const bool bIsReadOnly);
	UE_API void SetValueTextReadOnly(const bool bIsReadOnly);
	UE_API void SetShowLabelOnlyOnHover(const bool bShowLabelOnlyOnHover);
	UE_API void SetShowUnitsText(const bool bShowUnitsText);

protected:
	const FAudioSliderStyle* Style;

	// Holds the slider's current linear value, from 0.0 - 1.0f
	TAttribute<float> SliderValueAttribute;
	// Holds the slider's orientation
	TAttribute<EOrientation> Orientation;
	// Optional override for desired size 
	TAttribute<TOptional<FVector2D>> DesiredSizeOverride;

	// Various colors 
	TAttribute<FSlateColor> LabelBackgroundColor;
	TAttribute<FSlateColor> SliderBackgroundColor;
	TAttribute<FSlateColor> SliderBarColor;
	TAttribute<FSlateColor> SliderThumbColor;
	TAttribute<FSlateColor> WidgetBackgroundColor;

	// Widget components
	TSharedPtr<SSlider> Slider;
	TSharedPtr<SAudioTextBox> Label;
	TSharedPtr<SImage> SliderBackgroundImage;
	TSharedPtr<SImage> WidgetBackgroundImage;

	// Range for output, currently only used for frequency sliders and sliders without curves
	FVector2D OutputRange = FVector2D(0.0f, 1.0f);
	static UE_API const FVector2D NormalizedLinearSliderRange;
private:
	FSlateBrush SliderBackgroundBrush;
	FVector2D SliderBackgroundSize;
	/** Switches between the vertical and horizontal views */
	TSharedPtr<SWidgetSwitcher> LayoutWidgetSwitcher;

	UE_API TSharedRef<SWidgetSwitcher> CreateWidgetLayout();
};

/* 
*	An Audio Slider widget with customizable curves. 
*/
class SAudioSlider
	: public SAudioSliderBase
{
public:
	UE_API SAudioSlider();
	virtual ~SAudioSlider() = default;
	UE_API virtual void Construct(const SAudioSliderBase::FArguments& InDeclaration);
	UE_API void SetLinToOutputCurve(const TWeakObjectPtr<const UCurveFloat> LinToOutputCurve);
	UE_API void SetOutputToLinCurve(const TWeakObjectPtr<const UCurveFloat> OutputToLinCurve);
	UE_API const TWeakObjectPtr<const UCurveFloat> GetOutputToLinCurve();
	UE_API const TWeakObjectPtr<const UCurveFloat> GetLinToOutputCurve();
	UE_API const float GetOutputValue(const float InSliderValue);
	UE_API const float GetSliderValue(const float OutputValue);

protected:
	// Curves for mapping linear (0.0 - 1.0) to output (ex. dB for volume)  
	TWeakObjectPtr<const UCurveFloat> LinToOutputCurve = nullptr;
	TWeakObjectPtr<const UCurveFloat> OutputToLinCurve = nullptr;
};

/*
* An Audio Slider widget intended to be used for volume output, with output decibel range but no customizable curves.
*/
class SAudioVolumeSlider
	: public SAudioSliderBase
{
public:
	UE_API SAudioVolumeSlider();
	UE_API void Construct(const SAudioSlider::FArguments& InDeclaration);

	UE_API const float GetOutputValue(const float InSliderValue);
	UE_API const float GetSliderValue(const float OutputValue);
	UE_API const float GetOutputValueForText(const float InSliderValue) override;
	UE_API const float GetSliderValueForText(const float OutputValue) override;
	UE_API void SetUseLinearOutput(bool InUseLinearOutput);
	UE_API void SetOutputRange(const FVector2D Range) override;

private:
	// Min/max possible values for output range, derived to avoid Audio::ConvertToLinear/dB functions returning NaN
	static const float MinDbValue;
	static const float MaxDbValue; 
	
	// Use linear (converted from dB, not normalized) output value. Only applies to the output value reported by GetOutputValue(); the text displayed will still be in decibels. 
	bool bUseLinearOutput = true;

	const float GetDbValueFromSliderValue(const float InSliderValue);
	const float GetSliderValueFromDb(const float DbValue);
};

/*
* An Audio Slider widget intended to be used for frequency output, with output frequency range but no customizable curves. 
*/
class SAudioFrequencySlider
	: public SAudioSliderBase
{
public:
	UE_API SAudioFrequencySlider();
	UE_API void Construct(const SAudioSlider::FArguments& InDeclaration);
	UE_API const float GetOutputValue(const float InSliderValue);
	UE_API const float GetSliderValue(const float OutputValue);
};

#undef UE_API
