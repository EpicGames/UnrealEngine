// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioColorMapper.h"
#include "AudioSpectrogramViewport.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API AUDIOWIDGETS_API

struct FSynesthesiaSpectrumResults;
struct FConstantQResults;

DECLARE_DELEGATE_OneParam(FOnSpectrogramFrequencyAxisPixelBucketModeMenuEntryClicked, EAudioSpectrogramFrequencyAxisPixelBucketMode);
DECLARE_DELEGATE_OneParam(FOnSpectrogramFrequencyAxisScaleMenuEntryClicked, EAudioSpectrogramFrequencyAxisScale);
DECLARE_DELEGATE_OneParam(FOnSpectrogramColorMapMenuEntryClicked, EAudioColorGradient);
DECLARE_DELEGATE_OneParam(FOnSpectrogramOrientationMenuEntryClicked, EOrientation);

/**
 * Slate Widget for rendering a time-frequency representation of a series of audio power spectra.
 */
class SAudioSpectrogram : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAudioSpectrogram)
		: _ViewMinFrequency(20.0f)
		, _ViewMaxFrequency(20000.0f)
		, _ColorMapMinSoundLevel(-84.0f)
		, _ColorMapMaxSoundLevel(12.0f)
		, _ColorMap(EAudioColorGradient::BlackToWhite)
		, _FrequencyAxisScale(EAudioSpectrogramFrequencyAxisScale::Logarithmic)
		, _FrequencyAxisPixelBucketMode(EAudioSpectrogramFrequencyAxisPixelBucketMode::Average)
		, _Orientation(EOrientation::Orient_Horizontal)
		, _AllowContextMenu(true)
		, _FillBackground(false)
	{}
		SLATE_ATTRIBUTE(float, ViewMinFrequency)
		SLATE_ATTRIBUTE(float, ViewMaxFrequency)
		SLATE_ATTRIBUTE(float, ColorMapMinSoundLevel)
		SLATE_ATTRIBUTE(float, ColorMapMaxSoundLevel)
		SLATE_ATTRIBUTE(EAudioColorGradient, ColorMap)
		SLATE_ATTRIBUTE(EAudioSpectrogramFrequencyAxisScale, FrequencyAxisScale)
		SLATE_ATTRIBUTE(EAudioSpectrogramFrequencyAxisPixelBucketMode, FrequencyAxisPixelBucketMode)
		SLATE_ATTRIBUTE(EOrientation, Orientation)
		SLATE_ATTRIBUTE(bool, AllowContextMenu)
		SLATE_ATTRIBUTE(bool, FillBackground)
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
		SLATE_EVENT(FOnSpectrogramFrequencyAxisPixelBucketModeMenuEntryClicked, OnFrequencyAxisPixelBucketModeMenuEntryClicked)
		SLATE_EVENT(FOnSpectrogramFrequencyAxisScaleMenuEntryClicked, OnFrequencyAxisScaleMenuEntryClicked)
		SLATE_EVENT(FOnSpectrogramColorMapMenuEntryClicked, OnColorMapMenuEntryClicked)
		SLATE_EVENT(FOnSpectrogramOrientationMenuEntryClicked, OnOrientationMenuEntryClicked)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs);

	/** Add the data for one spectrum frame to the spectrogram display */
	UE_API void AddFrame(const FAudioSpectrogramFrameData& SpectrogramFrameData);

	/** Add the data for one spectrum frame to the spectrogram display (convenience helper for when using USynesthesiaSpectrumAnalyzer) */
	UE_API void AddFrame(const FSynesthesiaSpectrumResults& SpectrumResults, const EAudioSpectrumType SpectrumType, const float SampleRate);

	/** Add the data for one spectrum frame to the spectrogram display (convenience helper for when using UConstantQAnalyzer) */
	UE_API void AddFrame(const FConstantQResults& ConstantQResults, const float StartingFrequencyHz, const float NumBandsPerOctave, const EAudioSpectrumType SpectrumType);

	void SetViewMinFrequency(const float InViewMinFrequency) { ViewMinFrequency = InViewMinFrequency; }
	void SetViewMaxFrequency(const float InViewMaxFrequency) { ViewMaxFrequency = InViewMaxFrequency; }
	void SetColorMapMinSoundLevel(const float InColorMapMinSoundLevel) { ColorMapMinSoundLevel = InColorMapMinSoundLevel; }
	void SetColorMapMaxSoundLevel(const float InColorMapMaxSoundLevel) { ColorMapMaxSoundLevel = InColorMapMaxSoundLevel; }
	void SetColorMap(const EAudioColorGradient InColorMap) { ColorMap = InColorMap; }
	void SetFrequencyAxisScale(const EAudioSpectrogramFrequencyAxisScale InFrequencyAxisScale) { FrequencyAxisScale = InFrequencyAxisScale; }
	void SetFrequencyAxisPixelBucketMode(const EAudioSpectrogramFrequencyAxisPixelBucketMode InFrequencyAxisPixelBucketMode) { FrequencyAxisPixelBucketMode = InFrequencyAxisPixelBucketMode; }
	void SetOrientation(const EOrientation InOrientation) { Orientation = InOrientation; }
	void SetAllowContextMenu(bool bInAllowContextMenu) { bAllowContextMenu = bInAllowContextMenu; }

	UE_API TSharedRef<const FExtensionBase> AddContextMenuExtension(EExtensionHook::Position HookPosition, const TSharedPtr<FUICommandList>& CommandList, const FMenuExtensionDelegate& MenuExtensionDelegate);
	UE_API void RemoveContextMenuExtension(const TSharedRef<const FExtensionBase>& Extension);

	// Begin SWidget overrides.
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	// End SWidget overrides.

private:
	// Begin SWidget overrides.
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	// End SWidget overrides.

	UE_API TSharedRef<SWidget> BuildDefaultContextMenu();
	UE_API void BuildColorMapSubMenu(FMenuBuilder& SubMenu);
	UE_API void BuildFrequencyAxisScaleSubMenu(FMenuBuilder& SubMenu);
	UE_API void BuildFrequencyAxisPixelBucketModeSubMenu(FMenuBuilder& SubMenu);
	UE_API void BuildOrientationSubMenu(FMenuBuilder& SubMenu);

	static UE_API FName ContextMenuExtensionHook;
	TSharedPtr<FExtender> ContextMenuExtender;

	TAttribute<float> ViewMinFrequency;
	TAttribute<float> ViewMaxFrequency;
	TAttribute<float> ColorMapMinSoundLevel;
	TAttribute<float> ColorMapMaxSoundLevel;
	TAttribute<EAudioColorGradient> ColorMap;
	TAttribute<EAudioSpectrogramFrequencyAxisScale> FrequencyAxisScale;
	TAttribute<EAudioSpectrogramFrequencyAxisPixelBucketMode> FrequencyAxisPixelBucketMode;
	TAttribute<EOrientation> Orientation;
	TAttribute<bool> bAllowContextMenu;
	TAttribute<bool> bFillBackground;
	FOnContextMenuOpening OnContextMenuOpening;
	FOnSpectrogramFrequencyAxisPixelBucketModeMenuEntryClicked OnFrequencyAxisPixelBucketModeMenuEntryClicked;
	FOnSpectrogramFrequencyAxisScaleMenuEntryClicked OnFrequencyAxisScaleMenuEntryClicked;
	FOnSpectrogramColorMapMenuEntryClicked OnColorMapMenuEntryClicked;
	FOnSpectrogramOrientationMenuEntryClicked OnOrientationMenuEntryClicked;

	TSharedPtr<FAudioSpectrogramViewport> SpectrogramViewport;
};

#undef UE_API
