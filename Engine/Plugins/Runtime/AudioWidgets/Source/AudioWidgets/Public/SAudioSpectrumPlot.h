// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioSpectrumPlotStyle.h"
#include "AudioWidgetsStyle.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Framework/SlateDelegates.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SCompoundWidget.h"
#include "SAudioSpectrumPlot.generated.h"

#define UE_API AUDIOWIDGETS_API

UENUM(BlueprintType)
enum class EAudioSpectrumPlotTilt : uint8
{
	NoTilt UMETA(ToolTip = "0 dB/octave slope (white noise is flat)."),
	Plus1_5dBPerOctave UMETA(DisplayName = "1.5 dB/octave", ToolTip = "1.5 dB/octave slope."),
	Plus3dBPerOctave UMETA(DisplayName = "3 dB/octave", ToolTip = "3 dB/octave slope (pink noise is flat)."),
	Plus4_5dBPerOctave UMETA(DisplayName = "4.5 dB/octave", ToolTip = "4.5 dB/octave slope."),
	Plus6dBPerOctave UMETA(DisplayName = "6 dB/octave", ToolTip = "6 dB/octave slope (Brownian noise is flat)."),
};

UENUM(BlueprintType)
enum class EAudioSpectrumPlotFrequencyAxisScale : uint8
{
	Linear,
	Logarithmic,
};

UENUM(BlueprintType)
enum class EAudioSpectrumPlotFrequencyAxisPixelBucketMode : uint8
{
	Sample UMETA(ToolTip = "Plot one data point per frequency axis pixel bucket only, choosing the data point nearest the pixel center."),
	Peak UMETA(ToolTip = "Plot one data point per frequency axis pixel bucket only, choosing the data point with the highest sound level."),
	Average UMETA(ToolTip = "Plot the average of the data points in each frequency axis pixel bucket."),
};

/**
 * Utility class for converting between spectrum data and local/absolute screen space.
 */
class FAudioSpectrumPlotScaleInfo
{
public:
	FAudioSpectrumPlotScaleInfo(const FVector2f InWidgetSize, EAudioSpectrumPlotFrequencyAxisScale InFrequencyAxisScale, float InViewMinFrequency, float InViewMaxFrequency, float InViewMinSoundLevel, float InViewMaxSoundLevel)
		: WidgetSize(InWidgetSize)
		, FrequencyAxisScale(InFrequencyAxisScale)
		, TransformedViewMinFrequency(ForwardTransformFrequency(InViewMinFrequency))
		, TransformedViewMaxFrequency(ForwardTransformFrequency(InViewMaxFrequency))
		, TransformedViewFrequencyRange(TransformedViewMaxFrequency - TransformedViewMinFrequency)
		, PixelsPerTransformedHz((TransformedViewFrequencyRange > 0.0f) ? (InWidgetSize.X / TransformedViewFrequencyRange) : 0.0f)
		, ViewMinSoundLevel(InViewMinSoundLevel)
		, ViewMaxSoundLevel(InViewMaxSoundLevel)
		, ViewSoundLevelRange(InViewMaxSoundLevel - InViewMinSoundLevel)
		, PixelsPerDecibel((ViewSoundLevelRange > 0.0f) ? (InWidgetSize.Y / ViewSoundLevelRange) : 0.0f)
	{
		//
	}

	float LocalXToFrequency(float ScreenX) const
	{
		const float TransformedFrequency = (PixelsPerTransformedHz != 0.0f) ? ((ScreenX / PixelsPerTransformedHz) + TransformedViewMinFrequency) : 0.0f;
		return InverseTransformFrequency(TransformedFrequency);
	}

	float FrequencyToLocalX(float Frequency) const
	{
		return (ForwardTransformFrequency(Frequency) - TransformedViewMinFrequency) * PixelsPerTransformedHz;
	}

	float LocalYToSoundLevel(float ScreenY) const
	{
		return (PixelsPerDecibel != 0.0f) ? (ViewMaxSoundLevel - (ScreenY / PixelsPerDecibel)) : 0.0f;
	}

	float SoundLevelToLocalY(float SoundLevel) const
	{
		return (ViewMaxSoundLevel - SoundLevel) * PixelsPerDecibel;
	}

	FVector2f ToLocalPos(const FVector2f& FrequencyAndSoundLevel) const
	{
		return { FrequencyToLocalX(FrequencyAndSoundLevel.X), SoundLevelToLocalY(FrequencyAndSoundLevel.Y) };
	}

private:
	float ForwardTransformFrequency(float Frequency) const
	{
		return (FrequencyAxisScale == EAudioSpectrumPlotFrequencyAxisScale::Logarithmic) ? FMath::Loge(Frequency) : Frequency;
	}

	float InverseTransformFrequency(float TransformedFrequency) const
	{
		return (FrequencyAxisScale == EAudioSpectrumPlotFrequencyAxisScale::Logarithmic) ? FMath::Exp(TransformedFrequency) : TransformedFrequency;
	}

	const FVector2f WidgetSize;

	const EAudioSpectrumPlotFrequencyAxisScale FrequencyAxisScale;
	const float TransformedViewMinFrequency;
	const float TransformedViewMaxFrequency;
	const float TransformedViewFrequencyRange;
	const float PixelsPerTransformedHz;

	const float ViewMinSoundLevel;
	const float ViewMaxSoundLevel;
	const float ViewSoundLevelRange;
	const float PixelsPerDecibel;
};

DECLARE_DELEGATE_OneParam(FOnTiltSpectrumMenuEntryClicked, EAudioSpectrumPlotTilt);
DECLARE_DELEGATE_OneParam(FOnFrequencyAxisPixelBucketModeMenuEntryClicked, EAudioSpectrumPlotFrequencyAxisPixelBucketMode);
DECLARE_DELEGATE_OneParam(FOnFrequencyAxisScaleMenuEntryClicked, EAudioSpectrumPlotFrequencyAxisScale);
DECLARE_DELEGATE(FOnDisplayAxisLabelsButtonToggled);

/**
 * The audio spectrum data to plot.
 */
struct FAudioPowerSpectrumData
{
	TConstArrayView<float> CenterFrequencies;
	TConstArrayView<float> SquaredMagnitudes;
};

DECLARE_DELEGATE_RetVal(FAudioPowerSpectrumData, FGetAudioSpectrumData);

/**
 * Slate Widget for plotting an audio power spectrum, with linear or log frequency scale, and decibels sound levels.
 */
class SAudioSpectrumPlot : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAudioSpectrumPlot)
		: _Style(&FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioSpectrumPlotStyle>("AudioSpectrumPlot.Style"))
		, _ViewMinFrequency(20.0f)
		, _ViewMaxFrequency(20000.0f)
		, _ViewMinSoundLevel(-60.0f)
		, _ViewMaxSoundLevel(12.0f)
		, _TiltExponent(0.0f)
		, _TiltPivotFrequency(24000.0f)
		, _DisplayCrosshair(false)
		, _DisplayFrequencyAxisLabels(true)
		, _DisplaySoundLevelAxisLabels(true)
		, _DisplayFrequencyGridLines(true)
		, _DisplaySoundLevelGridLines(true)
		, _FrequencyAxisScale(EAudioSpectrumPlotFrequencyAxisScale::Logarithmic)
		, _FrequencyAxisPixelBucketMode(EAudioSpectrumPlotFrequencyAxisPixelBucketMode::Average)
		, _BackgroundColor(FSlateColor::UseStyle())
		, _GridColor(FSlateColor::UseStyle())
		, _AxisLabelColor(FSlateColor::UseStyle())
		, _CrosshairColor(FSlateColor::UseStyle())
		, _SpectrumColor(FSlateColor::UseStyle())
		, _AllowContextMenu(true)
	{}
		SLATE_STYLE_ARGUMENT(FAudioSpectrumPlotStyle, Style)
		SLATE_ATTRIBUTE(float, ViewMinFrequency)
		SLATE_ATTRIBUTE(float, ViewMaxFrequency)
		SLATE_ATTRIBUTE(float, ViewMinSoundLevel)
		SLATE_ATTRIBUTE(float, ViewMaxSoundLevel)
		SLATE_ATTRIBUTE(float, TiltExponent)
		SLATE_ATTRIBUTE(float, TiltPivotFrequency)
		SLATE_ATTRIBUTE(TOptional<float>, SelectedFrequency)
		SLATE_ATTRIBUTE(bool, DisplayCrosshair)
		SLATE_ATTRIBUTE(bool, DisplayFrequencyAxisLabels)
		SLATE_ATTRIBUTE(bool, DisplaySoundLevelAxisLabels)
		SLATE_ATTRIBUTE(bool, DisplayFrequencyGridLines)
		SLATE_ATTRIBUTE(bool, DisplaySoundLevelGridLines)
		SLATE_ATTRIBUTE(EAudioSpectrumPlotFrequencyAxisScale, FrequencyAxisScale)
		SLATE_ATTRIBUTE(EAudioSpectrumPlotFrequencyAxisPixelBucketMode, FrequencyAxisPixelBucketMode)
		SLATE_ATTRIBUTE(FSlateColor, BackgroundColor)
		SLATE_ATTRIBUTE(FSlateColor, GridColor)
		SLATE_ATTRIBUTE(FSlateColor, AxisLabelColor)
		SLATE_ATTRIBUTE(FSlateColor, CrosshairColor)
		SLATE_ATTRIBUTE(FSlateColor, SpectrumColor)
		SLATE_ATTRIBUTE(bool, AllowContextMenu)
		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
		SLATE_EVENT(FOnTiltSpectrumMenuEntryClicked, OnTiltSpectrumMenuEntryClicked)
		SLATE_EVENT(FOnFrequencyAxisPixelBucketModeMenuEntryClicked, OnFrequencyAxisPixelBucketModeMenuEntryClicked)
		SLATE_EVENT(FOnFrequencyAxisScaleMenuEntryClicked, OnFrequencyAxisScaleMenuEntryClicked)
		SLATE_EVENT(FOnDisplayAxisLabelsButtonToggled, OnDisplayFrequencyAxisLabelsButtonToggled)
		SLATE_EVENT(FOnDisplayAxisLabelsButtonToggled, OnDisplaySoundLevelAxisLabelsButtonToggled)
		SLATE_EVENT(FGetAudioSpectrumData, OnGetAudioSpectrumData)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	UE_API void Construct(const FArguments& InArgs);

	void SetViewMinFrequency(float InViewMinFrequency) { ViewMinFrequency = InViewMinFrequency; }
	void SetViewMaxFrequency(float InViewMaxFrequency) { ViewMaxFrequency = InViewMaxFrequency; }
	void SetViewMinSoundLevel(float InViewMinSoundLevel) { ViewMinSoundLevel = InViewMinSoundLevel; }
	void SetViewMaxSoundLevel(float InViewMaxSoundLevel) { ViewMaxSoundLevel = InViewMaxSoundLevel; }
	void SetTiltExponent(float InTiltExponent) { TiltExponent = InTiltExponent; }
	void SetTiltPivotFrequency(float InTiltPivotFrequency) { TiltPivotFrequency = InTiltPivotFrequency; }
	void SetSelectedFrequency(TOptional<float> InSelectedFrequency) { SelectedFrequency = InSelectedFrequency; }
	void SetDisplayCrosshair(bool bInDisplayCrosshair) { bDisplayCrosshair = bInDisplayCrosshair; }
	void SetDisplayFrequencyAxisLabels(bool bInDisplayFrequencyAxisLabels) { bDisplayFrequencyAxisLabels = bInDisplayFrequencyAxisLabels; }
	void SetDisplaySoundLevelAxisLabels(bool bInDisplaySoundLevelAxisLabels) { bDisplaySoundLevelAxisLabels = bInDisplaySoundLevelAxisLabels; }
	void SetDisplayFrequencyGridLines(bool bInDisplayFrequencyGridLines) { bDisplayFrequencyGridLines = bInDisplayFrequencyGridLines; }
	void SetDisplaySoundLevelGridLines(bool bInDisplaySoundLevelGridLines) { bDisplaySoundLevelGridLines = bInDisplaySoundLevelGridLines; }
	void SetFrequencyAxisScale(EAudioSpectrumPlotFrequencyAxisScale InFrequencyAxisScale) { FrequencyAxisScale = InFrequencyAxisScale; }
	void SetFrequencyAxisPixelBucketMode(EAudioSpectrumPlotFrequencyAxisPixelBucketMode InFrequencyAxisPixelBucketMode) { FrequencyAxisPixelBucketMode = InFrequencyAxisPixelBucketMode; }
	void SetAllowContextMenu(bool bInAllowContextMenu) { bAllowContextMenu = bInAllowContextMenu; }

	UE_API TSharedRef<const FExtensionBase> AddContextMenuExtension(EExtensionHook::Position HookPosition, const TSharedPtr<FUICommandList>& CommandList, const FMenuExtensionDelegate& MenuExtensionDelegate);
	UE_API void RemoveContextMenuExtension(const TSharedRef<const FExtensionBase>& Extension);

	// Begin SWidget overrides.
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	UE_API virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	// End SWidget overrides.

	void UnbindOnGetAudioSpectrumData() { OnGetAudioSpectrumData.Unbind(); }

	UE_API FAudioSpectrumPlotScaleInfo GetScaleInfo() const;

	static UE_API float GetTiltExponentValue(const EAudioSpectrumPlotTilt InTilt);

private:
	// Begin SWidget overrides.
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	// End SWidget overrides.

	UE_API int32 DrawSolidBackgroundRectangle(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle) const;

	UE_API int32 DrawGrid(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, const FAudioSpectrumPlotScaleInfo& ScaleInfo) const;
	UE_API void GetGridLineSoundLevels(TArray<float>& GridLineSoundLevels) const;
	UE_API void GetGridLineFrequencies(TArray<float>& AllGridLineFrequencies, TArray<float>& MajorGridLineFrequencies) const;

	UE_API int32 DrawPowerSpectrum(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, const FAudioSpectrumPlotScaleInfo& ScaleInfo) const;
	UE_API FAudioPowerSpectrumData GetPowerSpectrum() const;

	UE_API int32 DrawCrosshairAndAxisLabels(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, const FAudioSpectrumPlotScaleInfo& ScaleInfo, TConstArrayView<FVector2f> LinePoints) const;

	// This is a function to reduce the given array of data points to a possibly shorter array of points that will form the line to be plotted.
	// Where multiple data points map to the same frequency axis pixel bucket, the given 'cost function' will be used to select the best data point (the data point with the lowest 'cost').
	static UE_API void GetSpectrumLinePoints(TArray<FVector2f>& OutLinePoints, TConstArrayView<FVector2f> DataPoints, const FAudioSpectrumPlotScaleInfo& ScaleInfo, TFunctionRef<float(const FVector2f& DataPoint)> CostFunction);

	UE_API FLinearColor GetBackgroundColor(const FWidgetStyle& InWidgetStyle) const;
	UE_API FLinearColor GetGridColor(const FWidgetStyle& InWidgetStyle) const;
	UE_API FLinearColor GetAxisLabelColor(const FWidgetStyle& InWidgetStyle) const;
	UE_API FLinearColor GetCrosshairColor(const FWidgetStyle& InWidgetStyle) const;
	UE_API FLinearColor GetSpectrumColor(const FWidgetStyle& InWidgetStyle) const;	

	UE_API TSharedRef<SWidget> BuildDefaultContextMenu();
	UE_API void BuildTiltSpectrumSubMenu(FMenuBuilder& SubMenu);
	UE_API void BuildFrequencyAxisScaleSubMenu(FMenuBuilder& SubMenu);
	UE_API void BuildFrequencyAxisPixelBucketModeSubMenu(FMenuBuilder& SubMenu);

	static UE_API const float ClampMinSoundLevel;

	static UE_API FName ContextMenuExtensionHook;
	TSharedPtr<FExtender> ContextMenuExtender;

	const FAudioSpectrumPlotStyle* Style;
	TAttribute<float> ViewMinFrequency;
	TAttribute<float> ViewMaxFrequency;
	TAttribute<float> ViewMinSoundLevel;
	TAttribute<float> ViewMaxSoundLevel;
	TAttribute<float> TiltExponent;
	TAttribute<float> TiltPivotFrequency;
	TAttribute<TOptional<float>> SelectedFrequency;
	TAttribute<bool> bDisplayCrosshair;
	TAttribute<bool> bDisplayFrequencyAxisLabels;
	TAttribute<bool> bDisplaySoundLevelAxisLabels;
	TAttribute<bool> bDisplayFrequencyGridLines;
	TAttribute<bool> bDisplaySoundLevelGridLines;
	TAttribute<EAudioSpectrumPlotFrequencyAxisScale> FrequencyAxisScale;
	TAttribute<EAudioSpectrumPlotFrequencyAxisPixelBucketMode> FrequencyAxisPixelBucketMode;
	TAttribute<FSlateColor> BackgroundColor;
	TAttribute<FSlateColor> GridColor;
	TAttribute<FSlateColor> AxisLabelColor;
	TAttribute<FSlateColor> CrosshairColor;
	TAttribute<FSlateColor> SpectrumColor;
	TAttribute<bool> bAllowContextMenu;
	FOnContextMenuOpening OnContextMenuOpening;
	FOnTiltSpectrumMenuEntryClicked OnTiltSpectrumMenuEntryClicked;
	FOnFrequencyAxisPixelBucketModeMenuEntryClicked OnFrequencyAxisPixelBucketModeMenuEntryClicked;
	FOnFrequencyAxisScaleMenuEntryClicked OnFrequencyAxisScaleMenuEntryClicked;
	FOnDisplayAxisLabelsButtonToggled OnDisplayFrequencyAxisLabelsButtonToggled;
	FOnDisplayAxisLabelsButtonToggled OnDisplaySoundLevelAxisLabelsButtonToggled;
	FGetAudioSpectrumData OnGetAudioSpectrumData;
};

#undef UE_API
