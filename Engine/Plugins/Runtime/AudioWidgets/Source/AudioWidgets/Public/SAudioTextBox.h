// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsSlateTypes.h"
#include "Framework/SlateDelegates.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API AUDIOWIDGETS_API

class SBorder;
class SEditableText;

// FVariablePrecisionNumericInterface
// Taken from PropertyEditor/VariablePrecisionNumericInterface.h
// Allow more precision as the numbers get closer to zero
struct FVariablePrecisionNumericInterface : public TDefaultNumericTypeInterface<float>
{
	FVariablePrecisionNumericInterface() {}

	virtual FString ToString(const float& Value) const override
	{
		// examples: 1000, 100.1, 10.12, 1.123
		float AbsValue = FMath::Abs(Value);
		int32 FractionalDigits = 3;

		// special case because FastDecimalFormat::NumberToString does not parse decimal points with 0 fractional digits when a value is greater than uint64 max
		if (AbsValue >= static_cast<float>(TNumericLimits<uint64>::Max()))
			FractionalDigits = 1;
		else if ((AbsValue / 1000.f) >= 1.f)
			FractionalDigits = 0;
		else if ((AbsValue / 100.f) >= 1.f)
			FractionalDigits = 1;
		else if ((AbsValue / 10.f) >= 1.f)
			FractionalDigits = 2;

		const FNumberFormattingOptions NumberFormattingOptions = FNumberFormattingOptions()
			.SetUseGrouping(false)
			.SetMinimumFractionalDigits(FractionalDigits)
			.SetMaximumFractionalDigits(FractionalDigits);
		return FastDecimalFormat::NumberToString(Value, ExpressionParser::GetLocalizedNumberFormattingRules(), NumberFormattingOptions);
	}
};

class SAudioTextBox
	: public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SAudioTextBox)
	{
		_ShowLabelOnlyOnHover = false;
		_ShowUnitsText = true;
		const ISlateStyle* AudioWidgetsStyle = FSlateStyleRegistry::FindSlateStyle("AudioWidgetsStyle");
		if (ensure(AudioWidgetsStyle))
		{
			_Style = &AudioWidgetsStyle->GetWidgetStyle<FAudioTextBoxStyle>("AudioTextBox.Style");
		}
	}
	/** The style used to draw the audio text box. */
	SLATE_STYLE_ARGUMENT(FAudioTextBoxStyle, Style)

	/** If true, show label only on hover; if false always show label. */
	SLATE_ATTRIBUTE(bool, ShowLabelOnlyOnHover)

	/** Whether to show the units part of the text label. */
	SLATE_ATTRIBUTE(bool, ShowUnitsText)

	/** The color to draw the label background in. */
	SLATE_ATTRIBUTE(FSlateColor, LabelBackgroundColor)

	/** Delegate to call when the value text is committed. */
	SLATE_EVENT(FOnTextCommitted, OnValueTextCommitted)

	SLATE_END_ARGS()
public:
	UE_API virtual void Construct(const SAudioTextBox::FArguments& InArgs);
	UE_API void SetLabelBackgroundColor(FSlateColor InColor);
	UE_API void SetUnitsText(const FText Units);
	UE_API void SetValueText(const float OutputValue);
	UE_API void SetUnitsTextReadOnly(const bool bIsReadOnly);
	UE_API void SetValueTextReadOnly(const bool bIsReadOnly);
	UE_API void SetShowLabelOnlyOnHover(const bool bShowLabelOnlyOnHover);
	UE_API void SetShowUnitsText(const bool bShowUnitsText);
	UE_API void UpdateValueTextWidth(const FVector2D OutputRange);

protected:
	const FAudioTextBoxStyle* Style;
	TSharedPtr<SEditableText> ValueText;
	TSharedPtr<SEditableText> UnitsText;
	TSharedPtr<SBorder> LabelBorder;
	SHorizontalBox::FSlot* ValueTextSlot = nullptr;

	TAttribute<bool> ShowLabelOnlyOnHover;
	TAttribute<bool> ShowUnitsText;
	TAttribute<FSlateColor> LabelBackgroundColor;
	FOnTextCommitted OnValueTextCommitted;

	FVector2D OutputRange = FVector2D(0.0f, 1.0f);

	/** Used to convert and format value text strings **/
	static UE_API const FVariablePrecisionNumericInterface NumericInterface;
};

#undef UE_API
