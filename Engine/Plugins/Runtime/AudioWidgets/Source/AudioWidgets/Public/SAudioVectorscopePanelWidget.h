// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioVectorscopePanelStyle.h"
#include "AudioWidgetsEnums.h"
#include "AudioWidgetsStyle.h"
#include "Brushes/SlateColorBrush.h"
#include "IFixedSampledSequenceViewReceiver.h"
#include "SampledSequenceDrawingUtils.h"
#include "SSampledSequenceValueGridOverlay.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API AUDIOWIDGETS_API

class SAudioRadialSlider;
class SBorder;
class SFixedSampledSequenceViewer;
class SFixedSampledSequenceVectorViewer;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnTimeWindowValueChanged, const float /*InTimeWindowMs*/)

class SAudioVectorscopePanelWidget : public SCompoundWidget, public IFixedSampledSequenceViewReceiver
{
public:
	SLATE_BEGIN_ARGS(SAudioVectorscopePanelWidget)
		: _HideGrid(false)
		, _ValueGridMaxDivisionParameter(2)
		, _PanelLayoutType(EAudioPanelLayoutType::Basic)
		, _PanelStyle(&FAudioWidgetsStyle::Get().GetWidgetStyle<FAudioVectorscopePanelStyle>("AudioVectorscope.PanelStyle"))
	{}
		
		SLATE_DEFAULT_SLOT(FArguments, InArgs)

		/** Whether the value grid should be drawn or not */
		SLATE_ATTRIBUTE(bool, HideGrid)

		/** Maximum number of divisions in the value grid */
		SLATE_ARGUMENT(uint32, ValueGridMaxDivisionParameter)

		/** If we want to set the basic or advanced layout */
		SLATE_ARGUMENT(EAudioPanelLayoutType, PanelLayoutType)

		/** Vectorscope Panel widget style */
		SLATE_STYLE_ARGUMENT(FAudioVectorscopePanelStyle, PanelStyle)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const FFixedSampledSequenceView& InData);

	UE_API void BuildWidget(const FFixedSampledSequenceView& InData, const EAudioPanelLayoutType InPanelLayoutType);

	// IFixedSampledSequenceViewReceiver Interface
	UE_API virtual void ReceiveSequenceView(const FFixedSampledSequenceView InData, const uint32 FirstSampleIndex) override;

	UE_API void SetGridVisibility(const bool InbIsVisible);
	UE_API void SetValueGridOverlayMaxNumDivisions(const uint32 InGridMaxNumDivisions);

	UE_API void SetMaxDisplayPersistence(const float InMaxDisplayPersistenceInMs);

	UE_API void SetDisplayPersistence(const float InDisplayPersistenceInMs);
	UE_API void SetVectorViewerScaleFactor(const float InScaleFactor);

	UE_API void UpdateValueGridOverlayStyle(const FSampledSequenceValueGridOverlayStyle UpdatedValueGridOverlayStyle);
	UE_API void UpdateSequenceVectorViewerStyle(const FSampledSequenceVectorViewerStyle UpdatedSequenceVectorViewerStyle);

	EAudioPanelLayoutType GetPanelLayoutType() { return PanelLayoutType; }

	FOnTimeWindowValueChanged OnDisplayPersistenceValueChanged;

private:
	UE_API void CreateLayout();

	// Basic panel methods
	UE_API void CreateBackground(const FSampledSequenceVectorViewerStyle& VectorViewerStyle);

	UE_API TSharedPtr<SSampledSequenceValueGridOverlay> CreateValueGridOverlay(const uint32 MaxDivisionParameter,
		const SampledSequenceValueGridOverlay::EGridDivideMode DivideMode,
		const FSampledSequenceValueGridOverlayStyle& ValueGridStyle,
		const SampledSequenceDrawingUtils::ESampledSequenceDrawOrientation GridOrientation);

	UE_API void CreateSequenceVectorViewer(const FFixedSampledSequenceView& InData, const FSampledSequenceVectorViewerStyle& VectorViewerStyle);

	// Advanced panel methods
	UE_API void CreateDisplayPersistenceKnob();
	UE_API void CreateScaleKnob();
	UE_API void CreateVectorscopeControls();

	const FAudioVectorscopePanelStyle* PanelStyle;

	// Basic panel widgets
	TSharedPtr<SBorder> BackgroundBorder;
	TSharedPtr<SFixedSampledSequenceVectorViewer> SequenceVectorViewer;
	TSharedPtr<SSampledSequenceValueGridOverlay> ValueGridOverlayXAxis;
	TSharedPtr<SSampledSequenceValueGridOverlay> ValueGridOverlayYAxis;

	// Advanced panel widgets
	TSharedPtr<SAudioRadialSlider> DisplayPersistenceKnob;
	TSharedPtr<SAudioRadialSlider> ScaleKnob;

	FVector2D DisplayPersistenceKnobOutputRange = { 10.0, 500.0 };
	inline static const FVector2D ScaleFactorOutputKnobRange = { 0.0, 1.0 };

	EAudioPanelLayoutType PanelLayoutType = EAudioPanelLayoutType::Basic;

	uint32 ValueGridMaxDivisionParameter = 2;

	bool bIsInputWidgetTransacting = false;

	float DisplayPersistenceValue = 0.0f;
	float ScaleValue = 0.0f;

	float VectorscopeViewProportion = 1.0f;

	FFixedSampledSequenceView DataView;

	bool bHideValueGrid = false;
};

#undef UE_API
