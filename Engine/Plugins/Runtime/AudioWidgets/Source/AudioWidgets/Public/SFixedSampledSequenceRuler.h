// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IFixedSampledSequenceGridService.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "SampledSequenceDisplayUnit.h"
#include "AudioWidgetsSlateTypes.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API AUDIOWIDGETS_API

DECLARE_MULTICAST_DELEGATE_OneParam(FOnTimeUnitMenuSelection, const ESampledSequenceDisplayUnit /* Requested Display Unit */);

class SFixedSampledSequenceRuler : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SFixedSampledSequenceRuler)
		: _DisplayUnit(ESampledSequenceDisplayUnit::Seconds)
	{
	}

	SLATE_ARGUMENT(ESampledSequenceDisplayUnit, DisplayUnit)

	/** Whether the playhead should be drawn or not */
	SLATE_ATTRIBUTE(bool, DisplayPlayhead)

	SLATE_STYLE_ARGUMENT(FFixedSampleSequenceRulerStyle, Style)

	SLATE_END_ARGS()

public:
	UE_API void Construct(const FArguments& InArgs, TSharedRef<IFixedSampledSequenceGridService> InGridService);
	UE_API void UpdateGridMetrics();
	UE_API void UpdateDisplayUnit(const ESampledSequenceDisplayUnit InDisplayUnit);
	UE_API void SetPlayheadPosition(const float InNewPosition);
	UE_API void OnStyleUpdated(const FFixedSampleSequenceRulerStyle UpdatedStyle);
	UE_API FReply LaunchContextMenu();

	UE_API void SetPlayheadVisible(bool Enable);

	/** Delegate sent when the user selects a new display unit from the RMB menu*/
	FOnTimeUnitMenuSelection OnTimeUnitMenuSelection;

private:
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;

	void DrawPlayheadHandle(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32& LayerId) const;
	void DrawRulerTicks(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32& LayerId) const;
	void DrawTickTimeString(uint32 TickFrame, const double TickX, const double TickY, FSlateWindowElementList& OutDrawElements, int32& LayerId, const FGeometry& AllottedGeometry) const;

	TSharedRef<SWidget> MakeContextMenu();
	void MakeTimeUnitsSubMenu(FMenuBuilder& SubMenuBuilder);
	void NotifyTimeUnitMenuSelection(const ESampledSequenceDisplayUnit SelectedDisplayUnit) const;

	FFixedSampledSequenceGridMetrics GridMetrics;

	FSlateColor BackgroundColor = FLinearColor::Black;
	FSlateBrush BackgroundBrush;
	FSlateBrush HandleBrush;
	FSlateColor HandleColor = FLinearColor(255.f, 0.1f, 0.2f, 1.f);
	FSlateColor TicksColor = FLinearColor(1.f, 1.f, 1.f, 0.9f);
	FSlateColor TicksTextColor = FLinearColor(1.f, 1.f, 1.f, 0.9f);

	float DesiredHeight = 0.f;
	float DesiredWidth = 0.f;
	float HandleWidth = 15.f;
	float TicksTextOffset = 5.f;
	float PlayheadPosition = 0.f;

	FSlateFontInfo TicksTextFont;

	TSharedPtr<IFixedSampledSequenceGridService> GridService = nullptr;

	ESampledSequenceDisplayUnit DisplayUnit;

	bool bDisplayPlayhead = true;
};

#undef UE_API
