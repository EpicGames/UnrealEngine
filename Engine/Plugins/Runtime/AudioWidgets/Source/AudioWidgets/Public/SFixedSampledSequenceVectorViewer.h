// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SampledSequenceDrawingUtils.h"
#include "SampledSequenceVectorViewerStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SLeafWidget.h"

#define UE_API AUDIOWIDGETS_API


class SFixedSampledSequenceVectorViewer : public SLeafWidget
{
public: 
	SLATE_BEGIN_ARGS(SFixedSampledSequenceVectorViewer)
		: _ScaleFactor(1.0f)
	{
	}

	SLATE_ARGUMENT(SampledSequenceDrawingUtils::FSampledSequenceDrawingParams, SequenceDrawingParams)

	SLATE_ARGUMENT(float, ScaleFactor)

	SLATE_STYLE_ARGUMENT(FSampledSequenceVectorViewerStyle, Style)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TArrayView<const float> InSampleData, const uint8 InNumChannels);

	UE_API void UpdateView(TArrayView<const float> InSampleData, const uint8 InNumChannels);
	UE_API void SetScaleFactor(const float InScaleFactor);
	UE_API void OnStyleUpdated(const FSampledSequenceVectorViewerStyle UpdatedStyle);

private:
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, 
		const FGeometry& AllottedGeometry, 
		const FSlateRect& MyCullingRect, 
		FSlateWindowElementList& OutDrawElements, 
		int32 LayerId, 
		const FWidgetStyle& InWidgetStyle, 
		bool bParentEnabled) const override;

	UE_API virtual FVector2D ComputeDesiredSize(float) const override;

	bool bForceRedraw = false;

	TArrayView<const float> SampleData;
	SampledSequenceDrawingUtils::FSampledSequenceDrawingParams DrawingParams;

	uint8 NumChannels = 0;

	float DesiredHeight = 720.0f;
	float DesiredWidth  = 720.0f;

	FLinearColor LineColor = FLinearColor::White;
	float LineThickness    = 1.0f;

	float ScaleFactor = 1.0f;
};

#undef UE_API
