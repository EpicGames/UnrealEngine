// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsSlateTypes.h"
#include "IFixedSampledSequenceGridService.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "SampledSequenceDrawingUtils.h"
#include "Widgets/SLeafWidget.h"

#define UE_API AUDIOWIDGETS_API

namespace SampledSequenceViewerWidget
{
	typedef TRange<float> SampleRange;
}

class SFixedSampledSequenceViewer : public SLeafWidget
{
public:

	SLATE_BEGIN_ARGS(SFixedSampledSequenceViewer) 
		: _HideBackground(false)
		, _HideGrid(false)
	{
	}
	
	SLATE_ARGUMENT(SampledSequenceDrawingUtils::FSampledSequenceDrawingParams, SequenceDrawingParams)

	SLATE_ARGUMENT(bool, HideBackground)

	SLATE_ARGUMENT(bool, HideGrid)

	SLATE_STYLE_ARGUMENT(FSampledSequenceViewerStyle, Style)
		
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TArrayView<const float> InSampleData, const uint8 InNumChannels, TSharedRef<IFixedSampledSequenceGridService> InGridService);
	UE_API void UpdateView(TArrayView<const float> InSampleData, const uint8 InNumChannels);
	
	UE_API void OnStyleUpdated(const FSampledSequenceViewerStyle UpdatedStyle);

	UE_API void UpdateGridMetrics();

	UE_API void SetHideGrid(const bool InHideGrid);

private:
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void DrawGridLines(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32& LayerId) const;

	bool bForceRedraw = false;

	TArrayView<const float> SampleData;
	uint8 NumChannels = 0;
	float DurationInSeconds = 0.f;

	TSharedPtr<IFixedSampledSequenceGridService> GridService = nullptr;
	FFixedSampledSequenceGridMetrics GridMetrics;

	FSlateBrush BackgroundBrush;
	FSlateColor BackgroundColor = FLinearColor(0.02f, 0.02f, 0.02f, 1.f);
	FSlateColor SequenceColor = FLinearColor::White;
	FSlateColor MajorGridLineColor = FLinearColor::Black;
	FSlateColor MinorGridLineColor = FLinearColor(0.f, 0.f, 0.f, 0.5f);
	FSlateColor ZeroCrossingLineColor = FLinearColor::Black;
	float ZeroCrossingLineThickness = 1.f;
	float DesiredHeight = 0.f;
	float DesiredWidth = 0.f;
	float SampleMarkersSize = 2.5f;
	float SequenceLineThickness = 1.f;
	bool bHideBackground = false;
	bool bHideGrid = false;

	uint32 CachedPixelWidth = 0; 
	float CachedPixelHeight = 0.f;
	TArray<SampledSequenceViewerWidget::SampleRange> CachedPeaks;
	TArray<SampledSequenceDrawingUtils::F2DLineCoordinates> CachedBinsDrawCoordinates;
	SampledSequenceDrawingUtils::FSampledSequenceDrawingParams DrawingParams;
	TArray<FVector2D> CachedSampleDrawCoordinates;

	enum class ESequenceDrawMode {
		BinnedPeaks = 0,
		SequenceLine
	} SequenceDrawMode;
};

#undef UE_API
