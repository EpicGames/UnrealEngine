// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsSlateTypes.h"
#include "AudioWidgetsStyle.h"
#include "IFixedSampledSequenceViewReceiver.h"
#include "SampledSequenceDisplayUnit.h"
#include "SampledSequenceDrawingUtils.h"
#include "SSampledSequenceValueGridOverlay.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API AUDIOWIDGETS_API

class FFixedSampledSequenceGridData;
class FSparseSampledSequenceTransportCoordinator;
class FWaveformEditorGridData;
class SBorder;
class SFixedSampledSequenceRuler;
class SFixedSampledSequenceViewer;
class SPlayheadOverlay;

class SFixedSampledSequenceViewerPanel : public SCompoundWidget, public IFixedSampledSequenceViewReceiver
{
public:

	SLATE_BEGIN_ARGS(SFixedSampledSequenceViewerPanel) 
		: _HidePlayhead(false)
		, _HideSamplesGrid(false)
		, _HideSamplesRuler(false)
		, _HideValueGrid(false)
		, _ValueGridDivideMode(SampledSequenceValueGridOverlay::EGridDivideMode::MidSplit)
		, _ValueGridMaxDivisionParameter(2)
		, _SequenceDrawingParams(SampledSequenceDrawingUtils::FSampledSequenceDrawingParams())
		, _SamplesRulerDisplayUnit(ESampledSequenceDisplayUnit::Seconds)
		, _SequenceViewerStyle(&FAudioWidgetsStyle::Get().GetWidgetStyle< FSampledSequenceViewerStyle >("SampledSequenceViewer.Style"))
		, _SamplesRulerStyle(&FAudioWidgetsStyle::Get().GetWidgetStyle< FFixedSampleSequenceRulerStyle >("FixedSampledSequenceRuler.Style"))
		, _PlayheadOverlayStyle(&FAudioWidgetsStyle::Get().GetWidgetStyle< FPlayheadOverlayStyle >("PlayheadOverlay.Style"))
		, _ValueGridStyle(&FAudioWidgetsStyle::Get().GetWidgetStyle< FSampledSequenceValueGridOverlayStyle >("ValueGridOverlay.Style"))
	{}
		
		SLATE_DEFAULT_SLOT(FArguments, InArgs)

		/** Whether the playhead should be drawn or not */
		SLATE_ATTRIBUTE(bool, HidePlayhead)

		/** Whether the samples grid should be drawn or not */
		SLATE_ATTRIBUTE(bool, HideSamplesGrid)

		/** Whether the samples ruler should be drawn or not */
		SLATE_ATTRIBUTE(bool, HideSamplesRuler)

		/** Whether the value grid should be drawn or not */
		SLATE_ATTRIBUTE(bool, HideValueGrid)

		/** Regulates how the value grid is divided */
		SLATE_ARGUMENT(SampledSequenceValueGridOverlay::EGridDivideMode, ValueGridDivideMode)

		/** Maximum number of divisions in the value grid */
		SLATE_ARGUMENT(uint32, ValueGridMaxDivisionParameter)

		/** Allows customization of labels displayed in the value grid */
		SLATE_ARGUMENT(TFunction<FText(const double)>, ValueGridLabelGenerator)

		/** Sequence drawing params */
		SLATE_ARGUMENT(SampledSequenceDrawingUtils::FSampledSequenceDrawingParams, SequenceDrawingParams)

		/** Samples ruler display unit */
		SLATE_ARGUMENT(ESampledSequenceDisplayUnit, SamplesRulerDisplayUnit)

		/** Sequence viewer widget style */
		SLATE_STYLE_ARGUMENT(FSampledSequenceViewerStyle, SequenceViewerStyle)

		/** Time Ruler widget style */
		SLATE_STYLE_ARGUMENT(FFixedSampleSequenceRulerStyle, SamplesRulerStyle)

		/** Playhead overlay widget style */
		SLATE_STYLE_ARGUMENT(FPlayheadOverlayStyle, PlayheadOverlayStyle)

		/** Value grid widget style */
		SLATE_STYLE_ARGUMENT(FSampledSequenceValueGridOverlayStyle, ValueGridStyle)


	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const FFixedSampledSequenceView& InData);
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/*IFixedSampledSequenceViewReceiver Interface*/
	UE_API virtual void ReceiveSequenceView(const FFixedSampledSequenceView InData, const uint32 FirstSampleIndex) override;
	
	UE_API void SetPlayheadRatio(const float InRatio);

	UE_API void UpdateSequenceViewerStyle(const FSampledSequenceViewerStyle UpdatedStyle);
	UE_API void UpdateRulerStyle(const FFixedSampleSequenceRulerStyle UpdatedStyle);
	UE_API void UpdatePlayheadOverlayStyle(const FPlayheadOverlayStyle UpdatedStyle);
	UE_API void UpdateValueGridOverlayStyle(const FSampledSequenceValueGridOverlayStyle UpdatedStyle);
	
private:
	UE_API virtual void CreateLayout();

	TSharedPtr<FFixedSampledSequenceGridData> SamplesGridData;
	TSharedPtr<SBorder> BackgroundBorder;
	TSharedPtr<SFixedSampledSequenceRuler> SamplesRuler;
	TSharedPtr<SFixedSampledSequenceViewer> SequenceViewer;
	TSharedPtr<SPlayheadOverlay> PlayheadOverlay;
	TSharedPtr<SSampledSequenceValueGridOverlay> ValueGridOverlay;


	void CreateGridData(const FFixedSampleSequenceRulerStyle* RulerStyle);
	void CreatePlayheadOverlay(const FPlayheadOverlayStyle* PlayheadOverlayStyle);
	void CreateValueGridOverlay(const uint32 MaxDivisionParameter, const SampledSequenceValueGridOverlay::EGridDivideMode DivideMode, const TFunction<FText(const double)>& LabelGenerator, const FSampledSequenceValueGridOverlayStyle* ValueGridStyle);
	void CreateSamplesRuler(TSharedRef<FFixedSampledSequenceGridData> InGridData, const FFixedSampleSequenceRulerStyle* RulerStyle);
	void CreateSequenceViewer(TSharedRef<FFixedSampledSequenceGridData> InGridData, const FFixedSampledSequenceView& InData, const FSampledSequenceViewerStyle* ViewerStyle);
	void CreateBackground(const FSampledSequenceViewerStyle* ViewerStyle);
	
	void UpdateSamplesRulerDisplayUnit(const ESampledSequenceDisplayUnit InDisplayUnit);
	void UpdatePlayheadPosition(const float PaintedWidth);

	float CachedPixelWidth = 0.f;

	ESampledSequenceDisplayUnit SamplesRulerDisplayUnit;
	FFixedSampledSequenceView DataView;
	SampledSequenceDrawingUtils::FSampledSequenceDrawingParams DrawingParams;

	float CachedPlayheadRatio = 0.f;
	bool bHidePlayhead = false;
	bool bHideSamplesGrid = false;
	bool bHideSamplesRuler = false;
	bool bHideValueGrid = false;
};

#undef UE_API
