// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsSlateTypes.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SLeafWidget.h"

#define UE_API AUDIOWIDGETS_API

class SPlayheadOverlay : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SPlayheadOverlay)
	{
	}

	SLATE_STYLE_ARGUMENT(FPlayheadOverlayStyle, Style)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);
	UE_API void SetPlayheadPosition(const float InNewPosition);
	UE_API void OnStyleUpdated(const FPlayheadOverlayStyle UpdatedStyle);

	UE_API void SetPlayheadVisible(bool Enable);

private:
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;

	int32 DrawPlayhead(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

	const FPlayheadOverlayStyle* Style = nullptr;

	FSlateColor PlayheadColor = FLinearColor(255.f, 0.1f, 0.2f, 1.f);
	float PlayheadWidth = 1.0;
	float DesiredWidth = 0.f;
	float DesiredHeight = 0.f;

	float PlayheadPosition = 0.f;
	bool bIsVisible = true;
};

#undef UE_API
