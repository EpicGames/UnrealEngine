// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsSlateTypes.h"
#include "SampledSequenceDrawingUtils.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SLeafWidget.h"

#define UE_API AUDIOWIDGETS_API

namespace SampledSequenceValueGridOverlay
{
	enum class EGridDivideMode
	{
		EvenSplit = 0,
		MidSplit,
		COUNT
	};
}

class SSampledSequenceValueGridOverlay : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SSampledSequenceValueGridOverlay)
		: _DivideMode(SampledSequenceValueGridOverlay::EGridDivideMode::EvenSplit)
		, _NumDimensions(1)
		, _HideLabels(false)
		, _HideGrid(false)
		, _MaxDivisionParameter(2)
	{
	}

	SLATE_ARGUMENT(SampledSequenceValueGridOverlay::EGridDivideMode, DivideMode)

	SLATE_ARGUMENT(uint32, NumDimensions)

	SLATE_ARGUMENT(bool, HideLabels)

	SLATE_ARGUMENT(bool, HideGrid)

	SLATE_ARGUMENT(uint32, MaxDivisionParameter)

	SLATE_ARGUMENT(SampledSequenceDrawingUtils::FSampledSequenceDrawingParams, SequenceDrawingParams)

	SLATE_ARGUMENT(TFunction<FText(const double)>, ValueGridLabelGenerator)

	SLATE_STYLE_ARGUMENT(FSampledSequenceValueGridOverlayStyle, Style)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API void OnStyleUpdated(const FSampledSequenceValueGridOverlayStyle UpdatedStyle);

	UE_API void SetLabelGenerator(TFunction<FText(const double)> InLabelGenerator);
	UE_API void SetMaxDivisionParameter(const uint32 InDivisionParameter);
	UE_API void SetHideLabels(const bool InHideLabels);
	UE_API void SetHideGrid(const bool InHideGrid);

	UE_API void ForceRedraw();

private:
	void CacheDrawElements(const FGeometry& AllottedGeometry, const uint32 InDivisionParameter);
	
	void GenerateVerticalGridLabels(const uint32 InDivisionParameter, const FGeometry& AllottedGeometry);
	void GenerateHorizontalGridLabels(const uint32 InDivisionParameter, const FGeometry& AllottedGeometry);


	SampledSequenceValueGridOverlay::EGridDivideMode GridSplitMode;
	uint32 MaxDivisionParameter;
	uint32 NumDimensions;
	TFunction<FText(const double)> OnValueGridLabel;
	SampledSequenceDrawingUtils::FSampledSequenceDrawingParams DrawingParams;
	FVector2D CachedLocalSize;
	bool bForceRedraw = false;
	bool bHideLabels = false;
	bool bHideGrid = false;

	TArray<SampledSequenceDrawingUtils::FGridData> CachedGridSlotData;
	FNumberFormattingOptions ValueGridFormatOptions;
	const float LabelToGridPixelDistance = 2;

	const FSampledSequenceValueGridOverlayStyle* Style = nullptr;

	FSlateColor GridColor;
	FSlateColor LabelTextColor;
	FSlateFontInfo LabelTextFont;
	float DesiredWidth;
	float DesiredHeight;
	float GridThickness;


	struct FGridLabelData
	{
		TArray<FText> LabelTexts;
		TArray<FVector2D> LabelCoordinates;
	};

	TArray<FGridLabelData> CachedLabelData;

};

#undef UE_API
