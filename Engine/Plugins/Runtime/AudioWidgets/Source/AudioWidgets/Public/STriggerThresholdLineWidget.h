// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsStyle.h"
#include "Rendering/DrawElements.h"
#include "SampledSequenceDrawingUtils.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "TriggerThresholdLineStyle.h"
#include "Widgets/SLeafWidget.h"

#define UE_API AUDIOWIDGETS_API

struct FTriggerThresholdLineStyle;

class STriggerThresholdLineWidget : public SLeafWidget
{
public:
    SLATE_BEGIN_ARGS(STriggerThresholdLineWidget)
        : _TriggerThreshold(0.0f)
		, _Style(&FAudioWidgetsStyle::Get().GetWidgetStyle<FTriggerThresholdLineStyle>("TriggerThresholdLine.Style"))
    {}
    
		SLATE_ARGUMENT(float, TriggerThreshold)
		SLATE_ARGUMENT(SampledSequenceDrawingUtils::FSampledSequenceDrawingParams, SequenceDrawingParams)
		SLATE_STYLE_ARGUMENT(FTriggerThresholdLineStyle, Style)

    SLATE_END_ARGS()

    UE_API void Construct(const FArguments& InArgs);

    UE_API virtual int32 OnPaint(const FPaintArgs& Args, 
		const FGeometry& AllottedGeometry, 
		const FSlateRect& MyCullingRect, 
		FSlateWindowElementList& OutDrawElements, 
		int32 LayerId, 
		const FWidgetStyle& InWidgetStyle,
		bool bParentEnabled) const override;

	UE_API virtual FVector2D ComputeDesiredSize(float) const override;

	UE_API void OnStyleUpdated(const FTriggerThresholdLineStyle UpdatedStyle);

	UE_API void SetTriggerThreshold(float InTriggerThreshold);

private:
    float TriggerThreshold;
	FLinearColor LineColor;
	SampledSequenceDrawingUtils::FSampledSequenceDrawingParams DrawingParams;
	const FTriggerThresholdLineStyle* Style = nullptr;
};

#undef UE_API
