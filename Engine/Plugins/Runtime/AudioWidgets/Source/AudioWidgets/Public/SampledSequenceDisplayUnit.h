// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

enum class ESampledSequenceDisplayUnit
{
	Samples,
	Seconds
};