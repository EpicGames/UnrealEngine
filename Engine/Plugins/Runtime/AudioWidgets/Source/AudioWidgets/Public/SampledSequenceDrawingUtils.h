// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IFixedSampledSequenceGridService.h"
#include "Math/Vector2D.h"
#include "Math/Range.h"

#define UE_API AUDIOWIDGETS_API

struct FGeometry;

namespace SampledSequenceDrawingUtils
{
	/**
	* Groups samples evenly into a number of desired bins.
	* Each bin contains the min and max values of the grouped samples.
	*
	* @param OutBins			TArray where the bins will be written to
	* @param NumDesiredBins		Number of output bins
	* @param RawDataPtr			Ptr to the beginning of the samples
	* @param TotalNumSamples	The total number of samples of the input time series
	* @param NDimensions		Number of interleaved dimensions in the time series
	* @param StartRatio			The ratio of the total number of frames (in a range of 0-1) at which grouping should start
	* @param EndRatio			The ratio of the total number of frames (in a range of 0-1) at which grouping should end.
	*
	*/
	template<typename SamplesType>
	void GroupInterleavedSamplesIntoMinMaxBins(TArray<TRange<SamplesType>>& OutBins, const uint32 NumDesiredBins, const SamplesType* RawDataPtr, const uint32 TotalNumSamples, const uint16 NDimensions = 1, const double StartRatio = 0.0, double EndRatio = 1.0)
	{
		check(StartRatio >= 0.f && StartRatio < EndRatio);
		check(EndRatio <= 1.f);

		uint32 NumPeaks = NumDesiredBins * NDimensions;
		OutBins.SetNumUninitialized(NumPeaks);
		const uint32 NumFrames = TotalNumSamples / NDimensions;

		double FramesPerBin = ((EndRatio - StartRatio) * NumFrames) / NumDesiredBins;
		double IterationStartFrame = StartRatio * NumFrames;
		uint32 FirstBinnedFrame = FMath::RoundToInt(StartRatio * NumFrames);

		for (uint32 Peak = 0; Peak < NumPeaks; Peak += NDimensions)
		{
			uint32 LastBinnedFrame = IterationStartFrame + FramesPerBin;

			for (uint16 Channel = 0; Channel < NDimensions; ++Channel)
			{
				
				SamplesType MaxSampleValue = TNumericLimits<SamplesType>::Lowest();
				SamplesType MinSampleValue = TNumericLimits<SamplesType>::Max();

				for (uint32 Frame = FirstBinnedFrame; Frame < LastBinnedFrame; ++Frame)
				{
					const uint32 SampleIndex = Frame * NDimensions + Channel;
					check(SampleIndex < TotalNumSamples);
					SamplesType SampleValue = RawDataPtr[SampleIndex];

					MinSampleValue = FMath::Min(MinSampleValue, SampleValue);
					MaxSampleValue = FMath::Max(MaxSampleValue, SampleValue);

				}

				OutBins[Peak + Channel] = TRange<SamplesType>(MinSampleValue, MaxSampleValue);
			}

			IterationStartFrame += FramesPerBin;
			FirstBinnedFrame = LastBinnedFrame;
		}
	}

	/**
	* Groups samples of a time series into an equal number of desired bins.
	* Each bin contains the min and max values of the grouped samples.
	*
	* @param OutBins			TArray where the bins will be written to
	* @param NumDesiredBins		Number of output bins
	* @param RawDataPtr			Ptr to the beginning of the samples
	* @param TotalNumSamples	The total number of samples of the input time series
	* @param SampleRate			SampleRate of the time series
	* @param NDimensions		Number of interleaved dimensions in the time series
	* @param StartTime			The time at which grouping should start
	* @param EndTime 			The time at which grouping should end.
	*
	* Note: With a negative EndTime, the method will calculate automatically the EndTime by doing
	* TotalNumSamples / (SampleRate * NDimensions)
	*
	*/
	template<typename SamplesType>
	void GroupInterleavedSampledTSIntoMinMaxBins(TArray<TRange<SamplesType>>& OutBins, const uint32 NumDesiredBins, const SamplesType* RawDataPtr, const uint32 TotalNumSamples, const uint32 SampleRate, const uint16 NDimensions = 1, const float StartTime = 0.f, float EndTime = -1.f)
	{
		const double TotalTime = TotalNumSamples / ((float)SampleRate * NDimensions);
		const double StartRatio = StartTime / TotalTime;
		const double EndRatio = EndTime >= 0.f ? FMath::Clamp(EndTime / TotalTime, StartRatio, 1.0) : 1.0;

		GroupInterleavedSamplesIntoMinMaxBins(OutBins, NumDesiredBins, RawDataPtr, TotalNumSamples, NDimensions, StartRatio, EndRatio);
	}

	/**
		* Params for drawing sequences of samples/sample bins
		* @param MaxDisplayedValue					The highest value a sample can take
		* @param DimensionSlotMargin				Margin to keep from the dimension slot boundaries (pixels)
		* @param MinSequenceHeightRatio				The minimum height (ratio) the drawn sequence can take in a channel slot
		* @param MaxSequenceHeightRatio				The maximum height (ratio) the drawn sequence can take in a channel slot
		* @param MinScaledBinValue					Minimum value (in pixels) a bin can have
		* @param VerticalZoomFactor					Sequence zoom factor
	*/

	enum class ESampledSequenceDrawOrientation
	{
		Horizontal = 0, 
		Vertical, 
		COUNT
	};


	struct FSampledSequenceDrawingParams
	{
		double MaxDisplayedValue = 1;
		float DimensionSlotMargin = 2.f;
		float MaxSequenceHeightRatio = 0.95f;
		float MinSequenceHeightRatio = 0.1f;
		float MinScaledBinValue = 0.1f;
		float VerticalZoomFactor = 1.f;
		ESampledSequenceDrawOrientation Orientation = ESampledSequenceDrawOrientation::Horizontal;
	};

	/**
	* Represents a dimensional slot in which samples, sample bins or a grid can be drawn
	*/
	struct FDimensionSlot
	{
		UE_API explicit FDimensionSlot(const uint16 DimensionToDraw, const uint16 TotalNumDimensions, const FGeometry& InAllottedGeometry, const FSampledSequenceDrawingParams& Params);

		float Top;
		float Center;
		float Bottom;
		float Height;
	};

	struct F2DLineCoordinates
	{
		FVector2D A;
		FVector2D B;
	};

	/**
	 * DrawCoordinates: Coordinates of different grid lines
	 * PositionRatios: the Ratio of each grid line position in relationship to the main axis length
	 */
	struct FGridData
	{
		TArray<F2DLineCoordinates> DrawCoordinates;
		TArray<float> PositionRatios;
	};

	/**
		* Generates an array of coordinates to draw binned samples as vertical lines.
		* Coordinates will be generated for an horizontal view
		* 
		* @param OutDrawCoordinates			The generated coordinates
		* @param InAllottedGeometry			The geometry to draw into
		* @param InSampleBins				The bins to draw
		* @param NDimensions				Number of interleaved dimensions in the time series
		* @param Params						Drawing Params
	*/
	void GenerateSampleBinsCoordinatesForGeometry(TArray<F2DLineCoordinates>& OutDrawCoordinates, const FGeometry& InAllottedGeometry, const TArray<TRange<float>>& InSampleBins, const uint16 NDimensions, const FSampledSequenceDrawingParams Params = FSampledSequenceDrawingParams());

	/**
	* Generates an array of coordinates to draw single samples in succession horizontally.
	*
	* @param OutDrawCoordinates			The generated coordinates
	* @param InSampleData				The samples to draw
	* @param InAllottedGeometry			The geometry to draw into
	* @param NDimensions				Number of interleaved dimensions in the time series
	* @param InGridMetrics				GridMetrics followed to lay down the samples
	* @param Params						Drawing Params
	*/
	void GenerateSequencedSamplesCoordinatesForGeometry(TArray<FVector2D>& OutDrawCoordinates, TArrayView<const float> InSampleData, const FGeometry& InAllottedGeometry, const uint16 NDimensions, const FFixedSampledSequenceGridMetrics InGridMetrics, const FSampledSequenceDrawingParams Params = FSampledSequenceDrawingParams());

	/**
	* Generates an evenly divided grid from a given geometry, following the orientation provided in the Params struct argument.
	*
	* @param OutGridData			The generated grid coordinates and ratios per each dimension
	* @param InAllottedGeometry		The geometry the grid will be drawn into
	* @param NDimensions			Dimensions to account for in the given geometry
	* @param NumGridDivisions		Number of divisions the grid should be split into 
	* @param Params					Drawing Params
	*/
	void GenerateEvenlySplitGridForGeometry(TArray<FGridData>& OutGridData, const FGeometry& InAllottedGeometry, const uint16 NDimensions, const uint32 NumGridDivisions /*= 5*/, const FSampledSequenceDrawingParams Params = FSampledSequenceDrawingParams());
	
	void AUDIOWIDGETS_API GenerateEvenlySplitGridForLine(TArray<double>& OutDrawCoordinates, TArray<float>& OutLinePositionRatios, const float LineLength, const uint32 NumGridDivisions /*= 2*/, bool bEmptyOutArrays = true);

	/**
	* Generates a midline divided grid from a given geometry, following the orientation provided in the Params struct argument.
	* The grid space will be divided in half, with the resulting halves being divided again and so on until the desired Division Depth is reached. 
	*
	* @param OutGridData			The generated grid coordinates and ratios per each dimension
	* @param InAllottedGeometry		The geometry the grid will be drawn into
	* @param NDimensions			Dimensions to account for in the given geometry
	* @param DivisionDepth			Number of divisions that should be run
	* @param Params					Drawing Params
	*/
	void GenerateMidpointSplitGridForGeometry(TArray<FGridData>& OutGridData, const FGeometry& InAllottedGeometry, const uint16 NDimensions, const uint32 DivisionDepth = 3, const FSampledSequenceDrawingParams Params = FSampledSequenceDrawingParams());
	
	void AUDIOWIDGETS_API GenerateMidpointSplitGridForLine(TArray<double>& OutDrawCoordinates, TArray<float>& OutLinePositionRatios, const float LineLength, const uint32 DivisionDepth = 3, bool bEmptyOutArrays = true);
}

#undef UE_API
