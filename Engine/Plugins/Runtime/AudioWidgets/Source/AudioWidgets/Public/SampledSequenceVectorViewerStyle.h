// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Brushes/SlateColorBrush.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateWidgetStyle.h"

#include "SampledSequenceVectorViewerStyle.generated.h"

/**
 * Represents the appearance of a trigger threshold line
 */
USTRUCT(BlueprintType)
struct FSampledSequenceVectorViewerStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	FSampledSequenceVectorViewerStyle() {}

	virtual const FName GetTypeName() const override { return TypeName; }
	static const FSampledSequenceVectorViewerStyle& GetDefault() { static FSampledSequenceVectorViewerStyle Default; return Default; }
	virtual void GetResources(TArray< const FSlateBrush* >& OutBrushes) const override { OutBrushes.Add(&BackgroundBrush); };

	/** The background color. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateColor BackgroundColor = FLinearColor::Black;
	FSampledSequenceVectorViewerStyle& SetBackgroundColor(const FSlateColor InBackgroundColor) { BackgroundColor = InBackgroundColor; return *this; }

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush BackgroundBrush = FSlateColorBrush(FLinearColor::White);
	FSampledSequenceVectorViewerStyle& SetBackgroundBrush(const FSlateBrush InBackgroundBrush) { BackgroundBrush = InBackgroundBrush; return *this; }

	/** The vector view line color. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FLinearColor LineColor = FLinearColor::White;
	FSampledSequenceVectorViewerStyle& SetLineColor(const FLinearColor InLineColor) { LineColor = InLineColor; return *this; }

	/** The vector view line thickness. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float LineThickness = 1.0f;
	FSampledSequenceVectorViewerStyle& SetLineThickness(const float InLineThickness) { LineThickness = InLineThickness; return *this; }

	inline static FName TypeName = FName("FSampledSequenceVectorViewerStyle");
};
