// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "ISparseSampledSequenceTransportCoordinator.h"

#define UE_API AUDIOWIDGETS_API

DECLARE_MULTICAST_DELEGATE_OneParam(FOnDisplayRangeUpdated, const TRange<double> /* New Display Range */);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnFocusPointMoved, const double /* New Position */);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnFocusPointScrubUpdate, const float /* Focused Playback Ratio */, const bool /*Playhead is Moving*/);

class FSparseSampledSequenceTransportCoordinator : public ISparseSampledSequenceTransportCoordinator
{
public:
	FSparseSampledSequenceTransportCoordinator() = default;
	virtual ~FSparseSampledSequenceTransportCoordinator() = default;

	/** Called when the focus point is set to a new location */
	FOnFocusPointMoved OnFocusPointMoved;

	/** Called when the focus point is scrubbed */
	FOnFocusPointScrubUpdate OnFocusPointScrubUpdate;

	/** Called when the display range is updated */
	FOnDisplayRangeUpdated OnDisplayRangeUpdated;

	/** ISparseSampledSequenceTransportCoordinator interface */
	UE_API const TRange<double> GetDisplayRange() const;
	UE_API const double GetFocusPoint() const override;
	UE_API void ScrubFocusPoint(const float InTargetFocusPoint, const bool bIsMoving) override;
	UE_API const bool IsScrubbing() const;
	UE_API const bool IsScrolling() const;
	UE_API void SetScrolling(const bool bInIsScrolling);
	UE_API void SetProgressRatio(const float NewRatio) override;
	UE_API void SetZoomRatio(const float NewRatio) override;
	UE_API float ConvertAbsoluteRatioToZoomed(const float InAbsoluteRatio) const override;
	UE_API float ConvertZoomedRatioToAbsolute(const float InZoomedRatio) const override;
	UE_API double GetPlayBackRatioFromFocusPoint(const double InFocusPoint) const;
	UE_API void UpdatePlaybackRange(const TRange<double>& NewRange);
	UE_API void Stop();

	UE_API void MoveWaveformView(float MoveAmount);

private:
	inline void MoveFocusPoint(const double InFocusPoint);
	UE_API void UpdateZoomRatioAndDisplayRange(const float NewZoomRatio);

	UE_API void UpdateDisplayRange(const double MinValue, const double MaxValue);
	UE_API bool IsRatioWithinDisplayRange(const double Ratio) const;

	double FocusPointLockPosition = 0.95f;
	double FocusPoint = 0;
	float ZoomRatio = 1.f;

	/* The currently displayed render data range */
	TRange<double> DisplayRange = TRange<double>::Inclusive(0, 1);

	/* Progress range to scale the incoming progress ratio with*/
	TRange<double> ProgressRange = TRange<double>::Inclusive(0, 1);

	bool bIsScrubbing = false;
	bool bIsScrolling = false;
};

#undef UE_API
