// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsSlateTypes.h"
#include "Fonts/SlateFontInfo.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/StyleDefaults.h"

#include "TriggerThresholdLineStyle.generated.h"

/**
 * Represents the appearance of a trigger threshold line
 */
USTRUCT(BlueprintType)
struct FTriggerThresholdLineStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	FTriggerThresholdLineStyle() {}

	virtual const FName GetTypeName() const override { return TypeName; }
	static const FTriggerThresholdLineStyle& GetDefault() { static FTriggerThresholdLineStyle Default; return Default; }

	/** The trigger threshold line color. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FLinearColor LineColor = FLinearColor::White;
	FTriggerThresholdLineStyle& SetLineColor(const FLinearColor InLineColor) { LineColor = InLineColor; return *this; }

	inline static FName TypeName = FName("FTriggerThresholdLineStyle");
};
