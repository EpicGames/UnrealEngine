// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDefines.h"
#include "AudioOscilloscopeEnums.h"
#include "Containers/ArrayView.h"
#include "Containers/Ticker.h"
#include "DSP/Dsp.h"
#include "DSP/MultithreadedPatching.h"
#include "FixedSampledSequenceView.h"
#include "IFixedSampledSequenceViewProvider.h"

#define UE_API AUDIOWIDGETS_API

namespace Audio
{
	class FMixerDevice;
}

class UAudioBus;

struct FWaveformAudioSamplesResult;

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDataViewGenerated, FFixedSampledSequenceView /*InView*/, const uint32 /*FirstSampleIndex*/)

namespace AudioWidgets
{
	class FWaveformAudioSamplesDataProvider : public IFixedSampledSequenceViewProvider, 
															   public TSharedFromThis<FWaveformAudioSamplesDataProvider>
	{
	public:
		UE_API FWaveformAudioSamplesDataProvider(const Audio::FDeviceId InAudioDeviceId,
			UAudioBus* InAudioBus, 
			const uint32 InNumChannelToProvide, 
			const float InTimeWindowMs, 
			const float InMaxTimeWindowMs, 
			const float InAnalysisPeriodMs);

		UE_API virtual ~FWaveformAudioSamplesDataProvider();

		UE_API void ResetAudioBuffers();

		UE_API void StartProcessing();
		UE_API void StopProcessing();

		FFixedSampledSequenceView GetDataView() const { return DataView; };
		uint32 GetNumChannels() const { return NumChannels; }
		const UAudioBus* GetAudioBus() const { return AudioBus; }

		float GetMaxTimeWindowMs() const { return MaxTimeWindowMs; }
		UE_API void SetMaxTimeWindowMs(const float InMaxTimeWindowMs);

		UE_API void SetChannelToAnalyze(const int32 InChannel);
		UE_API void SetTriggerMode(const EAudioOscilloscopeTriggerMode InTriggerMode);
		UE_API void SetTriggerThreshold(const float InTriggerThreshold);
		UE_API void SetTimeWindow(const float InTimeWindowMs);
		UE_API void SetAnalysisPeriod(const float InAnalysisPeriodMs);

		UE_API virtual FFixedSampledSequenceView RequestSequenceView(const TRange<double> DataRatioRange) override;

		FOnDataViewGenerated OnDataViewGenerated;

	private:
		UE_API void PushAudioSamplesToCircularBuffer();
		UE_API bool Tick(float DeltaTime);

		FTSTicker::FDelegateHandle TickerHandle = nullptr;

		uint32 NumChannelsToProvide = 0;
		uint32 NumChannels          = 0;
		uint32 SampleRate           = 0;

		float MaxTimeWindowMs = 0.0f;

		const Audio::FMixerDevice* MixerDevice = nullptr;

		UAudioBus* AudioBus = nullptr;
		Audio::FPatchOutputStrongPtr PatchOutput = nullptr;

		TArray<float> TempAudioBuffer;
		Audio::TCircularAudioBuffer<float> AudioSamplesCircularBuffer;
		TArray<float> AudioSamplesForView;
		FFixedSampledSequenceView DataView;

		int32 ChannelIndexToAnalyze = 0;

		EAudioOscilloscopeTriggerMode TriggerMode = EAudioOscilloscopeTriggerMode::None;
		float TriggerThreshold = 0.0f;

		uint32 TimeWindowSamples     = 0;
		uint32 AnalysisPeriodSamples = 0;

		bool bIsProcessing = false;
		bool bHasTriggered = false;

		uint32 NumSamplesPushedToCircularBuffer = 0;
	};
} // namespace AudioWidgets

#undef UE_API
