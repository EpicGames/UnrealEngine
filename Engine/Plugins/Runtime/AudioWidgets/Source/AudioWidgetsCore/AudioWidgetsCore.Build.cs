// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioWidgetsCore : ModuleRules
{
	public AudioWidgetsCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange
		(
			new string[]
			{
				"Core",
			}
		);

		PrivateDependencyModuleNames.AddRange
		(
			new string[]
			{
				"AppFramework",
				"AudioMixerCore",
				"CoreUObject",
				"InputCore",
				"Slate",
				"SlateCore",
			}
		);

		bDisableAutoRTFMInstrumentation = true;
	}
}
