// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioMeterWidgetStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioMeterWidgetStyle)

const FAudioMeterWidgetStyle& FAudioMeterWidgetStyle::GetDefault()
{
	static FAudioMeterWidgetStyle Default;
	return Default;
}

void FAudioMeterWidgetStyle::GetResources(TArray<const FSlateBrush*>& OutBrushes) const
{
	OutBrushes.Add(&MeterValueImage);
	OutBrushes.Add(&MeterBackgroundImage);
	OutBrushes.Add(&MeterPeakImage);
	OutBrushes.Add(&MeterClippingImage);
}

void FAudioMeterWidgetStyle::UnlinkColors()
{
	MeterValueImage.UnlinkColors();
	MeterBackgroundImage.UnlinkColors();
	MeterPeakImage.UnlinkColors();
	MeterClippingImage.UnlinkColors();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FAudioMeterWidgetStyle& FAudioMeterWidgetStyle::SetFont(const FName& InFontName, uint16 InSize) { Font = FSlateFontInfo(InFontName, InSize); return *this; }
FAudioMeterWidgetStyle& FAudioMeterWidgetStyle::SetFont(const FString& InFontName, uint16 InSize) { Font = FSlateFontInfo(*InFontName, InSize); return *this; }
FAudioMeterWidgetStyle& FAudioMeterWidgetStyle::SetFont(const WIDECHAR* InFontName, uint16 InSize) { Font = FSlateFontInfo(InFontName, InSize); return *this; }
FAudioMeterWidgetStyle& FAudioMeterWidgetStyle::SetFont(const ANSICHAR* InFontName, uint16 InSize) { Font = FSlateFontInfo(InFontName, InSize); return *this; }
FAudioMeterWidgetStyle& FAudioMeterWidgetStyle::SetFontName(const FName& InFontName) { Font = FSlateFontInfo(InFontName, Font.Size); return *this; }
FAudioMeterWidgetStyle& FAudioMeterWidgetStyle::SetFontName(const FString& InFontName) { Font = FSlateFontInfo(InFontName, Font.Size); return *this; }
FAudioMeterWidgetStyle& FAudioMeterWidgetStyle::SetFontName(const WIDECHAR* InFontName) { Font = FSlateFontInfo(InFontName, Font.Size); return *this; }
FAudioMeterWidgetStyle& FAudioMeterWidgetStyle::SetFontName(const ANSICHAR* InFontName) { Font = FSlateFontInfo(InFontName, Font.Size); return *this; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS


const FAudioMeterDefaultColorWidgetStyle& FAudioMeterDefaultColorWidgetStyle::GetDefault()
{
	static const FAudioMeterDefaultColorWidgetStyle Default;
	return Default;
}
