// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioWidgetsCoreModule.h"

#include "Modules/ModuleManager.h"

void FAudioWidgetsCoreModule::StartupModule()
{
}

void FAudioWidgetsCoreModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FAudioWidgetsCoreModule, AudioWidgetsCore)
