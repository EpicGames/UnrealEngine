// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioWidgetsCoreStyle.h"

#include "Styling/AppStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

FAudioWidgetsCoreStyle::FAudioWidgetsCoreStyle()
	: FSlateStyleSet(StyleName)
{
	SetParentStyleName(FAppStyle::GetAppStyleSetName());
	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Runtime/AudioWidgets/Content"));
	SetCoreContentRoot(FPaths::EngineContentDir());

	const FVector2f Icon16x16 = FVector2f(16.0f, 16.0f);

	Set("AudioWidgetsCoreStyle.ColorMixing", new IMAGE_BRUSH_SVG(TEXT("Slate/Icons/ColorMixing"), Icon16x16));
	Set("AudioWidgetsCoreStyle.Reset", new CORE_IMAGE_BRUSH_SVG("Editor/Slate/Starship/Common/Reset", Icon16x16));
	Set("AudioWidgetsCoreStyle.Settings", new CORE_IMAGE_BRUSH_SVG("Slate/Starship/Common/Settings", Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FAudioWidgetsCoreStyle::~FAudioWidgetsCoreStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

const FAudioWidgetsCoreStyle& FAudioWidgetsCoreStyle::Get()
{
	static FAudioWidgetsCoreStyle Instance;
	return Instance;
}
