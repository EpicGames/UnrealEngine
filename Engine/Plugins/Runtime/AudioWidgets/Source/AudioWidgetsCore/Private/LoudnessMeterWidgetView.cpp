// Copyright Epic Games, Inc. All Rights Reserved.

#include "LoudnessMeterWidgetView.h"

#include "AudioDefines.h"
#include "AudioWidgetsCoreStyle.h"
#include "AudioMeterWidgetStyle.h"
#include "Brushes/SlateColorBrush.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LoudnessMeterSettings.h"
#include "Permutations.h"
#include "SAudioMeterWidget.h"
#include "SDragReorderableTileView.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"

#include <limits>

#define LOCTEXT_NAMESPACE "FLoudnessMeterWidgetView"

namespace AudioWidgetsCore
{
	namespace LoudnessMeterWidgetView_Private
	{
		using FLoudnessScaleParams = FLoudnessMeterWidgetView::FLoudnessScaleParams;
		using FLoudnessMetricRef = TSharedRef<const FLoudnessMeterWidgetView::FLoudnessMetric>;

		const FNumberFormattingOptions NumberFormattingOptions = FNumberFormattingOptions()
			.SetMinimumFractionalDigits(1)
			.SetMaximumFractionalDigits(1);

		const FText NegativeInfinityText = LOCTEXT("NegativeInfinity", "-∞");
		const FName ColorMixingBrushName(TEXT("AudioWidgetsCoreStyle.ColorMixing"));
		const FName ResetBrushName(TEXT("AudioWidgetsCoreStyle.Reset"));
		const FVector2D IconSize(16.0f, 16.0f);

		bool SupportsMaxValueDisplay(const FName& InLoudnessMeterName)
		{
			return InLoudnessMeterName == GET_MEMBER_NAME_CHECKED(FLoudnessMeterSettings, ShortTermLoudness) ||
				   InLoudnessMeterName == GET_MEMBER_NAME_CHECKED(FLoudnessMeterSettings, MomentaryLoudness) ||
			       InLoudnessMeterName == GET_MEMBER_NAME_CHECKED(FLoudnessMeterSettings, TruePeak);
		}

		void AddToggleMenuEntry(FMenuBuilder& InMenuBuilder, const FText& InLabel, const FExecuteAction& InToggleAction, const TAttribute<bool>& InIsChecked)
		{
			InMenuBuilder.AddMenuEntry(
				InLabel,
				FText(),
				FSlateIcon(),
				FUIAction(
					InToggleAction,
					FCanExecuteAction::CreateLambda([bCanExecute = InToggleAction.IsBound()]() { return bCanExecute; }),
					FIsActionChecked::CreateLambda([InIsChecked]() { return InIsChecked.Get(); })
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);
		}

		TSharedRef<SWidget> MakeColorPickerRow(
			const TAttribute<FLinearColor>& InColor,
			const FOnLinearColorValueChanged& InOnColorChanged,
			const FOnWindowClosed& InOnPickerClosed,
			const FSimpleDelegate& InOnReset,
			const FText& InLabel,
			const FText& InTooltip,
			const FText& InResetTooltip,
			const FLinearColor& InDefaultColor)
		{
			return SNew(SBox)
				.Padding(FMargin(10.0f, 0.0f, 0.0f, 0.0f))
				[
					SNew(SHorizontalBox)
					// Color picker button
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.VAlign(VAlign_Center)
					[
						SNew(SButton)
						.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
						.ToolTipText(InTooltip)
						.OnClicked_Lambda([ParentWidget = FSlateApplication::Get().GetMenuHostWidget().ToWeakPtr(), InColor, InOnColorChanged, InOnPickerClosed]()
						{
							FColorPickerArgs PickerArgs(InColor.Get(), InOnColorChanged);
							PickerArgs.ParentWidget = ParentWidget.Pin();
							PickerArgs.bUseAlpha = true;
							PickerArgs.bOnlyRefreshOnMouseUp = true;
							PickerArgs.OnColorPickerWindowClosed = InOnPickerClosed;
							OpenColorPicker(PickerArgs);

							return FReply::Handled();
						})
						[
							SNew(SHorizontalBox)
							// Color icon
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(0.0f, 0.0f, 12.0f, 0.0f)
							[
								SNew(SImage)
								.Image(FAudioWidgetsCoreStyle::Get().GetBrush(ColorMixingBrushName))
								.ColorAndOpacity(FSlateColor::UseForeground())
								.DesiredSizeOverride(IconSize)
							]
							// Color label
							+ SHorizontalBox::Slot()
							.FillWidth(1.0f)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.Text(InLabel)
								.ColorAndOpacity(FLinearColor::White)
							]
						]
					]
					// Reset button
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SButton)
						.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
						.ToolTipText(InResetTooltip)
						.Visibility_Lambda([InColor, InDefaultColor]()
						{
							return InColor.Get().Equals(InDefaultColor) ? EVisibility::Hidden : EVisibility::Visible;
						})
						.OnClicked_Lambda([InOnReset]()
						{
							InOnReset.ExecuteIfBound();

							return FReply::Handled();
						})
						[
							SNew(SImage)
							.Image(FAudioWidgetsCoreStyle::Get().GetBrush(ResetBrushName))
							.ColorAndOpacity(FSlateColor::UseForeground())
							.DesiredSizeOverride(IconSize)
						]
					]
				];
		}

		/**
		 * Derived meter widget type adds bindable attributes for scale DisplayRange and Offset.
		 */
		class SVariableDisplayRangeMeterWidget : public SAudioMeterWidget
		{
		public:
			SLATE_BEGIN_ARGS(SVariableDisplayRangeMeterWidget)
				: _MeterValueColor(FLinearColor::Green)
				, _MaxValueColor(FLinearColor::Transparent)
				, _TargetValueColor(FLinearColor::Transparent)
				, _DisplayRange(60)
				, _Offset(0)
				{
				}

				SLATE_ATTRIBUTE(FSlateColor, MeterValueColor)
				SLATE_ATTRIBUTE(FSlateColor, MaxValueColor)
				SLATE_ATTRIBUTE(FSlateColor, TargetValueColor)
				SLATE_ATTRIBUTE(TArray<FAudioMeterChannelInfo>, MeterChannelInfo)
				SLATE_ATTRIBUTE(int32, DisplayRange)
				SLATE_ATTRIBUTE(int32, Offset)

			SLATE_END_ARGS()

			void Construct(const SVariableDisplayRangeMeterWidget::FArguments& InArgs)
			{
				AudioMeterWidgetStyle->SetMeterSize({ 700.0, 25.0 });			// MaxLength, Width.
				AudioMeterWidgetStyle->SetScaleHashHeight(4.0f);				// This is the tick mark length.
				AudioMeterWidgetStyle->SetPeakValueWidth(1.0f);					// The width used for displaying the max value indicator
				AudioMeterWidgetStyle->SetClippingValueWidth(1.0f);				// The width used for displaying the loudness target / true peak threshold value
				AudioMeterWidgetStyle->SetFont(FStyleDefaults::GetFontInfo(8));	// The font size for the meter dB value labels

				DisplayRange = InArgs._DisplayRange;
				Offset = InArgs._Offset;

				SAudioMeterWidget::Construct(
					SAudioMeterWidget::FArguments()
						.Orientation(EOrientation::Orient_Vertical)
						.BackgroundColor(FLinearColor::Transparent)
						.MeterBackgroundColor(FAudioMeterDefaultColorWidgetStyle::GetDefault().MeterBackgroundColor)
						.MeterPeakColor(InArgs._MaxValueColor)
						.MeterClippingColor(InArgs._TargetValueColor)
						.MeterScaleColor(FLinearColor::White.CopyWithNewOpacity(0.25f))
						.MeterScaleLabelColor(FLinearColor::White.CopyWithNewOpacity(1.0f))
						.MeterValueColor(InArgs._MeterValueColor)
						.MeterChannelInfo(InArgs._MeterChannelInfo)
						.Style(AudioMeterWidgetStyle.Get())
					);
			}

			virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
			{
				// Update the style from our attribute bindings:
				const float DisplayRangeValue = DisplayRange.Get();
				const float OffsetValue = Offset.Get();
				AudioMeterWidgetStyle->SetValueRangeDb({ OffsetValue - DisplayRangeValue, OffsetValue });

				// Paint the widget using the updated style:
				return SAudioMeterWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
			}

		private:
			const TUniquePtr<FAudioMeterWidgetStyle> AudioMeterWidgetStyle = MakeUnique<FAudioMeterWidgetStyle>(FAudioMeterWidgetStyle::GetDefault());
			TAttribute<int32> Offset;
			TAttribute<int32> DisplayRange;
		};

		/**
		 * If the Relative Scale attribute is true this will convert values given in LUFS into values in LU, relative to the Target attribute.
		 * Otherwise it will leave the LUFS values as-is.
		 */
		class FLoudnessValueConverter
		{
		public:
			FLoudnessValueConverter(const FLoudnessScaleParams& LoudnessScaleParams)
				: bRelativeScale(LoudnessScaleParams.bRelativeScale)
				, Target(LoudnessScaleParams.Target)
			{
			}

			float ConvertValue(float Value) const
			{
				return (bRelativeScale.Get()) ? (Value - Target.Get()) : Value;
			}

			TOptional<float> ConvertValue(TOptional<float> Value) const
			{
				if (Value.IsSet())
				{
					return ConvertValue(*Value);
				}
				return NullOpt;
			}

			FText FormatNumericValueText(TOptional<float> Value) const
			{
				if (!Value.IsSet())
				{
					return FText::GetEmpty();
				}
				else if (*Value <= MIN_VOLUME_DECIBELS)
				{
					return NegativeInfinityText;
				}

				const float ConvertedValue = ConvertValue(*Value);
				return FText::AsNumber(ConvertedValue, &NumberFormattingOptions);
			}

			FText FormatRangeText(TOptional<FFloatInterval> Range) const
			{
				if (!Range.IsSet())
				{
					return FText::GetEmpty();
				}

				// Range size is already relative units, so no need to call ConvertValue.
				return FText::AsNumber(Range->Size(), &NumberFormattingOptions);
			}

		private:
			TAttribute<bool> bRelativeScale;
			TAttribute<int32> Target;
		};

		/** 
		 * Creates a widget that displays a max value checkbox.
		 */
		TSharedRef<SWidget> CreateMaxValueCheckboxWidget(FLoudnessMetricRef InItem, TAttribute<bool> bInHoldMax, const FSimpleDelegate& InOnToggleRequested)
		{
			return SNew(SCheckBox)
				.ToolTipText(LOCTEXT("MaxValueCheckboxWidget_Tooltip", "Enable to hold the max value for this loudness metric"))
				.Style(&FAppStyle::GetWidgetStyle<FCheckBoxStyle>("RadioButton"))
				.IsChecked_Lambda([InItem, bInHoldMax]()
				{
					return bInHoldMax.Get() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([InOnToggleRequested](ECheckBoxState NewState)
				{
					InOnToggleRequested.ExecuteIfBound();
				})
				[
					SNew(STextBlock)
					.ColorAndOpacity_Lambda([InItem, bInHoldMax]()
					{
						return bInHoldMax.Get() ? FColor::Yellow : FColor::Silver;
					})
					.Font(FStyleDefaults::GetFontInfo(7))
					.Text(LOCTEXT("HoldMax", "Max"))
				];
		}

		/**
		 * Creates a widget that displays the numeric value of the given loudness metric, with units.
		 */
		TSharedRef<SWidget> MakeValueReadoutWidget(FLoudnessMetricRef InItem, const FLoudnessScaleParams& LoudnessScaleParams, TAttribute<bool> bInHoldMax)
		{
			const FLoudnessValueConverter LoudnessValueConverter(LoudnessScaleParams);

			const auto GetValueColorAndOpacity = [InItem, LoudnessScaleParams, bInHoldMax]()
			{
				bool bIsValueAboveThreshold = false;

				const TOptional<float>& ValueOpt = bInHoldMax.Get() ? InItem->MaxValue.Get() : InItem->Value.Get();

				if (ValueOpt.IsSet())
				{
					const float Value = ValueOpt.GetValue();

					switch (InItem->MeterMetric.Get())
					{
						// True Peak meter
						case EAudioMeterMetric::Decibels:
						{
							const float TruePeakLimit = LoudnessScaleParams.TruePeakLimit.Get();
							if (Value > TruePeakLimit)
							{
								bIsValueAboveThreshold = true;
							}

							break;
						}

						// Loudness meters
						case EAudioMeterMetric::Loudness:
						{
							const int32 Target = LoudnessScaleParams.Target.Get();
							if (!InItem->IsRanged() && Value > Target)
							{
								bIsValueAboveThreshold = true;
							}

							break;
						}

						default:
							break;
					}
				}

				return bIsValueAboveThreshold ? FLinearColor::Red : InItem->Color.Get().CopyWithNewOpacity(1.0f);
			};

			return SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SSpacer)
						.Size(FVector2D(46.0f, 28.0f))
				]
				+ SOverlay::Slot()
				[
					SNew(STextBlock)
						.Text_Lambda([InItem, LoudnessValueConverter, bInHoldMax]()
							{
								switch (InItem->MeterMetric.Get())
								{
									// True Peak meter
									case EAudioMeterMetric::Decibels:
									{
										const TOptional<float>& ValueOpt = bInHoldMax.Get() ? InItem->MaxValue.Get() : InItem->Value.Get();
										if (!ValueOpt.IsSet())
										{
											return FText::GetEmpty();
										}

										const float Value = ValueOpt.GetValue();

										return (Value <= MIN_VOLUME_DECIBELS)
											? NegativeInfinityText
											: FText::AsNumber(Value, &NumberFormattingOptions);
									}

									// Loudness meters
									case EAudioMeterMetric::Loudness:
									{
										return InItem->IsRanged()
											? LoudnessValueConverter.FormatRangeText(InItem->Range.Get())
											: LoudnessValueConverter.FormatNumericValueText(bInHoldMax.Get() ? InItem->MaxValue.Get() : InItem->Value.Get());
									}

									default:
										break;
								}

								return FText::GetEmpty();
							})
						.Font(FStyleDefaults::GetFontInfo(14))
						.ColorAndOpacity_Lambda(GetValueColorAndOpacity)
						.Justification(ETextJustify::Right)
				]
				+ SOverlay::Slot()
				.VAlign(EVerticalAlignment::VAlign_Bottom)
				[
					SNew(STextBlock)
						.Text_Lambda([InItem, bRelativeScale = LoudnessScaleParams.bRelativeScale]()
						{
							switch (InItem->MeterMetric.Get())
							{
								case EAudioMeterMetric::Decibels:
									return INVTEXT("dB");

								case EAudioMeterMetric::Loudness:
									return (InItem->IsRanged() || bRelativeScale.Get()) ? INVTEXT("LU") : INVTEXT("LUFS");

								default:
									break;
							}

							return FText::GetEmpty();
						})
						.Font(FStyleDefaults::GetFontInfo(6))
						.ColorAndOpacity_Lambda(GetValueColorAndOpacity)
						.Justification(ETextJustify::Right)
				];
		}

		/**
		 * Creates a widget with the actual content of a Value tile, containing the numeric value readout widget and the loudness metric display name.
		 */
		TSharedRef<SWidget> MakeValueTileInnerContent(FLoudnessMetricRef InItem, FLoudnessScaleParams LoudnessScaleParams)
		{
			return SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.MinHeight(2)
				.MaxHeight(2)
				[
					SNew(SSpacer)
					.Size(FVector2D(68.0f, 2.0f))
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(EHorizontalAlignment::HAlign_Center)
				[
					MakeValueReadoutWidget(InItem, LoudnessScaleParams, InItem->bHoldMaxForValue)
				]
				+ SVerticalBox::Slot()
				.MinHeight(24)
				.MaxHeight(24)
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SNew(STextBlock)
						.Text(InItem->DisplayName)
						.Font(FStyleDefaults::GetFontInfo(8))
						.AutoWrapText(true)
						.Justification(ETextJustify::Center)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(EHorizontalAlignment::HAlign_Right)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SupportsMaxValueDisplay(InItem->Name) ? CreateMaxValueCheckboxWidget(InItem, InItem->bHoldMaxForValue, InItem->OnHoldMaxForValueToggleRequested) : SNullWidget::NullWidget
				];
		}

		/**
		 * Creates a widget with the actual content of a Meter tile, containing the numeric value readout widget, the actual meter widget, and the loudness metric display name.
		 */
		TSharedRef<SWidget> MakeMeterTileInnerContent(FLoudnessMetricRef InItem, FLoudnessScaleParams LoudnessScaleParams)
		{
			const FLoudnessValueConverter LoudnessValueConverter(LoudnessScaleParams);

			return SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(EHorizontalAlignment::HAlign_Center)
				[
					MakeValueReadoutWidget(InItem, LoudnessScaleParams, InItem->bHoldMaxForMeter)
				]
				+ SVerticalBox::Slot()
				.HAlign(EHorizontalAlignment::HAlign_Center)
				[
					SNew(SVariableDisplayRangeMeterWidget)
						.MeterValueColor_Lambda([InItem]() { return InItem->Color.Get(); })
						.MaxValueColor_Lambda([InItem, LoudnessScaleParams]()
						{
							if (!InItem->IsRanged())
							{
								const TOptional<float>& MaxValueOpt = InItem->MaxValue.Get();

								if (MaxValueOpt.IsSet() && MaxValueOpt.GetValue() > MIN_VOLUME_DECIBELS)
								{
									const float MaxValue = MaxValueOpt.GetValue();
									bool bIsValueAboveThreshold = false;

									switch (InItem->MeterMetric.Get())
									{
										case EAudioMeterMetric::Decibels:
											bIsValueAboveThreshold = MaxValue > LoudnessScaleParams.TruePeakLimit.Get();
											break;

										case EAudioMeterMetric::Loudness:
											bIsValueAboveThreshold = MaxValue > LoudnessScaleParams.Target.Get();
											break;

										default:
											break;
									}

									return bIsValueAboveThreshold ? FLinearColor::Red : InItem->Color.Get().CopyWithNewOpacity(1.0f);
								}
							}

							return FLinearColor::Transparent;
						})
						.TargetValueColor_Lambda([InItem, LoudnessScaleParams]()
						{
							const int32 Range  = LoudnessScaleParams.Range.Get();
							const int32 Offset = LoudnessScaleParams.Offset.Get();

							const int32 MinRangeValue = Offset - Range;
							const int32 MaxRangeValue = Offset;

							switch (InItem->MeterMetric.Get())
							{
								// True Peak meter
								case EAudioMeterMetric::Decibels:
								{
									const float TruePeakLimit = LoudnessScaleParams.TruePeakLimit.Get();

									if (TruePeakLimit >= MinRangeValue && TruePeakLimit <= MaxRangeValue)
									{
										return LoudnessScaleParams.TargetColor.Get();
									}

									break;
								}

								// Loudness meters
								case EAudioMeterMetric::Loudness:
								{
									if (!InItem->IsRanged())
									{
										const int32 Target = LoudnessScaleParams.Target.Get();

										if (Target >= MinRangeValue && Target <= MaxRangeValue)
										{
											return LoudnessScaleParams.TargetColor.Get();
										}
									}

									break;
								}

								default:
									break;
							}

							return FLinearColor::Transparent;
						})
						.MeterChannelInfo_Lambda([InItem, LoudnessValueConverter, Target = LoudnessScaleParams.Target, bRelativeScale = LoudnessScaleParams.bRelativeScale, TruePeakLimit = LoudnessScaleParams.TruePeakLimit]() -> TArray<FAudioMeterChannelInfo>
							{
								constexpr float DefaultInvalidValue = MIN_VOLUME_DECIBELS;

								if (InItem->MeterMetric.Get() == EAudioMeterMetric::Decibels)
								{
									const TOptional<float>& MaxValueOpt = InItem->MaxValue.Get();
									const TOptional<float>& ValueOpt    = InItem->bHoldMaxForMeter.Get() ? MaxValueOpt : InItem->Value.Get();

									return
									{
										{
											.MeterValue    = ValueOpt.Get(DefaultInvalidValue),
											.PeakValue     = MaxValueOpt.IsSet() ? MaxValueOpt.GetValue() : DefaultInvalidValue,
											.ClippingValue = TruePeakLimit.Get()
										}
									};
								}
								else if (InItem->IsRanged())
								{
									// Display loudness meter bar as a range:
									const TOptional<FFloatInterval> Range = InItem->Range.Get();
									const TOptional<float> ConvertedMin = LoudnessValueConverter.ConvertValue(Range.IsSet() ? Range->Min : TOptional<float>());
									const TOptional<float> ConvertedMax = LoudnessValueConverter.ConvertValue(Range.IsSet() ? Range->Max : TOptional<float>());
									return
									{
										{
											.MeterValue = ConvertedMax.Get(DefaultInvalidValue),
											.PeakValue = DefaultInvalidValue,
											.MeterValueBarStart = ConvertedMin
										}
									};
								}

								// Display standard loudness meter bar:
								const TOptional<float> MaxValueOpt = InItem->MaxValue.Get();
								const TOptional<float> ValueOpt    = InItem->bHoldMaxForMeter.Get() ? MaxValueOpt : InItem->Value.Get();
								
								const TOptional<float> ConvertedValueOpt = LoudnessValueConverter.ConvertValue(ValueOpt);
								const TOptional<float> ConvertedMaxOpt   = LoudnessValueConverter.ConvertValue(MaxValueOpt);

								return
								{
									{
										.MeterValue    = ConvertedValueOpt.Get(DefaultInvalidValue),
										.PeakValue     = ConvertedMaxOpt.Get(DefaultInvalidValue),
										.ClippingValue = bRelativeScale.Get() ? 0.0f : static_cast<float>(Target.Get())
									}
								};
							})
						.DisplayRange(LoudnessScaleParams.Range)
						.Offset_Lambda([InItem, Offset = LoudnessScaleParams.Offset, LoudnessValueConverter]
						{
							if (InItem->MeterMetric.Get() == EAudioMeterMetric::Decibels)
							{
								return Offset.Get();
							}

							return static_cast<int32>(LoudnessValueConverter.ConvertValue(Offset.Get()));
						})
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.MinHeight(28)
				.HAlign(EHorizontalAlignment::HAlign_Center)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SNew(STextBlock)
						.Text(InItem->DisplayName)
						.Font(FStyleDefaults::GetFontInfo(8))
						.AutoWrapText(true)
						.Justification(ETextJustify::Center)
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.MinHeight(20)
				.HAlign(EHorizontalAlignment::HAlign_Right)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SupportsMaxValueDisplay(InItem->Name) ? CreateMaxValueCheckboxWidget(InItem, InItem->bHoldMaxForMeter, InItem->OnHoldMaxForMeterToggleRequested) : SNullWidget::NullWidget
				];
		}

		DECLARE_DELEGATE_RetVal_OneParam(TSharedRef<SWidget>, FOnMakeTileInnerContent, FLoudnessMetricRef);

		/**
		 * Represents a loudness metric that is being dragged from a given source tile view.
		 * A decorator renders the content of the tile, with a rounded border selection highlight.
		 */
		class FLoudnessValueDragDropOp : public TItemDragDropOperation<FLoudnessMetricRef, FLoudnessValueDragDropOp>
		{
		public:
			using Base = TItemDragDropOperation<FLoudnessMetricRef, FLoudnessValueDragDropOp>;

			DRAG_DROP_OPERATOR_TYPE(FLoudnessValueDragDropOp, Base)

			FLoudnessValueDragDropOp(TSharedRef<Base::STileViewType> InSourceTileView, FLoudnessMetricRef InLoudnessMetric, FOnMakeTileInnerContent InOnMakeTileInnerContent)
				: Base(InSourceTileView)
				, LoudnessMetric(InLoudnessMetric)
				, OnMakeTileInnerContent(InOnMakeTileInnerContent)
			{
				Construct();
			}

			virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

			virtual const FLoudnessMetricRef& GetItem() const final { return LoudnessMetric; }

		private:
			const FLoudnessMetricRef LoudnessMetric;
			const FOnMakeTileInnerContent OnMakeTileInnerContent;
		};

		TSharedPtr<SWidget> FLoudnessValueDragDropOp::GetDefaultDecorator() const
		{
			static const FSlateColorBrush BackgroundPanelBrush(FStyleColors::Panel);
			static const FSlateRoundedBoxBrush BorderBrush(FStyleColors::Select, 4.0f);
			static const FSlateRoundedBoxBrush InteriorPanelBrush(FStyleColors::Panel, 4.0f);

			TSharedRef<SWidget> TileInnerContent = (OnMakeTileInnerContent.IsBound()) ? OnMakeTileInnerContent.Execute(LoudnessMetric) : SNullWidget::NullWidget;

			return SNew(SBox)
				.MaxDesiredHeight(180.0f)
				.MinDesiredWidth(66.0f)
				[
					SNew(SBorder)
						.BorderImage(&BackgroundPanelBrush)
						.Padding(0.0f)
						[
							SNew(SBorder)
								.BorderImage(&BorderBrush)
								.Padding(1.0f)
								[
									SNew(SBorder)
										.BorderImage(&InteriorPanelBrush)
										[
											TileInnerContent
										]
								]
						]
				];
		}

		/**
		 * Delegate adapter to execute the necessary delegate to show or hide the numeric value of the given loudness metric.
		 */
		void SetShowLoudnessMetricValue(const FLoudnessMetricRef& LoudnessMetric, bool bShowValue)
		{
			if (LoudnessMetric->bShowValue.Get() != bShowValue)
			{
				LoudnessMetric->OnShowValueToggleFromDragDropRequested.ExecuteIfBound();
			}
		}

		/**
		 * Delegate adapter to execute the necessary delegate to show or hide the meter for the given loudness metric.
		 */
		void SetShowLoudnessMetricMeter(const FLoudnessMetricRef& LoudnessMetric, bool bShowMeter)
		{
			if (LoudnessMetric->bShowMeter.Get() != bShowMeter)
			{
				LoudnessMetric->OnShowMeterToggleFromDragDropRequested.ExecuteIfBound();
			}
		}

		/**
		 * TileView widget for displaying an observable collection of loudness metrics.
		 * Dropping of a dragged loudness metric onto the tile view is supported if the tile view is currently empty.
		 */
		class SLoudnessMetricTileView : public SDragReorderableTileView<FLoudnessValueDragDropOp>
		{
		public:
			using Base = SDragReorderableTileView<FLoudnessValueDragDropOp>;

			void Construct(const Base::FArguments& InArgs, FOnMakeTileInnerContent InOnMakeTileInnerContent, FOnSetShowItem InOnSetShowItem, FOnReorderItems InOnReorderItems)
			{
				Base::Construct(InArgs, InOnSetShowItem, InOnReorderItems);

				OnMakeTileInnerContent = InOnMakeTileInnerContent;
			}

			TSharedRef<SWidget> MakeTileInnerContent(const FLoudnessMetricRef& LoudnessMetric) const
			{
				return (OnMakeTileInnerContent.IsBound()) ? OnMakeTileInnerContent.Execute(LoudnessMetric) : SNullWidget::NullWidget;
			}

			virtual TSharedPtr<FLoudnessValueDragDropOp> CreateDragDropOperation() override
			{
				ensure(GetSelectionMode() == ESelectionMode::Single);
				if (FLoudnessMetricRef* SelectedItem = SelectedItems.FindArbitraryElement()) // As we are using ESelectionMode::Single an arbitrary element is the only element.
				{
					TSharedRef<FLoudnessValueDragDropOp> DragDropOperation = MakeShared<FLoudnessValueDragDropOp>(SharedThis(this), *SelectedItem, OnMakeTileInnerContent);

					// Hold a weak pointer to maintain awareness of whether a tile is currently being dragged:
					WeakDragDropOperation = DragDropOperation;

					return DragDropOperation;
				}
				return nullptr;
			}

			bool HasDragDropOperation() const { return WeakDragDropOperation.IsValid(); }

		private:
			FOnMakeTileInnerContent OnMakeTileInnerContent;
			TWeakPtr<FDragDropOperation> WeakDragDropOperation;
		};


		/**
		 * Tile widget for a single loudness metric. Dragging and dropping tiles is supported.
		 */
		class SLoudnessMetricTile : public SDragReorderableTile<FLoudnessValueDragDropOp>
		{
		public:
			const FSlateBrush* GetHoveredBorder() const
			{
				if (IsHovered())
				{
					static const FSlateRoundedBoxBrush RoundedHoverBrush(FStyleColors::Hover, 4.0f);
					return &RoundedHoverBrush;
				}
				return nullptr;
			}

			FLinearColor GetOverlayColor() const
			{
				if (IsBeingDragged())
				{
					return FStyleColors::Recessed.GetSpecifiedColor().CopyWithNewOpacity(0.9f);
				}
				return FLinearColor::Transparent;
			}
		};


		/**
		 * Helper function to create an empty tile widget, with common styling applied.
		 */
		TSharedRef<SLoudnessMetricTile> MakeEmptyTileWidget(const TSharedRef<STableViewBase>& OwnerTable)
		{
			const auto MakeTableRowStyle = []() -> FTableRowStyle
				{
					const FSlateRoundedBoxBrush RoundedPanelBrush(FStyleColors::Panel, 4.0f);

					FTableRowStyle TableRowStyle = FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row");
					TableRowStyle.SetEvenRowBackgroundHoveredBrush(RoundedPanelBrush);
					TableRowStyle.SetOddRowBackgroundHoveredBrush(RoundedPanelBrush);
					TableRowStyle.SetSelectorFocusedBrush(FSlateNoResource());
					return TableRowStyle;
				};
			static FTableRowStyle LoudnessMetricTableRowStyle = MakeTableRowStyle();

			return SNew(SLoudnessMetricTile, OwnerTable)
				.ShowSelection(false)
				.Style(&LoudnessMetricTableRowStyle);
		}

		/**
		 * Callback to create a tile widget for the numeric Values tile view. A color block overlay will be used to darken the tile content when it's being dragged.
		 */
		TSharedRef<ITableRow> MakeNumericValueTileWidget(FLoudnessMetricRef InItem, const TSharedRef<STableViewBase>& OwnerTable)
		{
			TSharedRef<SLoudnessMetricTileView> OwnerTileView = StaticCastSharedRef<SLoudnessMetricTileView>(OwnerTable);

			TSharedRef<SLoudnessMetricTile> TileWidget = MakeEmptyTileWidget(OwnerTileView);

			TileWidget->SetContent(
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					OwnerTileView->MakeTileInnerContent(InItem)
				]
				+ SOverlay::Slot()
				[
					SNew(SColorBlock)
					.Visibility(EVisibility::HitTestInvisible)
					.Color(TileWidget, &SLoudnessMetricTile::GetOverlayColor)
				]);

			return TileWidget;
		}

		/**
		 * Callback to create a tile widget for the Meters tile view. A color block overlay will be used to darken the tile content when it's being dragged.
		 */
		TSharedRef<ITableRow> MakeMeterValueTileWidget(FLoudnessMetricRef InItem, const TSharedRef<STableViewBase>& OwnerTable)
		{
			static const FSlateRoundedBoxBrush RoundedPanelBrush(FStyleColors::Panel, 4.0f);

			TSharedRef<SLoudnessMetricTileView> OwnerTileView = StaticCastSharedRef<SLoudnessMetricTileView>(OwnerTable);

			TSharedRef<SLoudnessMetricTile> TileWidget = MakeEmptyTileWidget(OwnerTileView);

			TileWidget->SetContent(
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SColorBlock)
						.Color(FStyleColors::Recessed.GetSpecifiedColor())
				]
				+ SOverlay::Slot()
				[
					SNew(SBox)
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(3.0f)
						[
							SNew(SBorder)
								.BorderImage(TileWidget, &SLoudnessMetricTile::GetHoveredBorder)
								.Padding(1.0f)
								[
									SNew(SBorder)
										.BorderImage(&RoundedPanelBrush)
										.Padding(0.0f, 8.0f)
										[
											OwnerTileView->MakeTileInnerContent(InItem)
										]
								]
						]
				]
				+ SOverlay::Slot()
				[
					SNew(SColorBlock)
					.Visibility(EVisibility::HitTestInvisible)
					.Color(TileWidget, &SLoudnessMetricTile::GetOverlayColor)
				]);

			return TileWidget;
		}
	} // namespace LoudnessMeterWidgetView_Private

	FLinearColor FLoudnessMeterWidgetView::GetDefaultMeterColor()
	{
		return FStyleColors::AccentGreen.GetSpecifiedColor().CopyWithNewOpacity(0.5f);
	}

	FLinearColor FLoudnessMeterWidgetView::GetDefaultMeterColor(const FName& InLoudnessMeterName)
	{
		static const FName LoudnessRangeName = GET_MEMBER_NAME_CHECKED(FLoudnessMeterSettings, LoudnessRange);
		static const FName TruePeakName      = GET_MEMBER_NAME_CHECKED(FLoudnessMeterSettings, TruePeak);

		if (InLoudnessMeterName == LoudnessRangeName)
		{
			return FLoudnessMeterSettings::DefaultLoudnessRangeColor;
		}
		else if (InLoudnessMeterName == TruePeakName)
		{
			return FLoudnessMeterSettings::DefaultTruePeakColor;
		}

		return GetDefaultMeterColor();
	}

	FLinearColor FLoudnessMeterWidgetView::GetDefaultTargetColor()
	{
		return FStyleColors::AccentOrange.GetSpecifiedColor();
	}

	FLoudnessMeterWidgetView::FLoudnessMeterWidgetView()
		: LoudnessMetrics(MakeShared<FLoudnessMetricRefArray>())
		, ValuesCollectionView(MakeShared<FLoudnessMetricCollectionView>(LoudnessMetrics, [](const FLoudnessMetricRef& LoudnessMetricRef) { return LoudnessMetricRef->bShowValue.Get(); }))
		, MetersCollectionView(MakeShared<FLoudnessMetricCollectionView>(LoudnessMetrics, [](const FLoudnessMetricRef& LoudnessMetricRef) { return LoudnessMetricRef->bShowMeter.Get(); }))
	{
	}

	FLoudnessMeterWidgetView::~FLoudnessMeterWidgetView()
	{
		LoudnessMetrics->Reset();
		ValuesCollectionView->ResetVisibleItems();
		MetersCollectionView->ResetVisibleItems();
	}

	void FLoudnessMeterWidgetView::InitTimerPanel(const FTimerPanelParams& InTimerPanelParams)
	{
		TimerPanelParams = InTimerPanelParams;
	}

	void FLoudnessMeterWidgetView::InitLoudnessScale(const FLoudnessScaleParams& InLoudnessScaleParams)
	{
		LoudnessScaleParams = InLoudnessScaleParams;
	}

	void FLoudnessMeterWidgetView::InitValuesDisplayOrder(const FDisplayOrderParams& InDisplayOrderParams)
	{
		ValuesCollectionView->SetDisplayOrderParams(InDisplayOrderParams);
	}

	void FLoudnessMeterWidgetView::InitMetersDisplayOrder(const FDisplayOrderParams& InDisplayOrderParams)
	{
		MetersCollectionView->SetDisplayOrderParams(InDisplayOrderParams);
	}

	TSharedRef<SWidget> FLoudnessMeterWidgetView::MakeWidget() const
	{
		using namespace LoudnessMeterWidgetView_Private;

		static FSlateColorBrush BackgroundBrush(FStyleColors::Recessed);

		const TSharedRef<FUICommandList> CommandList = MakeShared<FUICommandList>();

		const auto MakeOptionsMenu = [
			CommandList,
			TimerPanelParams = this->TimerPanelParams,
			LoudnessScaleParams = this->LoudnessScaleParams,
			LoudnessMetrics = this->LoudnessMetrics]() -> TSharedRef<SWidget>
			{
				constexpr bool bShouldCloseWindowAfterMenuSelection = true;
				FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

				{
					MenuBuilder.BeginSection(NAME_None, LOCTEXT("LoudnessScale", "Loudness Scale"));

					if (LoudnessScaleParams.OnRangeValueCommitted.IsBound())
					{
						TSharedRef RangeWidget = SNew(SNumericEntryBox<int32>)
							.MinDesiredValueWidth(100.0f)
							.MinSliderValue(10)
							.MaxSliderValue(70)
							.AllowSpin(true)
							.Value_Lambda([Range = LoudnessScaleParams.Range]() { return Range.Get(); })
							.OnValueChanged(LoudnessScaleParams.OnRangeValueChanged)
							.OnValueCommitted(LoudnessScaleParams.OnRangeValueCommitted);

						MenuBuilder.AddWidget(RangeWidget, LOCTEXT("LoudnessScale_Range", "Range"), false, true, LOCTEXT("LoudnessScale_Range_ToolTip", "Range of the loudness meter scale in LUFS"));
					}

					if (LoudnessScaleParams.OnOffsetValueCommitted.IsBound())
					{
						TSharedRef OffsetWidget = SNew(SNumericEntryBox<int32>)
							.MinDesiredValueWidth(100.0f)
							.MinSliderValue(-10)
							.MaxSliderValue(30)
							.AllowSpin(true)
							.Value_Lambda([Offset = LoudnessScaleParams.Offset]() { return Offset.Get(); })
							.OnValueChanged(LoudnessScaleParams.OnOffsetValueChanged)
							.OnValueCommitted(LoudnessScaleParams.OnOffsetValueCommitted);

						MenuBuilder.AddWidget(OffsetWidget, LOCTEXT("LoudnessScale_Offset", "Offset"), false, true, LOCTEXT("LoudnessScale_Offset_ToolTip", "Offset of the loudness meter scale in LUFS"));
					}

					if (LoudnessScaleParams.OnRelativeScaleToggleRequested.IsBound())
					{
						MenuBuilder.AddMenuEntry(
							LOCTEXT("LoudnessScale_RelativeScale_Label", "Relative Scale"),
							LOCTEXT("LoudnessScale_RelativeScale_ToolTip", "Display loudness values relative to the loudness target"),
							FSlateIcon(),
							FUIAction(
								LoudnessScaleParams.OnRelativeScaleToggleRequested,
								FCanExecuteAction(),
								FIsActionChecked::CreateLambda([bRelativeScale = LoudnessScaleParams.bRelativeScale]() { return bRelativeScale.Get(); })
							),
							NAME_None,
							EUserInterfaceActionType::ToggleButton);
					}

					MenuBuilder.EndSection();

					MenuBuilder.BeginSection(NAME_None, LOCTEXT("ReferenceLevels", "Reference Levels"));

					if (LoudnessScaleParams.OnTargetValueCommitted.IsBound())
					{
						TSharedRef TargetWidget = SNew(SNumericEntryBox<int32>)
							.MinDesiredValueWidth(100.0f)
							.MinSliderValue(-60)
							.MaxSliderValue(30)
							.AllowSpin(true)
							.Value_Lambda([Target = LoudnessScaleParams.Target]() { return Target.Get(); })
							.OnValueChanged(LoudnessScaleParams.OnTargetValueChanged)
							.OnValueCommitted(LoudnessScaleParams.OnTargetValueCommitted);

						MenuBuilder.AddWidget(TargetWidget, LOCTEXT("ReferenceLevels_LoudnessTarget", "Loudness Target"), false, true, LOCTEXT("ReferenceLevels_LoudnessTarget_ToolTip", "Target loudness level in LUFS"));
					}

					if (LoudnessScaleParams.OnTruePeakLimitValueCommitted.IsBound())
					{
						TSharedRef TruePeakLimitWidget = SNew(SNumericEntryBox<float>)
							.MinDesiredValueWidth(100.0f)
							.MinSliderValue(-60.0f)
							.MaxSliderValue(30.0f)
							.MinFractionalDigits(1)
							.MaxFractionalDigits(1)
							.Delta(0.5f)
							.AllowSpin(true)
							.Value_Lambda([TruePeakLimit = LoudnessScaleParams.TruePeakLimit]() { return TruePeakLimit.Get(); })
							.OnValueChanged(LoudnessScaleParams.OnTruePeakLimitValueChanged)
							.OnValueCommitted(LoudnessScaleParams.OnTruePeakLimitValueCommitted);

						MenuBuilder.AddWidget(TruePeakLimitWidget, LOCTEXT("ReferenceLevels_TruePeakLimit", "True Peak Limit  "), false, true, LOCTEXT("ReferenceLevels_TruePeakLimit_ToolTip", "Maximum allowed true peak level in dBTP"));
					}

					if (LoudnessScaleParams.OnTargetColorChanged.IsBound())
					{
						MenuBuilder.AddWidget(MakeColorPickerRow(
							LoudnessScaleParams.TargetColor,
							LoudnessScaleParams.OnTargetColorChanged,
							LoudnessScaleParams.OnTargetColorPickerWindowClosed,
							LoudnessScaleParams.OnResetTargetColorRequested,
							LOCTEXT("ReferenceLevels_TargetColor", "Loudness Target / True Peak Limit Color"),
							LOCTEXT("ReferenceLevels_TargetColor_ToolTip", "Color used for the target and true peak limit indicators"),
							LOCTEXT("ReferenceLevels_ResetTargetColor_ToolTip", "Reset to default target color"),
							FLoudnessMeterWidgetView::GetDefaultTargetColor()), FText::GetEmpty());
					}

					MenuBuilder.EndSection();
				}

				{
					MenuBuilder.BeginSection(NAME_None, LOCTEXT("LayoutOptions", "Layout Options"));

					if (TimerPanelParams.OnVisibilityToggleRequested.IsBound())
					{
						MenuBuilder.AddMenuEntry(
							LOCTEXT("LayoutOptions_Time_Label", "Integrated Loudness Timer"),
							LOCTEXT("LayoutOptions_Time_ToolTip", "Display the elapsed time since the last loudness analysis reset"),
							FSlateIcon(),
							FUIAction(
								TimerPanelParams.OnVisibilityToggleRequested,
								FCanExecuteAction(),
								FIsActionChecked::CreateLambda([bIsVisible = TimerPanelParams.bIsVisible]() { return bIsVisible.Get(); })
							),
							NAME_None,
							EUserInterfaceActionType::ToggleButton);
					}

					for (const FLoudnessMetricRef& LoudnessMetricRef : LoudnessMetrics.Get())
					{
						MenuBuilder.AddSubMenu(
							LoudnessMetricRef->DisplayName,
							FText(),
							FNewMenuDelegate::CreateSPLambda(LoudnessMetricRef, [&LoudnessMetric = LoudnessMetricRef.Get()](FMenuBuilder& SubMenu)
								{
									AddToggleMenuEntry(SubMenu, LOCTEXT("LayoutOptions_LoudnessMetric_ShowValue", "Show Value"), LoudnessMetric.OnShowValueToggleRequested, LoudnessMetric.bShowValue);
									AddToggleMenuEntry(SubMenu, LOCTEXT("LayoutOptions_LoudnessMetric_ShowMeter", "Show Meter"), LoudnessMetric.OnShowMeterToggleRequested, LoudnessMetric.bShowMeter);

									if (LoudnessMetric.OnColorChanged.IsBound())
									{
										SubMenu.AddWidget(MakeColorPickerRow(
											LoudnessMetric.Color,
											LoudnessMetric.OnColorChanged,
											LoudnessMetric.OnColorPickerWindowClosed,
											LoudnessMetric.OnResetColorRequested,
											LOCTEXT("LayoutOptions_LoudnessMetric_Color", "Color"),
											FText(),
											LOCTEXT("LayoutOptions_LoudnessMetric_ResetColor_ToolTip", "Reset to default color"),
											FLoudnessMeterWidgetView::GetDefaultMeterColor(LoudnessMetric.Name)), FText::GetEmpty());
									}
								}));
					}

					MenuBuilder.EndSection();
				}

				return MenuBuilder.MakeWidget();
			};

		TSharedRef<SWidget> SettingsButton = SNew(SComboButton)
			.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
			.OnGetMenuContent_Lambda(MakeOptionsMenu)
			.HasDownArrow(false)
			.MenuPlacement(EMenuPlacement::MenuPlacement_ComboBoxRight)
			.ToolTipText(LOCTEXT("OptionsButtonDisplayName", "Options"))
			.ButtonContent()
			[
				// Cog icon
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(5.0f, 0.0f)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAudioWidgetsCoreStyle::Get().GetBrush("AudioWidgetsCoreStyle.Settings"))
					.DesiredSizeOverride(FVector2D(20.0f, 20.0f))
				]
			];

		// Ensure collection views have been synchronized at least once before displaying UI:
		ValuesCollectionView->RefreshVisibleItems();
		MetersCollectionView->RefreshVisibleItems();

		const auto GetTimerPanelVisibility = [bIsVisible = TimerPanelParams.bIsVisible]() { return (bIsVisible.Get()) ? EVisibility::Visible : EVisibility::Collapsed; };

		constexpr float MeterWidth = 79.0f;
		constexpr float MeterMinHeight = 160.0f;

		TSharedRef<SLoudnessMetricTileView> MeterTileView = SNew(SLoudnessMetricTileView,
				FOnMakeTileInnerContent::CreateStatic(&MakeMeterTileInnerContent, LoudnessScaleParams),
				SLoudnessMetricTileView::FOnSetShowItem::CreateStatic(&SetShowLoudnessMetricMeter),
				SLoudnessMetricTileView::FOnReorderItems::CreateSP(MetersCollectionView, &FLoudnessMetricCollectionView::DraggedItemReorder))
			.OnGenerateTile_Static(&MakeMeterValueTileWidget)
			.ListItemsSource(MetersCollectionView->GetVisibleItems())
			.ItemWidth(MeterWidth)
			.ItemAlignment(EListItemAlignment::CenterAligned)
			.SelectionMode(ESelectionMode::Single)
			.ScrollbarVisibility(EVisibility::Collapsed)
			.OnMouseButtonClick_Lambda([](FLoudnessMetricRef InItem)
			{
				if (InItem->bHoldMaxForMeter.Get())
				{
					InItem->OnClicked.ExecuteIfBound();
				}
			});

		MeterTileView->SetItemHeight(TAttribute<float>::CreateLambda([&TileView = *MeterTileView]()
			{
				const int32 NumItems = TileView.GetItems().Num();
				const FVector2f PanelSize = TileView.GetTickSpaceGeometry().GetLocalSize();
				const int32 NumItemsPerLine = FMath::Max(1, FMath::FloorToInt(PanelSize.X / MeterWidth));
				const int32 NumLines = (NumItems + NumItemsPerLine - 1) / NumItemsPerLine;
				const int32 HeightDivisor = FMath::Max(1, NumLines);
				return FMath::Max(PanelSize.Y / HeightDivisor, MeterMinHeight);
			}));

		const TAttribute<EVisibility> GetNumericValuesPanelVisibility = TAttribute<EVisibility>::CreateSPLambda(ValuesCollectionView,
			[&InValuesCollectionView = ValuesCollectionView.Get(), MeterTileViewWeak = MeterTileView.ToWeakPtr()]()
			{
				if (TSharedPtr<SLoudnessMetricTileView> PinnedMeterTileView = MeterTileViewWeak.Pin())
				{
					if (PinnedMeterTileView->HasDragDropOperation())
					{
						// Never hide the numeric values panel if a meter is being dragged so that the meter can be dropped onto it.
						return EVisibility::Visible;
					}
				}
				return (InValuesCollectionView.HasVisibleItems()) ? EVisibility::Visible : EVisibility::Collapsed;
			});

		return SNew(SBorder)
			.Padding(0.0f)
			.BorderImage(&BackgroundBrush)
			[
				SNew(SVerticalBox)
					.Clipping(EWidgetClipping::ClipToBounds)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBox)
						.Padding(0.0f, 10.0f)
						[
							// Integrated loudness timer
							SNew(SOverlay)
							+ SOverlay::Slot()
							.HAlign(EHorizontalAlignment::HAlign_Center)
							.VAlign(EVerticalAlignment::VAlign_Center)
							[
								// Timer
								SNew(SHorizontalBox)
								.Visibility_Lambda(GetTimerPanelVisibility)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SBorder)
									.Padding(8.0f, 4.0f)
									[
										SNew(STextBlock)
										.Font(FStyleDefaults::GetFontInfo(10))
										.Text_Lambda([AnalysisTime = TimerPanelParams.AnalysisTime]() { return FText::AsTimespan(AnalysisTime.Get()); })
										.ToolTipText(LOCTEXT("IntegratedLoudnessTimerToolTip", "Elapsed analysis time for integrated (long-term) loudness.\nIntegrated loudness averages over the entire measurement window.\nReset to start a new measurement."))
									]
								]
								// Reset button
								+ SHorizontalBox::Slot()
								.AutoWidth()
								[
									SNew(SButton)
									.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
									.Visibility_Lambda([bIsEnabled = TimerPanelParams.bIsResetButtonEnabled]() { return bIsEnabled.Get() ? EVisibility::Visible : EVisibility::Hidden; })
									.OnClicked(TimerPanelParams.OnResetButtonClicked)
									[
										SNew(SImage)
										.Image(FAudioWidgetsCoreStyle::Get().GetBrush(ResetBrushName))
										.ColorAndOpacity(FSlateColor::UseForeground())
										.ToolTipText(LOCTEXT("ResetButtonToolTip", "Reset loudness analyzers"))
									]
								]
							]
							// Settings button
							+ SOverlay::Slot()
							.HAlign(EHorizontalAlignment::HAlign_Right)
							.VAlign(EVerticalAlignment::VAlign_Center)
							[
								SettingsButton
							]
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(8.0f, 2.0f)
					[
						SNew(SSeparator)
							.SeparatorImage(FAppStyle::Get().GetBrush("WhiteBrush"))
							.Orientation(Orient_Horizontal)
							.Thickness(1.0f)
							.ColorAndOpacity(FLinearColor::White.CopyWithNewOpacity(0.25f))
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(8.0f, 2.0f)
					[
						// Target Loudness and True Peak Limit readout
						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0, 0, 0, 6)
						.HAlign(HAlign_Center)
						[
							// Target Loudness readout
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(STextBlock)
								.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
								.Text(LOCTEXT("TargetLoudnessLabel", "  Target Loudness: "))
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(STextBlock)
								.Font(FStyleDefaults::GetFontInfo(10))
								.Text_Lambda([this]()
								{
									return FText::Format(
										LOCTEXT("TargetLoudnessValue", "{0} LUFS"),
										FText::AsNumber(LoudnessScaleParams.Target.Get())
									);
								})
							]
						]
						+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Center)
						[
							// True Peak Limit readout
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(STextBlock)
								.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
								.Text(LOCTEXT("TruePeakLabel", "True Peak Limit: "))
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(STextBlock)
								.Font(FStyleDefaults::GetFontInfo(10))
								.Text_Lambda([this]()
								{
									return FText::Format(
										LOCTEXT("TruePeakValue", "{0} dB"),
										FText::AsNumber(LoudnessScaleParams.TruePeakLimit.Get(), &NumberFormattingOptions)
									);
								})
							]
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(8.0f, 2.0f)
					[
						SNew(SSeparator)
							.SeparatorImage(FAppStyle::Get().GetBrush("WhiteBrush"))
							.Orientation(Orient_Horizontal)
							.Thickness(1.0f)
							.ColorAndOpacity(FLinearColor::White.CopyWithNewOpacity(0.25f))
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SLoudnessMetricTileView,
								FOnMakeTileInnerContent::CreateStatic(&MakeValueTileInnerContent, LoudnessScaleParams),
								SLoudnessMetricTileView::FOnSetShowItem::CreateStatic(&SetShowLoudnessMetricValue),
								SLoudnessMetricTileView::FOnReorderItems::CreateSP(ValuesCollectionView, &FLoudnessMetricCollectionView::DraggedItemReorder))
							.Visibility(GetNumericValuesPanelVisibility)
							.OnGenerateTile_Static(&MakeNumericValueTileWidget)
							.ListItemsSource(ValuesCollectionView->GetVisibleItems())
							.ItemHeight(75.0f)
							.ItemWidth(68.0f)
							.ItemAlignment(EListItemAlignment::CenterAligned)
							.SelectionMode(ESelectionMode::Single)
							.OnMouseButtonClick_Lambda([](FLoudnessMetricRef InItem)
							{
								if (InItem->bHoldMaxForValue.Get())
								{
									InItem->OnClicked.ExecuteIfBound();
								}
							})
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(8.0f, 2.0f)
					[
						SNew(SSeparator)
							.Visibility(GetNumericValuesPanelVisibility)
							.SeparatorImage(FAppStyle::Get().GetBrush("WhiteBrush"))
							.Orientation(Orient_Horizontal)
							.Thickness(1.0f)
							.ColorAndOpacity(FLinearColor::White.CopyWithNewOpacity(0.25f))
					]
					+ SVerticalBox::Slot()
					.FillHeight(1.0f)
					[
						MeterTileView
					]
			];
	}

	void FLoudnessMeterWidgetView::AddLoudnessMetric(const FLoudnessMetric& LoudnessMetric)
	{
		FLoudnessMetricRef LoudnessMetricRef = MakeShared<FLoudnessMetric>(LoudnessMetric);
		LoudnessMetrics->Add(LoudnessMetricRef);
	}

	void FLoudnessMeterWidgetView::RefreshVisibleLoudnessMetrics()
	{
		ValuesCollectionView->RefreshVisibleItems();
		MetersCollectionView->RefreshVisibleItems();
	}

	FLoudnessMeterWidgetView::FLoudnessMetricCollectionView::FLoudnessMetricCollectionView(TSharedRef<FLoudnessMetricRefArray> InLoudnessMetrics, TFunction<bool(const FLoudnessMetricRef&)> InVisibilityPredicate)
		: LoudnessMetrics(InLoudnessMetrics)
		, VisibilityPredicate(InVisibilityPredicate)
		, VisibleItems(MakeShared<FLoudnessMetricRefArray>())
	{
	}

	void FLoudnessMeterWidgetView::FLoudnessMetricCollectionView::RefreshVisibleItems()
	{
		const auto GetDesiredVisibleItems = [this]() -> TArray<FLoudnessMetricRef>
			{
				const int64 PermutationIndex = DisplayOrderParams.PermutationIndex.Get();
				const int32 NumLoudnessMetrics = LoudnessMetrics->Num();
				const TArray<uint8> PermutationArray = GetPermutationArray(PermutationIndex, NumLoudnessMetrics);

				TArray<FLoudnessMetricRef> DesiredVisibleItems;
				DesiredVisibleItems.Reserve(NumLoudnessMetrics);
				for (uint8 LoudnessMetricIndex : PermutationArray)
				{
					const FLoudnessMetricRef& LoudnessMetricRef = (*LoudnessMetrics)[LoudnessMetricIndex];
					if (VisibilityPredicate(LoudnessMetricRef))
					{
						DesiredVisibleItems.Add(LoudnessMetricRef);
					}
				}
				return DesiredVisibleItems;
			};

		const TArray<FLoudnessMetricRef> DesiredVisibleItems = GetDesiredVisibleItems();

		// Sync VisibleItems to match DesiredVisibleItems:

		const int32 NumDesired = DesiredVisibleItems.Num();
		if (VisibleItems->Num() > NumDesired)
		{
			VisibleItems->RemoveAt(NumDesired, VisibleItems->Num() - NumDesired);
		}

		for (int32 Index = 0; Index < NumDesired; Index++)
		{
			const FLoudnessMetricRef& LoudnessMetricRef = DesiredVisibleItems[Index];

			if (Index >= VisibleItems->Num())
			{
				VisibleItems->Add(LoudnessMetricRef);
			}
			else if ((*VisibleItems)[Index] != LoudnessMetricRef)
			{
				VisibleItems->RemoveAt(Index);
				VisibleItems->EmplaceAt(Index, LoudnessMetricRef);
			}
		}
	}

	void FLoudnessMeterWidgetView::FLoudnessMetricCollectionView::DraggedItemReorder(const FLoudnessMetricRef& ItemDroppedOnto, bool bInsertAfter, const FLoudnessMetricRef& DragDropItem)
	{
		// Find the given items in the array of all loudness metrics (these are unchanging indexes that uniquely identify the loudness metrics):
		const int32 DroppedOntoLoudnessMetricIndex = LoudnessMetrics->Find(ItemDroppedOnto);
		const int32 DragDropLoudnessMetricIndex = LoudnessMetrics->Find(DragDropItem);
		if (ensure(DroppedOntoLoudnessMetricIndex != INDEX_NONE && DragDropLoudnessMetricIndex != INDEX_NONE && DragDropLoudnessMetricIndex != DroppedOntoLoudnessMetricIndex))
		{
			// We get the current PermutationArray and then modify it as required.
			const uint64 CurrentPermutationIndex = DisplayOrderParams.PermutationIndex.Get();
			TArray<uint8> PermutationArray = GetPermutationArray(CurrentPermutationIndex, LoudnessMetrics->Num());

			// Remove dragged item from current position:
			PermutationArray.Remove(DragDropLoudnessMetricIndex);

			const int32 DroppedOntoPosition = PermutationArray.Find(DroppedOntoLoudnessMetricIndex);
			if (ensure(DroppedOntoPosition != INDEX_NONE))
			{
				// Place dragged item at new position:
				const int32 InsertPosition = bInsertAfter ? DroppedOntoPosition + 1 : DroppedOntoPosition;
				PermutationArray.EmplaceAt(InsertPosition, DragDropLoudnessMetricIndex);

				// Get the new permutation index with the reordering applied:
				const uint64 NewPermutationIndex = GetPermutationIndex(PermutationArray);
				if (NewPermutationIndex != CurrentPermutationIndex && DisplayOrderParams.OnPermutationChanged.IsBound())
				{
					// Raise request to change ordering with new PermutationIndex:
					DisplayOrderParams.OnPermutationChanged.Execute(NewPermutationIndex);

					// Proactive refresh:
					RefreshVisibleItems();
				}
			}
		}
	}

	uint64 FLoudnessMeterWidgetView::FLoudnessMetricCollectionView::GetPermutationIndex(TArrayView<uint8> PermutationArray)
	{
		using namespace Permutations;

		const uint64 NumPermuations = Factorial(PermutationArray.Num());
		const uint64 MaxPermutationIndex = NumPermuations - 1;

		// We alter the returned index so that the permutation (0,1,2,...) is represented as PermutationIndex zero:
		return MaxPermutationIndex - AnalyzePermutation(PermutationArray);
	}

	TArray<uint8> FLoudnessMeterWidgetView::FLoudnessMetricCollectionView::GetPermutationArray(uint64 PermutationIndex, uint8 NumValues)
	{
		using namespace Permutations;

		const uint64 NumPermuations = Factorial(NumValues);
		const uint64 MaxPermutationIndex = NumPermuations - 1;

		// Ensure valid bounds:
		PermutationIndex = PermutationIndex % NumPermuations;

		// We use a reversed index so that a given PermutationIndex of zero will result in the permutation (0,1,2,...):
		return GeneratePermutation(MaxPermutationIndex - PermutationIndex, NumValues);
	}
} // namespace AudioWidgetsCore

#undef LOCTEXT_NAMESPACE
