// Copyright Epic Games, Inc. All Rights Reserved.

#include "Permutations.h"

#include "Algo/MaxElement.h"

namespace AudioWidgetsCore::Permutations
{
	uint64 AnalyzePermutation(TArrayView<uint8> PermutationArray)
	{
		check(PermutationArray.Num() >= 0);
		check(PermutationArray.Num() <= MaxFactorial);

		// See Knuth, Vol. 2, 3rd ed., Section 3.3.2, pp. 65-66, Algorithm P (Analyze a permutation).

		uint64 F = 0;
		for (uint8 R = PermutationArray.Num(); R > 1; R--)
		{
			PermutationArray.LeftInline(R);
			const uint64 S = Algo::MaxElement(PermutationArray) - PermutationArray.GetData();
			F = R * F + S;
			Swap(PermutationArray[R - 1], PermutationArray[S]);
		}
		return F;
	}

	TArray<uint8> GeneratePermutation(uint64 PermutationID, uint8 NumValues)
	{
		check(NumValues <= MaxFactorial);
		check(PermutationID < Factorial(NumValues));

		// Algorithm P, reverse

		TArray<uint8> PermutationArray;
		PermutationArray.Reserve(NumValues);
		while (PermutationArray.Num() < NumValues)
		{
			PermutationArray.Add(PermutationArray.Num());
		}

		uint64 F = PermutationID;
		for (uint8 R = 2; R <= NumValues; R++)
		{
			const uint64 S = F % R;
			F /= R;
			PermutationArray.Swap(R - 1, S);
		}
		return PermutationArray;
	}

} // namespace AudioWidgetsCore::Permutations
