// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"

namespace AudioWidgetsCore::Permutations
{
	constexpr uint8 MaxFactorial = 20;

	/**
	 * Compute exact factorial. Only valid for small non-negative integers.
	 */
	constexpr uint64 Factorial(uint8 N)
	{
		if (!ensure(N <= MaxFactorial))
		{
			return UINT64_MAX;
		}

		uint64 F = 1;
		for (uint8 R = N; R > 1; R--)
		{
			F *= R;
		}
		return F;
	}

	/**
	 * Get an integer that uniquely represents the given permutation.
	 * 
	 * @param PermutationArray The elements of the array should be distinct values in the interval [0, N-1] where N == PermutationArray.Num(). On return, the array will have been sorted.
	 * @return A value in the interval [0, N!-1] where N == PermutationArray.Num(). If N > MaxFactorial, the result is undefined.
	 */
	uint64 AnalyzePermutation(TArrayView<uint8> PermutationArray);

	/**
	 * Create the permutation that is uniquely represented by the given integer.
	 * 
	 * @param PermutationID A value in the interval [0, N!-1] where N == NumValues.
	 * @param NumValues The desired number of values in the permutation, must be no greater than MaxFactorial.
	 * @return An array of distinct values with length == NumValues. If NumValues > MaxFactorial, the result is undefined.
	 */
	TArray<uint8> GeneratePermutation(uint64 PermutationID, uint8 NumValues);

} // namespace AudioWidgetsCore::Permutations
