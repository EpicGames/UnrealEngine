// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAudioMeterWidget.h"

#include "Framework/Application/SlateApplication.h"
#include "Fonts/FontMeasure.h"
#include "Rendering/SlateRenderer.h"

void SAudioMeterWidget::Construct(const SAudioMeterWidget::FArguments& InArgs)
{
	check(InArgs._Style);

	Orientation = InArgs._Orientation;

	BackgroundColor = InArgs._BackgroundColor;
	MeterBackgroundColor = InArgs._MeterBackgroundColor;
	MeterValueColor = InArgs._MeterValueColor;
	MeterPeakColor = InArgs._MeterPeakColor;
	MeterScaleColor = InArgs._MeterScaleColor;
	MeterScaleLabelColor = InArgs._MeterScaleLabelColor;
	MeterClippingColor = InArgs._MeterClippingColor;

	Style = InArgs._Style;

	MeterChannelInfoAttribute = InArgs._MeterChannelInfo;
}

int32 SAudioMeterWidget::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// we draw the meter like a horizontal meter regardless of the orientation, and apply a render transform to make it display correctly.
	// However, the AllottedGeometry is computed as it will be rendered, so we have to use the "horizontal orientation" when doing drawing computations.
	const float AllottedWidth  = Orientation == Orient_Horizontal ? AllottedGeometry.GetLocalSize().X : AllottedGeometry.GetLocalSize().Y;
	const float AllottedHeight = Orientation == Orient_Horizontal ? AllottedGeometry.GetLocalSize().Y : AllottedGeometry.GetLocalSize().X;

	FGeometry MeterGeometry = AllottedGeometry;

	// rotate the meter 90deg if it's vertical. The 0 side goes on the bottom, the 1 side on the top.
	if (Orientation == Orient_Vertical)
	{
		// Do this by translating along -X by the width of the geometry, then rotating 90 degrees CCW (left-hand coords)
		const FSlateRenderTransform SlateRenderTransform = TransformCast<FSlateRenderTransform>(Concatenate(Inverse(FVector2D(AllottedWidth, 0)), FQuat2D(FMath::DegreesToRadians(-90.0f))));

		// create a child geometry matching this one, but with the render transform.
		MeterGeometry = AllottedGeometry.MakeChild(FVector2D(AllottedWidth, AllottedHeight), FSlateLayoutTransform(), SlateRenderTransform, FVector2D::ZeroVector);
	}
 
 	// We clamp to make sure that the meter cannot go out of the meter Length.
 	const int32 DecibelsPerHash = Style->DecibelsPerHash;
 
 	int32 MinValueDb = FMath::Min(static_cast<int32>(Style->ValueRangeDb.X), static_cast<int32>(Style->ValueRangeDb.Y));
 	int32 MaxValueDb = FMath::Max(static_cast<int32>(Style->ValueRangeDb.X), static_cast<int32>(Style->ValueRangeDb.Y));
 
 	const float ValueDeltaDb = MaxValueDb - MinValueDb;

	// Draw the background image
	{
		const FVector2D BackgroundImageTopLeft(0.0f, 0.0f);
		const FVector2D BackgroundImageSize(AllottedWidth, AllottedHeight);

		// Draw background image
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			MeterGeometry.ToPaintGeometry(BackgroundImageSize, FSlateLayoutTransform(BackgroundImageTopLeft)),
			&Style->MeterBackgroundImage,
			ESlateDrawEffect::None,
			Style->BackgroundImage.GetTint(InWidgetStyle) * BackgroundColor.Get().GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
		);
	}
 	++LayerId;

	float ScaleOffset = 0.0f;
	if (Style->bShowScale && Style->bScaleSide)
	{
		 ScaleOffset = GetScaleHeight();
	}

	const TArray<FAudioMeterChannelInfo> ChannelInfos = MeterChannelInfoAttribute.Get();
	const int32 NumChannels = ChannelInfos.Num();

	// Draw the meter backgrounds
	for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
	{		
		const FVector2D MeterBackgroundTopLeft(Style->MeterPadding.X, Style->MeterPadding.Y + ScaleOffset + ChannelIndex * (Style->MeterSize.Y + Style->MeterPadding.Y));
		const FVector2D MeterBackgroundSize(AllottedWidth - 2.0f * Style->MeterPadding.X, Style->MeterSize.Y);

 		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			MeterGeometry.ToPaintGeometry(MeterBackgroundSize, FSlateLayoutTransform(MeterBackgroundTopLeft)),
			&Style->MeterBackgroundImage,
			ESlateDrawEffect::None,
			Style->MeterBackgroundImage.GetTint(InWidgetStyle) * MeterBackgroundColor.Get().GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
		);
	}
	++LayerId;

	if (ValueDeltaDb == 0.0f)
	{
		// ValueRangeDb is zero, don't draw any values.
		return LayerId;
	}

	// Draw the meter value
	for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
	{
		const float ChannelMeterValueDb = ChannelInfos[ChannelIndex].MeterValue;
		const float ChannelMeterValue = FMath::Clamp((ChannelMeterValueDb - MinValueDb) / ValueDeltaDb, 0.0f, 1.0f);

		const float ChannelMeterValueBarStartDb = ChannelInfos[ChannelIndex].MeterValueBarStart.Get(MinValueDb);
		const float ChannelMeterValueBarStart = FMath::Clamp((ChannelMeterValueBarStartDb - MinValueDb) / ValueDeltaDb, 0.0f, 1.0f);

		const FFloatInterval MeterValueBarInterval(
			FMath::Min(ChannelMeterValueBarStart, ChannelMeterValue),
			FMath::Max(ChannelMeterValueBarStart, ChannelMeterValue));

		const float AllottedWidthMinusPadding = AllottedWidth - 2.0f * Style->MeterPadding.X;

		const FVector2D MeterValueTopLeft(Style->MeterPadding.X + MeterValueBarInterval.Min * AllottedWidthMinusPadding,
									Style->MeterPadding.Y + ScaleOffset + Style->MeterValuePadding + ChannelIndex * (Style->MeterSize.Y + Style->MeterPadding.Y));
		
		const FVector2D MeterValueSize(MeterValueBarInterval.Size() * AllottedWidthMinusPadding,
								 Style->MeterSize.Y - 2.0f * Style->MeterValuePadding);
 
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			MeterGeometry.ToPaintGeometry(MeterValueSize, FSlateLayoutTransform(MeterValueTopLeft)),
			&Style->MeterValueImage,
			ESlateDrawEffect::None,
			Style->MeterValueImage.GetTint(InWidgetStyle) * MeterValueColor.Get().GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
		);
	}
	++LayerId;

	// Draw the meter peak value
	for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
	{	
		const float ChannelPeakValueDb = ChannelInfos[ChannelIndex].PeakValue;
		const float ChannelPeakValue = FMath::Clamp((ChannelPeakValueDb - MinValueDb) / ValueDeltaDb, 0.0f, 1.0f);

		const float PeakPixel = ChannelPeakValue * (AllottedWidth - 2.0f * Style->MeterPadding.X);

		FVector2D MeterPeakValueTopLeft(Style->MeterPadding.X + PeakPixel - 0.5f * Style->PeakValueWidth, 
										Style->MeterPadding.Y + ScaleOffset + Style->MeterValuePadding + ChannelIndex * (Style->MeterSize.Y + Style->MeterPadding.Y));
		
		FVector2D MeterPeakValueSize(Style->PeakValueWidth, 
									 Style->MeterSize.Y - 2.0f * Style->MeterValuePadding);

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			MeterGeometry.ToPaintGeometry(MeterPeakValueSize, FSlateLayoutTransform(MeterPeakValueTopLeft)),
			&Style->MeterPeakImage,
			ESlateDrawEffect::None,
			Style->MeterPeakImage.GetTint(InWidgetStyle) * MeterPeakColor.Get().GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
		);
	}
	++LayerId;

	// Draw the meter clipping value
	const FLinearColor ClippingColor = MeterClippingColor.Get().GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint();
	if (ClippingColor.A > 0.0f)
	{
		for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
		{
			const float ChannelClippingValueDb = ChannelInfos[ChannelIndex].ClippingValue;
			const float ChannelClippingValue   = FMath::Clamp((ChannelClippingValueDb - MinValueDb) / ValueDeltaDb, 0.0f, 1.0f);

			const float ClippingPixel = ChannelClippingValue * (AllottedWidth - 2.0f * Style->MeterPadding.X);

			const FVector2D ClippingValueTopLeft(Style->MeterPadding.X + ClippingPixel - 0.5f * Style->ClippingValueWidth,
												 Style->MeterPadding.Y + ScaleOffset + Style->MeterValuePadding + ChannelIndex * (Style->MeterSize.Y + Style->MeterPadding.Y));

			const FVector2D ClippingValueSize(Style->ClippingValueWidth,
											  Style->MeterSize.Y - 2.0f * Style->MeterValuePadding);

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				MeterGeometry.ToPaintGeometry(ClippingValueSize, FSlateLayoutTransform(ClippingValueTopLeft)),
				&Style->MeterClippingImage,
				ESlateDrawEffect::None,
				Style->MeterClippingImage.GetTint(InWidgetStyle) * ClippingColor
			);
		}
	}
	++LayerId;

	// Draw the scale hash
	if (Style->bShowScale)
	{
		const float ScaleHashHalfWidth = 0.5f * Style->ScaleHashWidth;
		const FVector2D HashSize(Style->ScaleHashWidth, Style->ScaleHashHeight);

		const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

		// Measure the min value label size so we can right-justify if needed
		const FVector2D MinValueLabelSize = FontMeasureService->Measure(FString::FromInt(MinValueDb), Style->Font);

		// Get the size of the negative sign to use to offset the label text in horizontal mode
		FVector2D NegativeSignSize = FVector2D::ZeroVector;
		if (Orientation == Orient_Horizontal)
		{
			NegativeSignSize = FontMeasureService->Measure(TEXT("-"), Style->Font);
		}

		const int32 ValueDelta = MaxValueDb - MinValueDb;

		// Round down MaxValueDb to find the first hash mark to draw:
		int32 CurrentHashValue = DecibelsPerHash * FMath::FloorToInt32(float(MaxValueDb) / float(DecibelsPerHash));
		while (CurrentHashValue >= MinValueDb)
		{
			// Get the fractional value for hash mark
			const float CurrentHashMeterValuePercent = FMath::Clamp((static_cast<float>(CurrentHashValue) - static_cast<float>(MinValueDb)) / ValueDelta, 0.0f, 1.0f);

			// Draw hash
			const float HashPixelCenter = CurrentHashMeterValuePercent * (AllottedWidth - 2.0f * Style->MeterPadding.X);

			FVector2D HashTopLeft;
			HashTopLeft.X = Style->MeterPadding.X + HashPixelCenter - ScaleHashHalfWidth;
			HashTopLeft.Y = Style->bScaleSide 
							? Style->MeterPadding.Y + ScaleOffset - Style->ScaleHashOffset - Style->ScaleHashHeight
							: (Style->MeterSize.Y + Style->MeterPadding.Y) * NumChannels + Style->ScaleHashOffset;


			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				MeterGeometry.ToPaintGeometry(HashSize, FSlateLayoutTransform(HashTopLeft)),
				&Style->MeterPeakImage,
				ESlateDrawEffect::None,
				MeterScaleColor.Get().GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
			);
		
			const FString LabelString = FString::FromInt(CurrentHashValue);
			bool bIsNegative = (CurrentHashValue < 0);

			FVector2D LabelSize; 
			if (Orientation == Orient_Horizontal && bIsNegative)
			{
				// We want to center the text on just the positive portion of the number
				FString LabelStringPositive = FString::FromInt(FMath::Abs(CurrentHashValue));
				LabelSize = FontMeasureService->Measure(LabelStringPositive, Style->Font);
			}
			else
			{
				LabelSize = FontMeasureService->Measure(LabelString, Style->Font);
			}
			
			const FText LabelText = FText::FromString(LabelString);

			if (Orientation == Orient_Vertical)
			{
				FVector2D LabelTopLeft;
				
				LabelTopLeft.X = Style->MeterPadding.X + HashPixelCenter + 0.5f * LabelSize.Y;
				LabelTopLeft.Y = Style->bScaleSide 
								 ? Style->MeterPadding.Y - 2.0f + (MinValueLabelSize.X - LabelSize.X)
								 : (Style->MeterSize.Y + Style->MeterPadding.Y) * NumChannels + Style->ScaleHashOffset + Style->ScaleHashHeight + 2.0f;
				
				// Undo the rotation for vertical right before we do the rendering of the scale value 
				const FSlateRenderTransform RotationTransform = FSlateRenderTransform(FQuat2D(FMath::DegreesToRadians(90.0f)));

				const FSlateLayoutTransform ChildLayoutTransform(1.0f, TransformPoint(1.0f, LabelTopLeft));

				const FGeometry TextGeometry = MeterGeometry.MakeChild(
					LabelSize,
					ChildLayoutTransform,
					RotationTransform,
					FVector2D(0.0f, 0.0f)
				);

				FSlateDrawElement::MakeText(
					OutDrawElements,
					LayerId,
					TextGeometry.ToOffsetPaintGeometry(FVector2D(0.0f, 0.0f)),
					LabelText,
					Style->Font,
					ESlateDrawEffect::None,
					MeterScaleLabelColor.Get().GetColor(InWidgetStyle)* InWidgetStyle.GetColorAndOpacityTint()
				);
			}
			else
			{
				FVector2D LabelTopLeft;

				// Center it based off the positive value width, then subtract from there the size of the negative sign
				LabelTopLeft.X = Style->MeterPadding.X + HashPixelCenter - 0.5f * LabelSize.X;
				if (bIsNegative)
				{
					LabelTopLeft.X -= NegativeSignSize.X;
				}

				LabelTopLeft.Y = Style->bScaleSide 
								 ? Style->MeterPadding.Y + ScaleOffset - LabelSize.Y - Style->ScaleHashOffset - Style->ScaleHashHeight
								 : (Style->MeterSize.Y + Style->MeterPadding.Y) * NumChannels + Style->ScaleHashOffset + Style->ScaleHashHeight + 2.0f;

				// Draw text label
				FSlateDrawElement::MakeText(
					OutDrawElements,
					LayerId,
					MeterGeometry.ToOffsetPaintGeometry(LabelTopLeft),
					LabelText,
					Style->Font,
					ESlateDrawEffect::None,
					MeterScaleLabelColor.Get().GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
				);

			}

			CurrentHashValue -= Style->DecibelsPerHash;
		}
	}

	return LayerId;
}

float SAudioMeterWidget::GetScaleHeight() const
{
	const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	FVector2D LabelSize = FontMeasureService->Measure(FString::FromInt(-60), Style->Font);

	float ScaleHeight = Style->ScaleHashHeight + Style->ScaleHashOffset;

	if (Orientation == Orient_Vertical)
	{
		return ScaleHeight + LabelSize.X;
	}
	return ScaleHeight + LabelSize.Y;
}

FVector2D SAudioMeterWidget::ComputeDesiredSize(float) const
{
	static const FVector2D SAudioMeterDesiredSize(50.0f, 50.0f);

	if (Style == nullptr)
	{
		return SAudioMeterDesiredSize;
	}

	TArray<FAudioMeterChannelInfo> ChannelInfo = MeterChannelInfoAttribute.Get();
	const int32 NumChannels = FMath::Max(ChannelInfo.Num(), 1);

	FVector2D Size = FVector2D(Style->MeterSize.X, (Style->MeterSize.Y + Style->MeterPadding.Y) * NumChannels);

	// Add the end padding
	Size.X += Style->MeterPadding.X;
	Size.Y += Style->MeterPadding.Y;

	// Add the width for the scale if it's been set to show
	if (Style->bShowScale)
	{
		Size.Y += GetScaleHeight();
	}

	// The thickness is going to be the thickness plus the scale image width
	if (Orientation == Orient_Vertical)
	{
		return FVector2D(Size.Y, Size.X);
	}

	return Size; 
}

bool SAudioMeterWidget::ComputeVolatility() const
{
	return true; 
}

void SAudioMeterWidget::SetMeterChannelInfo(const TAttribute<TArray<FAudioMeterChannelInfo>>& InMeterChannelInfo)
{
	SetAttribute(MeterChannelInfoAttribute, InMeterChannelInfo, EInvalidateWidgetReason::Paint);
}

TArray<FAudioMeterChannelInfo> SAudioMeterWidget::GetMeterChannelInfo() const
{
	return MeterChannelInfoAttribute.Get();
}

void SAudioMeterWidget::SetOrientation(EOrientation InOrientation)
{
	if (Orientation != InOrientation)
	{
		Orientation = InOrientation;
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

void SAudioMeterWidget::SetBackgroundColor(FSlateColor InBackgroundColor)
{
	SetAttribute(BackgroundColor, TAttribute<FSlateColor>(InBackgroundColor), EInvalidateWidgetReason::Paint);
}

void SAudioMeterWidget::SetMeterBackgroundColor(FSlateColor InMeterBackgroundColor)
{
	SetAttribute(MeterBackgroundColor, TAttribute<FSlateColor>(InMeterBackgroundColor), EInvalidateWidgetReason::Paint);
}

void SAudioMeterWidget::SetMeterValueColor(FSlateColor InMeterValueColor)
{
	SetAttribute(MeterValueColor, TAttribute<FSlateColor>(InMeterValueColor), EInvalidateWidgetReason::Paint);
}

void SAudioMeterWidget::SetMeterPeakColor(FSlateColor InMeterPeakColor)
{
	SetAttribute(MeterPeakColor, TAttribute<FSlateColor>(InMeterPeakColor), EInvalidateWidgetReason::Paint);
}

void SAudioMeterWidget::SetMeterClippingColor(FSlateColor InMeterClippingColor)
{
	SetAttribute(MeterClippingColor, TAttribute<FSlateColor>(InMeterClippingColor), EInvalidateWidgetReason::Paint);
}

void SAudioMeterWidget::SetMeterScaleColor(FSlateColor InMeterScaleColor)
{
	SetAttribute(MeterScaleColor, TAttribute<FSlateColor>(InMeterScaleColor), EInvalidateWidgetReason::Paint);
}

void SAudioMeterWidget::SetMeterScaleLabelColor(FSlateColor InMeterScaleLabelColor)
{
	SetAttribute(MeterScaleLabelColor, TAttribute<FSlateColor>(InMeterScaleLabelColor), EInvalidateWidgetReason::Paint);
}
