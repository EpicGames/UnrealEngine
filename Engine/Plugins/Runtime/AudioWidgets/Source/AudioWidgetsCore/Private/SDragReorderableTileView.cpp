// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDragReorderableTileView.h"

#include "Styling/StyleColors.h"

namespace AudioWidgetsCore
{
	int32 DrawDropIndicator(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, EHorizontalAlignment HorizontalAlignment)
	{
		const FVector2f TileSize = AllottedGeometry.GetLocalSize();

		const float DropIndicatorTranslateX = [TileWidth = TileSize.X, HorizontalAlignment]()
			{
				switch (HorizontalAlignment)
				{
				default:
				case HAlign_Fill:
				case HAlign_Left:
					return 0.0f;
				case HAlign_Center:
					return 0.5f * TileWidth;
				case HAlign_Right:
					return TileWidth;
				}
			}();

		constexpr float DropIndicatorScale = 1.0f;
		const FVector2f DropIndicatorSize(1.0f, TileSize.Y);
		const FVector2f DropIndicatorTranslation(DropIndicatorTranslateX, 0.0f);
		const FGeometry DropIndicatorGeometry = AllottedGeometry.MakeChild(DropIndicatorSize, FSlateLayoutTransform(DropIndicatorScale, DropIndicatorTranslation));

		FSlateDrawElement::MakeBox
		(
			OutDrawElements,
			LayerId++,
			DropIndicatorGeometry.ToPaintGeometry(),
			FAppStyle::GetBrush("PropertyEditor.VerticalDottedLine"),
			ESlateDrawEffect::None,
			FStyleColors::Select.GetColor(InWidgetStyle) * InWidgetStyle.GetColorAndOpacityTint()
		);

		return LayerId;
	}
} // namespace AudioWidgetsCore
