// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Widgets/Views/STileView.h"

namespace AudioWidgetsCore
{
	/**
	 * Function to draw a dotted line representing where the current drag drop would place the dragged item.
	 */
	int32 DrawDropIndicator(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, EHorizontalAlignment HorizontalAlignment);

	template<typename DragDropOperationT>
	class SDragReorderableTileView;

	/**
	 * Abstract base class, represents an item that is being dragged from a given source tile view.
	 * Uses Curiously Recurring Template Pattern so that FDragDropOperation type checking operations can be used.
	 */
	template<typename ItemT, typename DragDropOperationT>
	class TItemDragDropOperation : public FDragDropOperation
	{
	public:
		using ItemType = ItemT;
		using STileViewType = SDragReorderableTileView<DragDropOperationT>;

		TItemDragDropOperation(TSharedRef<STileViewType> InSourceTileView)
			: WeakSourceTileView(InSourceTileView.ToWeakPtr())
		{
		}

		virtual const ItemType& GetItem() const = 0;

		TSharedPtr<STileViewType> GetSourceTileView() const { return WeakSourceTileView.Pin(); }

		virtual FCursorReply OnCursorQuery() override { return FCursorReply::Cursor(EMouseCursor::GrabHand); }

	private:
		const TWeakPtr<STileViewType> WeakSourceTileView;
	};

	template<typename T>
	concept ItemDragDropOperation =
		requires { typename T::ItemType; };// && std::derived_from<T, TItemDragDropOperation<typename T::ItemType, T>>; // Can uncomment when all platform stdlibs have <concepts> header.


	/**
	 * Abstract base class for TileView widget that allow its tiles to be dragged and dropped for reordering etc.
	 * Dropping of a dragged item onto the tile view is supported if the tile view is currently empty.
	 */
	template<typename DragDropOperationT>
	class SDragReorderableTileView : public STileView<typename DragDropOperationT::ItemType>
	{
	public:
		using Base = STileView<typename DragDropOperationT::ItemType>;
		using ItemType = typename DragDropOperationT::ItemType;

		DECLARE_DELEGATE_TwoParams(FOnSetShowItem, const ItemType&, bool);
		DECLARE_DELEGATE_ThreeParams(FOnReorderItems, const ItemType&/*ItemDroppedOnto*/, bool/*bInsertAfter*/, const ItemType&/*DragDropItem*/);

		void Construct(const typename Base::FArguments& InArgs, FOnSetShowItem InOnSetShowItem, FOnReorderItems InOnReorderItems)
		{
			Base::Construct(InArgs);

			OnSetShowItem = InOnSetShowItem;
			OnReorderItems = InOnReorderItems;
		}

		virtual TSharedPtr<DragDropOperationT> CreateDragDropOperation() = 0;

		void SetShowItem(const ItemType& Item, bool bShow)
		{
			OnSetShowItem.ExecuteIfBound(Item, bShow);
		}

		void ReorderItems(const ItemType& ItemDroppedOnto, bool bInsertAfter, const ItemType& DragDropItem)
		{
			ensure(ItemDroppedOnto != DragDropItem);
			OnReorderItems.ExecuteIfBound(ItemDroppedOnto, bInsertAfter, DragDropItem);
		}

		void HandleItemDrop(const ItemType& InItemDroppedOnto, bool bInInsertAfter, const ItemType& InDragDropItem, TSharedPtr<SDragReorderableTileView> InDragSourceTileView)
		{
			if (InDragSourceTileView.IsValid() && InDragSourceTileView.Get() != this && Base::GetItems().Find(InDragDropItem) != INDEX_NONE)
			{
				return;
			}

			if (InItemDroppedOnto != InDragDropItem)
			{
				Base::ClearSelection();
				ReorderItems(InItemDroppedOnto, bInInsertAfter, InDragDropItem);
			}

			if (InDragSourceTileView.IsValid() && InDragSourceTileView.Get() != this)
			{
				InDragSourceTileView->SetShowItem(InDragDropItem, false);
				SetShowItem(InDragDropItem, true);
			}
		}

		// Begin SWidget overrides.
		virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
		{
			if (TSharedPtr<DragDropOperationT> DragDropOp = DragDropEvent.GetOperationAs<DragDropOperationT>())
			{
				bPaintDropIndicator = Base::GetItems().IsEmpty();

				return FReply::Handled();
			}

			return FReply::Unhandled();
		}

		virtual void OnDragLeave(FDragDropEvent const& DragDropEvent) override
		{
			bPaintDropIndicator = false;
		}

		virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
		{
			bPaintDropIndicator = false;

			if (TSharedPtr<DragDropOperationT> DragDropOp = DragDropEvent.GetOperationAs<DragDropOperationT>())
			{
				TSharedPtr<SDragReorderableTileView> DragSourceTileView = DragDropOp->GetSourceTileView();
				if (!DragSourceTileView.IsValid())
				{
					return FReply::Unhandled();
				}

				const ItemType& DragDropItem = DragDropOp->GetItem();

				if (Base::GetItems().IsEmpty())
				{
					// Move into empty tile view:
					DragSourceTileView->SetShowItem(DragDropItem, false);
					SetShowItem(DragDropItem, true);

					return FReply::Handled();
				}

				// Non-empty: drop at the nearest boundary based on cursor position
				const TConstArrayView<ItemType> Items = Base::GetItems();
				const FVector2f LocalPos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());
				const FVector2f Size     = MyGeometry.GetLocalSize();
				const bool bDropAtEnd    = (LocalPos.X >= Size.X * 0.5f);

				HandleItemDrop(bDropAtEnd ? Items.Last() : Items[0], bDropAtEnd, DragDropItem, DragSourceTileView);

				return FReply::Handled();
			}
			return FReply::Unhandled();
		}

		virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
		{
			LayerId = Base::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);

			if (bPaintDropIndicator)
			{
				LayerId = DrawDropIndicator(AllottedGeometry, OutDrawElements, LayerId, InWidgetStyle, EHorizontalAlignment::HAlign_Center);
			}

			return LayerId;
		}
		// End SWidget overrides

	private:
		FOnSetShowItem OnSetShowItem;
		FOnReorderItems OnReorderItems;
		bool bPaintDropIndicator = false;
	};

	/**
	 * Base class for tiles displayed by SDragReorderableTileView.
	 */
	template<ItemDragDropOperation DragDropOperationT>
	class SDragReorderableTile : public STableRow<typename DragDropOperationT::ItemType>
	{
	public:
		using Base = STableRow<typename DragDropOperationT::ItemType>;
		using ItemType = typename DragDropOperationT::ItemType;
		using STileViewType = SDragReorderableTileView<DragDropOperationT>;

		bool IsBeingDragged() const { return WeakDragDropOperation.IsValid(); }

	protected:
		// Begin SWidget overrides.
		virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& PointerEvent) override
		{
			const FReply BaseReply = Base::OnDragDetected(MyGeometry, PointerEvent);
			if (!BaseReply.IsEventHandled() && PointerEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
			{
				if (TSharedPtr<STileViewType> OwnerTileView = GetPinnedTileView())
				{
					if (TSharedPtr<FDragDropOperation> DragDropOperation = OwnerTileView->CreateDragDropOperation())
					{
						// Hold a weak pointer to maintain awareness of whether this tile is currently being dragged:
						WeakDragDropOperation = DragDropOperation;

						return FReply::Handled().BeginDragDrop(DragDropOperation.ToSharedRef());
					}
				}
			}
			return BaseReply;
		}

		virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
		{
			// Setting STableRow::ItemDropZone TOptional will enable painting the drop indicator:
			Base::SetItemDropZone(CanAcceptDrop(MyGeometry, DragDropEvent));

			return FReply::Handled();
		}

		virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override
		{
			// Clear ItemDropZone to no longer paint the drop indicator:
			Base::SetItemDropZone(TOptional<EItemDropZone>());

			if (TOptional<EItemDropZone> AcceptableDropZone = CanAcceptDrop(MyGeometry, DragDropEvent))
			{
				return AcceptDrop(DragDropEvent, *AcceptableDropZone);
			}

			return FReply::Unhandled();
		}
		// End SWidget overrides

		// Begin STableRow overrides.
		virtual int32 OnPaintDropIndicator(EItemDropZone InItemDropZone, const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override
		{
			const EHorizontalAlignment HorizontalAlignment = (InItemDropZone == EItemDropZone::BelowItem) ? EHorizontalAlignment::HAlign_Right : EHorizontalAlignment::HAlign_Left;
			return DrawDropIndicator(AllottedGeometry, OutDrawElements, LayerId, InWidgetStyle, HorizontalAlignment);
		}
		// End STableRow overrides.

	private:
		TOptional<EItemDropZone> CanAcceptDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
		{
			if (TSharedPtr<DragDropOperationT> DragDropOp = DragDropEvent.GetOperationAs<DragDropOperationT>())
			{
				if (TSharedPtr<STileViewType> OwnerTileView = GetPinnedTileView())
				{
					const EItemDropZone ItemHoverZone = GetItemHoverZone(MyGeometry, DragDropEvent);
					if (IsValidDrop(OwnerTileView.ToSharedRef(), ItemHoverZone, DragDropOp.ToSharedRef()))
					{
						return ItemHoverZone;
					}
				}
			}
			return NullOpt;
		}

		bool IsValidDrop(const TSharedRef<STileViewType>& OwnerTileView, EItemDropZone ItemHoverZone, TSharedRef<DragDropOperationT> DragDropOp) const
		{
			const auto& DragDropItem = DragDropOp->GetItem();

			TConstArrayView<ItemType> Items = OwnerTileView->GetItems();
			const int32 DragDroppedIndex = Items.Find(DragDropItem);
			if (DragDroppedIndex == INDEX_NONE)
			{
				// Item does not exist in the current tile view so it is always valid.
				return true;
			}

			if (DragDropOp->GetSourceTileView() != OwnerTileView)
			{
				// Don't alow drag from another tile view as the current tile view already contains the loudness metric.
				return false;
			}

			if (const ItemType* ItemPtr = Base::GetItemForThis(OwnerTileView))
			{
				const int32 DroppedOntoIndex = Items.Find(*ItemPtr);
				if (DroppedOntoIndex != INDEX_NONE && DragDroppedIndex != DroppedOntoIndex)
				{
					const bool bInsertAfter = (ItemHoverZone == EItemDropZone::BelowItem);
					const int32 InsertIndex = (bInsertAfter) ? DroppedOntoIndex + 1 : DroppedOntoIndex - 1;

					// Return valid if insert position causes change:
					return (InsertIndex != DragDroppedIndex);
				}
			}
			return false;
		}

		FReply AcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone ItemHoverZone)
		{
			if (TSharedPtr<DragDropOperationT> DragDropOp = DragDropEvent.GetOperationAs<DragDropOperationT>())
			{
				if (TSharedPtr<STileViewType> OwnerTileView = GetPinnedTileView())
				{
					if (const ItemType* ItemPtr = Base::GetItemForThis(OwnerTileView.ToSharedRef()))
					{
						const bool bInsertAfter = (ItemHoverZone == EItemDropZone::BelowItem);
						OwnerTileView->HandleItemDrop(*ItemPtr, bInsertAfter, DragDropOp->GetItem(), DragDropOp->GetSourceTileView());
						return FReply::Handled();
					}
				}
			}
			return FReply::Unhandled();
		}

		TSharedPtr<STileViewType> GetPinnedTileView()
		{
			return StaticCastSharedPtr<STileViewType>(Base::OwnerTablePtr.Pin());
		}

		EItemDropZone GetItemHoverZone(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
		{
			const FVector2f LocalPointerPos = MyGeometry.AbsoluteToLocal(DragDropEvent.GetScreenSpacePosition());
			return Base::ZoneFromPointerPosition(LocalPointerPos, MyGeometry.GetLocalSize(), EOrientation::Orient_Horizontal);
		}

		TWeakPtr<FDragDropOperation> WeakDragDropOperation; // Will be set when a drag is detected.
	};
} // namespace AudioWidgetsCore
