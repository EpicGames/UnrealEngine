// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMeterChannelInfo.generated.h"

USTRUCT(BlueprintType)
struct FAudioMeterChannelInfo
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AudioMeter)
	float MeterValue = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AudioMeter)
	float PeakValue = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AudioMeter)
	float ClippingValue = 0.0f;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = AudioMeter, meta = (ToolTip = "By default the meter value bar will be drawn from the minimum of the ValueRangeDb up to the MeterValue. Set this property to draw the meter value bar starting at a different value."))
	TOptional<float> MeterValueBarStart;
};

inline bool operator==(const FAudioMeterChannelInfo& Lhs, const FAudioMeterChannelInfo& Rhs)
{
	return FMath::IsNearlyEqual(Lhs.MeterValue, Rhs.MeterValue) &&
		   FMath::IsNearlyEqual(Lhs.PeakValue, Rhs.PeakValue)   &&
		   FMath::IsNearlyEqual(Lhs.ClippingValue, Rhs.ClippingValue) &&
		   ((!Lhs.MeterValueBarStart.IsSet() && !Rhs.MeterValueBarStart.IsSet()) || (Lhs.MeterValueBarStart.IsSet() && Rhs.MeterValueBarStart.IsSet() && FMath::IsNearlyEqual(*Lhs.MeterValueBarStart, *Rhs.MeterValueBarStart)));
}
