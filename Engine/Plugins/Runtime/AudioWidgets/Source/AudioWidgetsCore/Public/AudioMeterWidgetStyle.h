// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Fonts/SlateFontInfo.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/StyleDefaults.h"

#include "AudioMeterWidgetStyle.generated.h"

#define UE_API AUDIOWIDGETSCORE_API

struct FCompositeFont;

/**
 * Represents the appearance of an AudioMeter widget
 */
USTRUCT(BlueprintType)
struct FAudioMeterWidgetStyle : public FSlateWidgetStyle
{
	GENERATED_BODY()

	UE_API static const FAudioMeterWidgetStyle& GetDefault();

	UE_API virtual void GetResources(TArray<const FSlateBrush*>& OutBrushes) const override;
	virtual const FName GetTypeName() const override { return TypeName; };

	void UnlinkColors();

	inline static const FName TypeName = "FAudioMeterWidgetStyle";

	// Image to use to represent the meter value.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FSlateBrush MeterValueImage;
	FAudioMeterWidgetStyle& SetMeterValueImage(const FSlateBrush& InMeterValueImage){ MeterValueImage = InMeterValueImage; return *this; }

	// Image to use to represent the background.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush BackgroundImage;
	FAudioMeterWidgetStyle& SetBackgroundImage(const FSlateBrush& InBackgroundImage) { BackgroundImage = InBackgroundImage; return *this; }

	// Image to use to represent the meter background.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush MeterBackgroundImage;
	FAudioMeterWidgetStyle& SetMeterBackgroundImage(const FSlateBrush& InMeterBackgroundImage) { MeterBackgroundImage = InMeterBackgroundImage; return *this; }

	// Image to use to draw behind the meter value.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush MeterValueBackgroundImage;
	FAudioMeterWidgetStyle& SetMeterValueBackgroundImage(const FSlateBrush& InMeterValueBackgroundImage) { MeterValueBackgroundImage = InMeterValueBackgroundImage; return *this; }

	// Image to use to represent the meter peak.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush MeterPeakImage;
	FAudioMeterWidgetStyle& SetMeterPeakImage(const FSlateBrush& InMeterPeakImage) { MeterPeakImage = InMeterPeakImage; return *this; }

	// Image to use to represent the meter clipping indicator.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateBrush MeterClippingImage;
	FAudioMeterWidgetStyle& SetMeterClippingImage(const FSlateBrush& InMeterClippingImage) { MeterClippingImage = InMeterClippingImage; return *this; }

	// How thick to draw the meter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Appearance)
	FVector2D MeterSize = FVector2D(250.0f, 25.0f);
	FAudioMeterWidgetStyle& SetMeterSize(const FVector2D& InMeterSize) { MeterSize = InMeterSize; return *this; }

	// How much padding to add around the meter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FVector2D MeterPadding = FVector2D(10.0f, 5.0f);
	FAudioMeterWidgetStyle& SetMeterPadding(const FVector2D& InMeterPadding) { MeterPadding = InMeterPadding; return *this; }

	// How much padding to add around the meter value
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float MeterValuePadding = 3.0f;
	FAudioMeterWidgetStyle& SetMeterValuePadding(float InMeterValuePadding) { MeterValuePadding = InMeterValuePadding; return *this; }

	// How wide to draw the peak value indicator
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float PeakValueWidth = 2.0f;
	FAudioMeterWidgetStyle& SetPeakValueWidth(float InPeakValueWidth) { PeakValueWidth = InPeakValueWidth; return *this; }

	// How wide to draw the clipping value indicator
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float ClippingValueWidth = 2.0f;
	FAudioMeterWidgetStyle& SetClippingValueWidth(float InClippingValueWidth) { ClippingValueWidth = InClippingValueWidth; return *this; }

	// The minimum and maximum value to display in dB (values are clamped in this range)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FVector2D ValueRangeDb = FVector2D(-160, 10);
	FAudioMeterWidgetStyle& SetValueRangeDb(const FVector2D& InValueRangeDb) { ValueRangeDb = InValueRangeDb; return *this; }

	// Whether or not to show the decibel scale alongside the meter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	bool bShowScale = true;
	FAudioMeterWidgetStyle& SetShowScale(bool bInShowScale) { bShowScale = bInShowScale; return *this; }

	// Which side to show the scale. If vertical, true means left side, false means right side. If horizontal, true means above, false means below.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	bool bScaleSide = true;
	FAudioMeterWidgetStyle& SetScaleSide(bool bInScaleSide) { bScaleSide = bInScaleSide; return *this; }

	// Offset for the hashes
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float ScaleHashOffset = 5.0f;
	FAudioMeterWidgetStyle& SetScaleHashOffset(float InScaleHashOffset) { ScaleHashOffset = InScaleHashOffset; return *this; }

	// The width of each hash mark
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float ScaleHashWidth = 1.0f;
	FAudioMeterWidgetStyle& SetScaleHashWidth(float InScaleHashWidth) { ScaleHashWidth = InScaleHashWidth; return *this; }

	// The height of each hash mark
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	float ScaleHashHeight = 10.0f;
	FAudioMeterWidgetStyle& SetScaleHashHeight(float InScaleHashHeight) { ScaleHashHeight = InScaleHashHeight; return *this; }

	// How wide to draw the decibel scale, if it's enabled
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance, meta = (UIMin = "3", ClampMin="3", UIMax = "10"))
	int32 DecibelsPerHash = 10;
	FAudioMeterWidgetStyle& SetDecibelsPerHash(float InDecibelsPerHash) { DecibelsPerHash = InDecibelsPerHash; return *this; }

	/** Font family and size to be used when displaying the meter scale. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Appearance)
	FSlateFontInfo Font = FStyleDefaults::GetFontInfo(5);
	FAudioMeterWidgetStyle& SetFont(const FSlateFontInfo& InFont) { Font = InFont; return *this; }
	FAudioMeterWidgetStyle& SetFont(TSharedPtr<const FCompositeFont> InCompositeFont, const int32 InSize, const FName& InTypefaceFontName = NAME_None) { Font = FSlateFontInfo(InCompositeFont, InSize, InTypefaceFontName); return *this; }
	FAudioMeterWidgetStyle& SetFont(const UObject* InFontObject, const int32 InSize, const FName& InTypefaceFontName = NAME_None) { Font = FSlateFontInfo(InFontObject, InSize, InTypefaceFontName); return *this; }

	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMeterWidgetStyle& SetFont(const FName& InFontName, uint16 InSize);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMeterWidgetStyle& SetFont(const FString& InFontName, uint16 InSize);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMeterWidgetStyle& SetFont(const WIDECHAR* InFontName, uint16 InSize);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMeterWidgetStyle& SetFont(const ANSICHAR* InFontName, uint16 InSize);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMeterWidgetStyle& SetFontName(const FName& InFontName);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMeterWidgetStyle& SetFontName(const FString& InFontName);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMeterWidgetStyle& SetFontName(const WIDECHAR* InFontName);
	UE_DEPRECATED(5.6, "Use SetFont using FSlateFontInfo instead. FSlateFontInfo's constructors using a FontName are deprecated.")
	UE_API FAudioMeterWidgetStyle& SetFontName(const ANSICHAR* InFontName);

	FAudioMeterWidgetStyle& SetFontSize(uint16 InSize) { Font.Size = InSize; return *this; }
	FAudioMeterWidgetStyle& SetTypefaceFontName(const FName& InTypefaceFontName) { Font.TypefaceFontName = InTypefaceFontName; return *this; }
};


USTRUCT(BlueprintType)
struct FAudioMeterDefaultColorWidgetStyle : public FSlateWidgetStyle
{
	GENERATED_BODY()

	UE_API static const FAudioMeterDefaultColorWidgetStyle& GetDefault();

	virtual const FName GetTypeName() const override { return TypeName; };

	inline static const FName TypeName = "FAudioMeterDefaultColorWidgetStyle";

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style")
	FLinearColor MeterBackgroundColor = FLinearColor(0.031f, 0.031f, 0.031f, 1.0f);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style")
	FLinearColor MeterValueColor = FLinearColor(0.025719f, 0.208333f, 0.069907f, 1.0f);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style")
	FLinearColor MeterPeakColor = FLinearColor(0.24349f, 0.708333f, 0.357002f, 1.0f);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style")
	FLinearColor MeterClippingColor = FLinearColor::Transparent;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style")
	FLinearColor MeterScaleColor = FLinearColor(0.017642f, 0.017642f, 0.017642f, 1.0f);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Style")
	FLinearColor MeterScaleLabelColor = FLinearColor(0.442708f, 0.442708f, 0.442708f, 1.0f);
};

#undef UE_API
