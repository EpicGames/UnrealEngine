// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

#define UE_API AUDIOWIDGETSCORE_API

/**
 * Slate style set that defines all the styles for audio widgets core
 */
class FAudioWidgetsCoreStyle : public FSlateStyleSet
{
public:
	/** Access the singleton instance for this style set */
	UE_API static const FAudioWidgetsCoreStyle& Get();

	inline static const FName StyleName = "AudioWidgetsCoreStyle";

private:
	FAudioWidgetsCoreStyle();
	~FAudioWidgetsCoreStyle();
};

#undef UE_API
