// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/StyleColors.h"

#include "LoudnessMeterSettings.generated.h"

USTRUCT()
struct FLoudnessMeterDisplayOptions
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter)
	bool bShowValue = false;

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter)
	bool bShowMeter = false;

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter) 
	bool bHoldMaxForValue = false;

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter)
	bool bHoldMaxForMeter = false;

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter)
	FLinearColor Color = FStyleColors::AccentGreen.GetSpecifiedColor().CopyWithNewOpacity(0.5f);
};

USTRUCT()
struct FLoudnessMeterSettings
{
	GENERATED_BODY()

	static constexpr FLinearColor DefaultLoudnessRangeColor = FLinearColor(0.631f, 0.024f, 1.0f, 0.502f);
	static constexpr FLinearColor DefaultTruePeakColor      = FLinearColor(0.019f, 0.497f, 1.0f, 0.502f);

	static constexpr int32 LatestConfigVersion = 1;

	UPROPERTY(config)
	int32 ConfigVersion = 0;

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter, meta = (ClampMin = "10", ClampMax = "70", UIMin = "10", UIMax = "70"))
	int LoudnessScaleRange = 60;

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter, meta = (ClampMin = "-10", ClampMax = "30", UIMin = "-10", UIMax = "30"))
	int LoudnessScaleOffset = 0;

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter, meta = (ClampMin = "-60", ClampMax = "30", UIMin = "-60", UIMax = "30"))
	int LoudnessScaleTarget = -23;

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter, meta = (ClampMin = "-60.0", ClampMax = "30.0", UIMin = "-60.0", UIMax = "30.0"))
	float TruePeakLimit = -1.0f;

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter)
	FLinearColor TargetColor = FStyleColors::AccentOrange.GetSpecifiedColor();

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter)
	bool bUseRelativeLoudnessScale = false;

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter)
	bool bDisplayAnalysisTimer = true;

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter)
	FLoudnessMeterDisplayOptions LongTermLoudness = { .bShowValue = true, .bShowMeter = false };

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter)
	FLoudnessMeterDisplayOptions ShortTermLoudness = { .bShowValue = false, .bShowMeter = true };

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter)
	FLoudnessMeterDisplayOptions MomentaryLoudness = { .bShowValue = false, .bShowMeter = true };

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter)
	FLoudnessMeterDisplayOptions LoudnessRange = { .bShowValue = true, .bShowMeter = false, .Color = DefaultLoudnessRangeColor };

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter)
	FLoudnessMeterDisplayOptions TruePeak = { .bShowValue = true, .bShowMeter = false, .bHoldMaxForValue = true, .bHoldMaxForMeter = true, .Color = DefaultTruePeakColor };

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter, meta = (ClampMin = "0", ClampMax = "119", UIMin = "0", UIMax = "119"))
	uint64 ValuesOrderingPermutation = 0; // Max == 5! - 1

	UPROPERTY(EditAnywhere, config, Category = LoudnessMeter, meta = (ClampMin = "0", ClampMax = "119", UIMin = "0", UIMax = "119"))
	uint64 MetersOrderingPermutation = 0; // Max == 5! - 1
};
