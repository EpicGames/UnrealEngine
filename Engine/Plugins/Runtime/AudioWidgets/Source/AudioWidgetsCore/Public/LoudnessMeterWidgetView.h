// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ObservableArray.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SWidget.h"
#include "Widgets/SWindow.h"

#define UE_API AUDIOWIDGETSCORE_API

namespace AudioWidgetsCore
{
	enum class EAudioMeterMetric : uint8
	{
		Loudness,
		Decibels
	};

	/**
	 * A widget that displays various loudness metrics as numeric values and/or meters.
	 */
	class FLoudnessMeterWidgetView
	{
	public:
		UE_API static FLinearColor GetDefaultMeterColor();
		UE_API static FLinearColor GetDefaultMeterColor(const FName& InLoudnessMeterName);
		UE_API static FLinearColor GetDefaultTargetColor();

		struct FLoudnessMetric
		{
			FName Name;
			FText DisplayName;
			TAttribute<EAudioMeterMetric> MeterMetric;
			TAttribute<TOptional<float>> Value;
			TAttribute<TOptional<float>> MaxValue;
			TAttribute<TOptional<FFloatInterval>> Range;
			TAttribute<bool> bShowValue = false;
			TAttribute<bool> bShowMeter = false;
			TAttribute<bool> bHoldMaxForValue = false;
			TAttribute<bool> bHoldMaxForMeter = false;
			FSimpleDelegate OnShowValueToggleRequested;
			FSimpleDelegate OnShowMeterToggleRequested;
			FSimpleDelegate OnShowValueToggleFromDragDropRequested;
			FSimpleDelegate OnShowMeterToggleFromDragDropRequested;
			FSimpleDelegate OnHoldMaxForValueToggleRequested;
			FSimpleDelegate OnHoldMaxForMeterToggleRequested;
			TAttribute<FLinearColor> Color = GetDefaultMeterColor();
			FOnLinearColorValueChanged OnColorChanged;
			FOnWindowClosed OnColorPickerWindowClosed;
			FSimpleDelegate OnResetColorRequested;
			FSimpleDelegate OnClicked;

			bool IsRanged() const { return Range.IsBound() || Range.Get(NullOpt).IsSet(); }
		};

		struct FTimerPanelParams
		{
			TAttribute<FTimespan> AnalysisTime = FTimespan::Zero();
			FOnClicked OnResetButtonClicked;
			TAttribute<bool> bIsResetButtonEnabled = true;
			TAttribute<bool> bIsVisible = false;
			FSimpleDelegate OnVisibilityToggleRequested;
		};

		DECLARE_DELEGATE_OneParam(FOnIntValueChanged, int32 /*NewValue*/);
		DECLARE_DELEGATE_TwoParams(FOnIntValueCommitted, int32 /*NewValue*/, ETextCommit::Type /*CommitType*/);

		DECLARE_DELEGATE_OneParam(FOnFloatValueChanged, float /*NewValue*/);
		DECLARE_DELEGATE_TwoParams(FOnFloatValueCommitted, float /*NewValue*/, ETextCommit::Type /*CommitType*/);

		struct FLoudnessScaleParams
		{
			TAttribute<int32> Range = 60;
			FOnIntValueChanged OnRangeValueChanged;
			FOnIntValueCommitted OnRangeValueCommitted;

			TAttribute<int32> Offset = 0;
			FOnIntValueChanged OnOffsetValueChanged;
			FOnIntValueCommitted OnOffsetValueCommitted;

			TAttribute<int32> Target = -23;
			FOnIntValueChanged OnTargetValueChanged;
			FOnIntValueCommitted OnTargetValueCommitted;

			// Using float here to have a more fine grained representation of true peak in dB where fractional precision matters (unlike LUFS values).
			TAttribute<float> TruePeakLimit = -1.0f; 
			FOnFloatValueChanged OnTruePeakLimitValueChanged;
			FOnFloatValueCommitted OnTruePeakLimitValueCommitted;

			TAttribute<FLinearColor> TargetColor = GetDefaultTargetColor();
			FOnLinearColorValueChanged OnTargetColorChanged;
			FOnWindowClosed OnTargetColorPickerWindowClosed;
			FSimpleDelegate OnResetTargetColorRequested;

			TAttribute<bool> bRelativeScale = false;
			FSimpleDelegate OnRelativeScaleToggleRequested;
		};

		DECLARE_DELEGATE_OneParam(FOnPermutationChanged, uint64 /*NewPermutationIndex*/);

		struct FDisplayOrderParams
		{
			TAttribute<uint64> PermutationIndex = 0;
			FOnPermutationChanged OnPermutationChanged;
		};

		UE_API FLoudnessMeterWidgetView();
		UE_API ~FLoudnessMeterWidgetView();

		UE_API void InitTimerPanel(const FTimerPanelParams& InTimerPanelParams);
		UE_API void InitLoudnessScale(const FLoudnessScaleParams& InLoudnessScaleParams);
		UE_API void InitValuesDisplayOrder(const FDisplayOrderParams& InDisplayOrderParams);
		UE_API void InitMetersDisplayOrder(const FDisplayOrderParams& InDisplayOrderParams);

		UE_API TSharedRef<SWidget> MakeWidget() const;

		UE_API void AddLoudnessMetric(const FLoudnessMetric& LoudnessMetric);
		UE_API void RefreshVisibleLoudnessMetrics();

	private:
		using FLoudnessMetricRef = TSharedRef<const FLoudnessMetric>;
		using FLoudnessMetricRefArray = UE::Slate::Containers::TObservableArray<FLoudnessMetricRef>;

		class FLoudnessMetricCollectionView : public TSharedFromThis<FLoudnessMetricCollectionView>
		{
		public:
			FLoudnessMetricCollectionView(TSharedRef<FLoudnessMetricRefArray> InLoudnessMetrics, TFunction<bool(const FLoudnessMetricRef&)> InVisibilityPredicate);

			void SetDisplayOrderParams(const FDisplayOrderParams& InDisplayOrderParams) { DisplayOrderParams = InDisplayOrderParams; }
			void RefreshVisibleItems();
			void ResetVisibleItems() { VisibleItems->Reset(); }
			TSharedRef<FLoudnessMetricRefArray> GetVisibleItems() { return VisibleItems; }
			bool HasVisibleItems() const { return !VisibleItems->IsEmpty(); }

			void DraggedItemReorder(const FLoudnessMetricRef& ItemDroppedOnto, bool bInsertAfter, const FLoudnessMetricRef& DragDropItem);

		private:
			static uint64 GetPermutationIndex(TArrayView<uint8> PermutationArray);
			static TArray<uint8> GetPermutationArray(uint64 PermutationIndex, uint8 NumValues);

			const TSharedRef<FLoudnessMetricRefArray> LoudnessMetrics;
			const TFunction<bool(const FLoudnessMetricRef&)> VisibilityPredicate;
			const TSharedRef<FLoudnessMetricRefArray> VisibleItems;
			FDisplayOrderParams DisplayOrderParams;
		};

		TSharedRef<FLoudnessMetricRefArray> LoudnessMetrics;
		TSharedRef<FLoudnessMetricCollectionView> ValuesCollectionView;
		TSharedRef<FLoudnessMetricCollectionView> MetersCollectionView;
		FTimerPanelParams TimerPanelParams;
		FLoudnessScaleParams LoudnessScaleParams;
	};
} // namespace AudioWidgetsCore

#undef UE_API
