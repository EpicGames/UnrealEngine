// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMeterChannelInfo.h"
#include "AudioMeterWidgetStyle.h"
#include "Math/Color.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SLeafWidget.h"

#define UE_API AUDIOWIDGETSCORE_API

class FPaintArgs;
class FSlateWindowElementList;

class SAudioMeterWidgetBase : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SAudioMeterWidgetBase){}
	SLATE_END_ARGS()

	virtual TArray<FAudioMeterChannelInfo> GetMeterChannelInfo() const = 0;
	virtual void SetMeterChannelInfo(const TAttribute<TArray<FAudioMeterChannelInfo>>& InMeterChannelInfo) = 0;

	// The active timer registered to refresh the meter channel info
	bool bIsActiveTimerRegistered = false;
};

/**
 * A widget that displays audio channel's levels.
 */
class SAudioMeterWidget : public SAudioMeterWidgetBase
{
public:
	SLATE_BEGIN_ARGS(SAudioMeterWidget)
		: _Orientation(EOrientation::Orient_Horizontal)
		, _BackgroundColor(FLinearColor::Black)
		, _MeterBackgroundColor(FLinearColor::Gray)
		, _MeterValueColor(FLinearColor::Green)
		, _MeterPeakColor(FLinearColor::Blue)
		, _MeterClippingColor(FLinearColor::Transparent)
		, _MeterScaleColor(FLinearColor::White)
		, _MeterScaleLabelColor(FLinearColor::Gray)
		, _Style(&FAudioMeterWidgetStyle::GetDefault())
	{
	}

		// The audio meter's orientation
		SLATE_ARGUMENT(EOrientation, Orientation)

		// The color to draw the background in
		SLATE_ATTRIBUTE(FSlateColor, BackgroundColor)

		// The color to draw the meter background in
		SLATE_ATTRIBUTE(FSlateColor, MeterBackgroundColor)

		// The color to draw the meter value in
		SLATE_ATTRIBUTE(FSlateColor, MeterValueColor)

		// The color to draw the meter peak
		SLATE_ATTRIBUTE(FSlateColor, MeterPeakColor)

		// The color to draw the clipping value in
		SLATE_ATTRIBUTE(FSlateColor, MeterClippingColor)

		// The color to draw the scale in
		SLATE_ATTRIBUTE(FSlateColor, MeterScaleColor)

		// The color to draw the scale in
		SLATE_ATTRIBUTE(FSlateColor, MeterScaleLabelColor)

		// The style used to draw the slider
		SLATE_STYLE_ARGUMENT(FAudioMeterWidgetStyle, Style)

		// A value representing the audio meter value
		SLATE_ATTRIBUTE(TArray<FAudioMeterChannelInfo>, MeterChannelInfo)

	SLATE_END_ARGS()

	UE_API void Construct(const SAudioMeterWidget::FArguments& InDeclaration);

	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API virtual FVector2D ComputeDesiredSize(float) const override;
	UE_API virtual bool ComputeVolatility() const override;

	UE_API virtual TArray<FAudioMeterChannelInfo> GetMeterChannelInfo() const override;
	UE_API virtual void SetMeterChannelInfo(const TAttribute<TArray<FAudioMeterChannelInfo>>& InMeterChannelInfo) override;

	UE_API void SetOrientation(EOrientation InOrientation);
	UE_API void SetBackgroundColor(FSlateColor InBackgroundColor);
	UE_API void SetMeterBackgroundColor(FSlateColor InMeterBackgroundColor);
	UE_API void SetMeterValueColor(FSlateColor InMeterValueColor);
	UE_API void SetMeterPeakColor(FSlateColor InMeterPeakColor);
	UE_API void SetMeterClippingColor(FSlateColor InMeterPeakColor);
	UE_API void SetMeterScaleColor(FSlateColor InMeterScaleColor);
	UE_API void SetMeterScaleLabelColor(FSlateColor InMeterScaleLabelColor);

	UE_API float GetScaleHeight() const;

protected:
	const FAudioMeterWidgetStyle* Style = nullptr;

	// Audio meter's orientation
	EOrientation Orientation = EOrientation::Orient_Vertical;

	// Colors
	TAttribute<FSlateColor> BackgroundColor;
	TAttribute<FSlateColor> MeterBackgroundColor;
	TAttribute<FSlateColor> MeterValueColor;
	TAttribute<FSlateColor> MeterPeakColor;
	TAttribute<FSlateColor> MeterClippingColor;
	TAttribute<FSlateColor> MeterScaleColor;
	TAttribute<FSlateColor> MeterScaleLabelColor;

	TAttribute<TArray<FAudioMeterChannelInfo>> MeterChannelInfoAttribute;
};

#undef UE_API
