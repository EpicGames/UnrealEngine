// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioWidgetsEditorModule.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "SoundBaseDetails.h"

void FAudioWidgetsEditorModule::StartupModule()
{
	RegisterCustomClassLayout("SoundBase", FOnGetDetailCustomizationInstance::CreateStatic(&FSoundBaseDetails::MakeInstance));

}

void FAudioWidgetsEditorModule::ShutdownModule()
{
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		// Unregister all classes customized by name
		for (auto It = RegisteredClassNames.CreateConstIterator(); It; ++It)
		{
			if (It->IsValid())
			{
				PropertyModule.UnregisterCustomClassLayout(*It);
			}
		}
	}
}

void FAudioWidgetsEditorModule::RegisterCustomClassLayout(FName ClassName, FOnGetDetailCustomizationInstance DetailLayoutDelegate )
{
	check(ClassName != NAME_None);

	RegisteredClassNames.Add(ClassName);

	const FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);
	PropertyModule.RegisterCustomClassLayout( ClassName, DetailLayoutDelegate );
}

	
IMPLEMENT_MODULE(FAudioWidgetsEditorModule, AudioWidgetsEditor)