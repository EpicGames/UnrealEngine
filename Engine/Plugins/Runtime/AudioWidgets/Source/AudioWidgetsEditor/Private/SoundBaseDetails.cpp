// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundBaseDetails.h"

#include "IAudioPropertiesDetailsInjector.h"
#include "DetailLayoutBuilder.h"
#include "Features/IModularFeatures.h"
#include "IPropertyUtilities.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "Sound/AudioSettings.h"
#include "Sound/SoundBase.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"

TSharedRef<IDetailCustomization> FSoundBaseDetails::MakeInstance()
{
	return MakeShareable(new FSoundBaseDetails);
}

FSoundBaseDetails::~FSoundBaseDetails()
{
}

void FSoundBaseDetails::PendingDelete()
{
	if (AudioPropertiesInjector.IsValid())
	{
		AudioPropertiesInjector->UnbindFromPropertySheetChanges();
	}
}

void FSoundBaseDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	if (!FModuleManager::Get().IsModuleLoaded("AudioProperties") || !FModuleManager::Get().IsModuleLoaded("AudioPropertiesEditor"))
	{
		DetailBuilder.HideCategory("AudioProperties");
	}
	else
	{
		InjectPropertySheetView(DetailBuilder);
	}
}

void FSoundBaseDetails::InjectPropertySheetView(IDetailLayoutBuilder& DetailBuilder)
{
	check(FModuleManager::Get().IsModuleLoaded("AudioPropertiesEditor"))
	
	IModularFeatures::FScopedLockModularFeatureList ScopedLockModularFeatureList;
	TArray<IAudioPropertiesDetailsInjectorBuilder*> AvailableDetailsInjectorBuilders = IModularFeatures::Get().GetModularFeatureImplementations<IAudioPropertiesDetailsInjectorBuilder>(AudioPropertiesDetailsInjector::BuilderModularFeatureName);

	if(IAudioPropertiesDetailsInjectorBuilder* Builder = AvailableDetailsInjectorBuilders[0])
	{
		AudioPropertiesInjector = TSharedPtr<IAudioPropertiesDetailsInjector>(Builder->CreateAudioPropertiesDetailsInjector());
	}
	
	if (!AudioPropertiesInjector || !AudioPropertiesInjector.IsValid())
	{
		return;
	}

	TSharedRef<IPropertyHandle> PropertySheetHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(USoundBase, AudioPropertiesSheet));

	if (!PropertySheetHandle->IsValidHandle())
	{
		UE_LOGF(LogInit, Log, "Invalid Property Sheet Handle found when customizing SoundBase details, property sheet view will not be injected")
		return;
	}

	AudioPropertiesInjector->CustomizeInjectedPropertiesDetails(DetailBuilder, PropertySheetHandle);
	AudioPropertiesInjector->BindDetailCustomizationToPropertySheetChanges(DetailBuilder, PropertySheetHandle);
}