// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Modules/ModuleInterface.h"
#include "PropertyEditorDelegates.h"
#include "UObject/NameTypes.h"

class FAudioWidgetsEditorModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
private:
	void RegisterCustomClassLayout(FName ClassName, FOnGetDetailCustomizationInstance DetailLayoutDelegate);

	/** List of registered class that we must unregister when the module shuts down */
	TSet<FName> RegisteredClassNames;
};